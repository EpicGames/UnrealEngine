// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetDefinition.h"
#include "CoreMinimal.h"
#include "GraphEditor.h"
#include "PropertyPath.h"

class IDetailsView;
class SBox;
class SKismetInspector;
class SMaterialEditor3DPreviewViewport;
class SMaterialEditorUIPreviewViewport;
class SMyBlueprint;
class UMaterial;
class UMaterialExpression;
class UMaterialGraph;

class UMaterialInstance;
class UMaterialEditorInstanceConstant;

struct FRevisionInfo;

/** Panel used to display the MaterialGraph */
struct FMaterialDiffPanel : public FGCObject
{
	/** Generate a panel for NewGraph diffed against OldGraph */
	void GeneratePanel(UEdGraph* NewGraph, UEdGraph* OldGraph);

	/** Generate a panel that displays the Graph and reflects the items in the DiffResults */
	void GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults, TAttribute<int32> FocusedDiffResult);

	/** Called when user hits keyboard shortcut to copy nodes */
	void CopySelectedNodes();

	/** Gets whatever nodes are selected in the Graph Editor */
	FGraphPanelSelectionSet GetSelectedNodes() const;

	/** Can user copy any of the selected nodes? */
	bool CanCopyNodes() const;

	void SelectDownstreamNodes() const;
	void SelectUpstreamNodes() const;
	
	void OnPreviewNode();

	/** Functions used to focus/find a particular change in a diff result */
	void FocusDiff(UEdGraphPin& Pin);
	void FocusDiff(UEdGraphNode& Node);

	TSharedRef<SWidget> GetMaterialNodeDetailsViewWidget() const;
	
	void SetViewportToDisplay();
	
	TSharedRef<SWidget> GetViewportToDisplay() const;
	
	void SetViewportVisibility(bool bShowViewport);

	void SetPreviewExpression(UMaterialExpression* NewPreviewExpression);

	void SetPreviewMaterial(UMaterialInterface* InMaterialInterface) const;

	void UpdatePreviewMaterial() const;

	/** The MaterialGraph we are showing */
	TObjectPtr<UMaterialGraph> MaterialGraph = nullptr;

	/** The box around the graph editor, used to change the content when new graphs are set */
	TSharedPtr<SBox> GraphEditorBox;

	/** The details view associated with the MaterialNode of the focused MaterialGraphNode */
	TSharedPtr<IDetailsView> MaterialNodeDetailsView;

	TSharedPtr<SMaterialEditor3DPreviewViewport> Preview3DViewport;

	TSharedPtr<SMaterialEditorUIPreviewViewport> Preview2DViewport;

	TObjectPtr<UMaterial> ExpressionPreviewMaterial = nullptr;

	TObjectPtr<UMaterialExpression> PreviewExpression = nullptr;

	/** The graph editor which does the work of displaying the graph */
	TWeakPtr<SGraphEditor> GraphEditor;

	/** Revision information for this Material */
	FRevisionInfo RevisionInfo;

	/** True if we should show a name identifying which asset this panel is displaying */
	bool bShowAssetName = true;

	/** The widget that contains the revision info in graph mode */
	TSharedPtr<SWidget> OverlayGraphRevisionInfo;

	/** The property that we will highlight in MaterialNodeDetailsView */
	FPropertyPath PropertyToFocus;

	/** The material instance being diffed (nullptr when diffing base materials) */
	TObjectPtr<UMaterialInstance> MaterialInstance = nullptr;

	/** Editor proxy used to display all parameter types including static switches (nullptr when diffing base materials) */
	TObjectPtr<UMaterialEditorInstanceConstant> EditorProxy = nullptr;
	
	// Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	// End FGCObject interface

private:
	/** Command list for this diff panel */
	TSharedPtr<FUICommandList> GraphEditorCommands;
};