// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition.h"
#include "CoreMinimal.h"
#include "DiffControl.h"
#include "DiffTool/MaterialDiffPanel.h"
#include "SBlueprintDiff.h"
#include "Widgets/SCompoundWidget.h"
#include "Materials/MaterialInstance.h"

#define UE_API MATERIALEDITOR_API

class FBlueprintDifferenceTreeEntry;
class UEdGraph;
class UMaterialGraph;

class IDetailsView;
class UMaterialEditorInstanceConstant;

struct FMaterialToDiff;

enum class EAssetEditorCloseReason : uint8;

struct FMaterialDiffResultItem : public FDiffResultItem
{
	/** Not empty if the DiffResultItem is a property changed */
	FPropertyPath Property;
};

/* Visual Diff between two Material Graphs */
class SMaterialDiff : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialDiff) {}
		SLATE_ARGUMENT(TObjectPtr<UMaterialGraph>, OldMaterialGraph)
		SLATE_ARGUMENT(TObjectPtr<UMaterialGraph>, NewMaterialGraph)
		SLATE_ARGUMENT(FRevisionInfo, OldRevision)
		SLATE_ARGUMENT(FRevisionInfo, NewRevision)
		SLATE_ARGUMENT(bool, ShowAssetNames)
		SLATE_ARGUMENT(TObjectPtr<UMaterialInstance>, OldMaterialInstance)
		SLATE_ARGUMENT(TObjectPtr<UMaterialInstance>, NewMaterialInstance)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual ~SMaterialDiff() override;

	/** Called when a new Graph is clicked on by user */
	UE_API void OnGraphChanged(FMaterialToDiff* Diff);

	/** Called when user clicks on a new graph list item */
	UE_API void OnGraphSelectionChanged(TSharedPtr<FMaterialToDiff> Item, ESelectInfo::Type SelectionType);

	/** Called when user clicks on an entry in the listview of differences */
	UE_API void OnDiffListSelectionChanged(TSharedPtr<FMaterialDiffResultItem> TheDiff);

	/** Helper function for generating an empty widget */
	static UE_API TSharedRef<SWidget> DefaultEmptyPanel();

	/** Helper function to create a window that holds a diff widget */
	static UE_API TSharedPtr<SWindow> CreateDiffWindow(FText WindowTitle, TObjectPtr<UMaterialGraph> OldMaterialGraph, TObjectPtr<UMaterialGraph> NewMaterialGraph, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision);

	/** Helper function to create a window that holds a diff widget (default window title) */
	static UE_API TSharedPtr<SWindow> CreateDiffWindow(TObjectPtr<UMaterialGraph> OldMaterialGraph, TObjectPtr<UMaterialGraph> NewMaterialGraph, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* ObjectClass);

	/** Create a diff window for two material instances */
	static UE_API TSharedPtr<SWindow> CreateDiffWindow(FText WindowTitle, TObjectPtr<UMaterialInstance> OldMaterialInstance, TObjectPtr<UMaterialInstance> NewMaterialInstance, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision);

	/** Create a diff window for two material instances (default window title) */
	static UE_API TSharedPtr<SWindow> CreateDiffWindow(TObjectPtr<UMaterialInstance> OldMaterialInstance, TObjectPtr<UMaterialInstance> NewMaterialInstance, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* ObjectClass);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWindowClosedEvent, TSharedRef<SMaterialDiff>)
	FOnWindowClosedEvent OnWindowClosedEvent;

protected:
	/** Called when user clicks button to go to next difference */
	UE_API void NextDiff();

	/** Called when user clicks button to go to prev difference */
	UE_API void PrevDiff();

	/** Called to determine whether we have a list of differences to cycle through */
	UE_API bool HasNextDiff() const;
	UE_API bool HasPrevDiff() const;

	/** User toggles the option to show/hide preview viewports */
	UE_API void ToggleViewport();

	/** Get the image to show for the toggle viewport option */
	UE_API FSlateIcon GetViewportImage() const;

	/** Find the FMaterialToDiff that displays the graph with GraphPath relative path */
	UE_API FMaterialToDiff* FindGraphToDiffEntry(const FString& GraphPath);

	/** Bring these revisions of graph into focus on main display */
	UE_API void FocusOnMaterialGraphRevisions(FMaterialToDiff* Diff);

	/** User toggles the option to lock the views between the two material graphs */
	UE_API void OnToggleLockView();

	/** User toggles the option to change the split view mode betwwen vertical and horizontal */
	UE_API void OnToggleSplitViewMode();

	/** Get the image to show for the toggle lock option */
	UE_API FSlateIcon GetLockViewImage() const;

	/** Get the image to show for the toggle split view mode option */
	UE_API FSlateIcon GetSplitViewModeImage() const;

	/** Reset the graph editor, called when user switches graphs to display */
	UE_API void ResetGraphEditors();

	/** Material Graph to diff, is added to panel */
	TSharedPtr<FMaterialToDiff> MaterialGraphToDiff;

	/** Get Graph editor associated with this Graph */
	UE_API FMaterialDiffPanel& GetDiffPanelForNode(UEdGraphNode& Node);

	/** Event handler that updates the graph view when user selects a new graph */
	UE_API void HandleGraphChanged(const FString& GraphPath);

	/** Function used to generate the list of differences and the widgets needed to calculate that list */
	UE_API void GenerateDifferencesList();

	/** Called when editor may need to be closed */
	UE_API void OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason);

	struct FMaterialGraphDiffControl
	{
		FMaterialGraphDiffControl() : Widget(), DiffControl(nullptr)
		{
		}

		TSharedPtr<SWidget> Widget;
		TSharedPtr<IDiffControl> DiffControl;
	};

	UE_API FMaterialGraphDiffControl GenerateMaterialGraphPanel();

	UE_API TSharedRef<SOverlay> GenerateMaterialGraphWidgetForPanel(FMaterialDiffPanel& OutDiffPanel) const;
	UE_API TSharedRef<SBox> GenerateRevisionInfoWidgetForPanel(TSharedPtr<SWidget>& OutGeneratedWidget, const FText& InRevisionText) const;

	/** Accessor and event handler for toggling between diff view modes - only GraphMode for now: */
	UE_API void SetCurrentMode(FName NewMode);

	FName GetCurrentMode() const { return CurrentMode; }

	UE_API void OnModeChanged(const FName& InNewViewMode) const;

	UE_API void UpdateTopSectionVisibility(const FName& InNewViewMode) const;

	UE_API void SetCurrentWidgetIndex(int32 Index);
	UE_API int32 GetCurrentWidgetIndex() const;

	FName CurrentMode;

	FMaterialDiffPanel PanelOld, PanelNew;

	/** If the two views should be locked */
	bool bLockViews = true;

	/** If the view on Graph Mode should be divided vertically */
	bool bVerticalSplitGraphMode = true;

	/** If the preview Viewports should be shown */
	bool bShowViewport = true;

	/** Should we show the graph diff? */
	bool bShowGraphDiff = true;

	/** Contents widget that we swap when mode changes (defaults, components, etc) */
	TSharedPtr<SBox> ModeContents;

	TSharedPtr<SSplitter> TopRevisionInfoWidget;

	TSharedPtr<SSplitter> DiffGraphSplitter;

	TSharedPtr<SSplitter> GraphToolBarWidget;

	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;

	int32 CurrentWidgetIndex = 0;

	/** Tree of differences collected across all panels: */
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> PrimaryDifferencesList;

	/** List of all differences, cached so that we can iterate only the differences and not labels, etc: */
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> RealDifferences;

	/** Tree view that displays the differences, cached for the buttons that iterate the differences: */
	TSharedPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> DifferencesTreeView;

	/** Stored references to widgets used to display various parts of a material, from the mode name */
	TMap<FName, FMaterialGraphDiffControl> ModePanels;

	/** A pointer to the window holding this */
	TWeakPtr<SWindow> WeakParentWindow;

	FDelegateHandle AssetEditorCloseDelegate;

private:
	/** Registers FMaterialInstanceParameterDetails customization on instance diff details views */
	void AddMaterialInstanceDetailCustomization(const TSharedRef<IDetailsView>& DetailsView);
};

#undef UE_API
