// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MaterialEditor : ModuleRules
{
	public MaterialEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(
			new string[] {
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] 
			{
				"AssetTools",
				"EditorWidgets",
				"MessageLog",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AppFramework",
				"ContentBrowser",
				"ContentBrowserData",
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
                "EditorStyle",
				"RenderCore",
				"RHI",
                "MaterialUtilities",
                "PropertyEditor",
				"EditorFramework",
				"UnrealEd",
				"GraphEditor",
                "AdvancedPreviewScene",
                "Projects",
                "AssetRegistry",
				"ToolMenus",
				"ToolWidgets",
				"MainFrame",
				"Landscape",
				"Kismet", // for rich diffing machinery which was mostly implemented in the kismet module
				"SourceControl",
				"EditorSubsystem",
				"PhysicsCore"
            }
        );

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"SceneOutliner",
				"ClassViewer",
				"WorkspaceMenuStructure"
			}
		);
	}
}
