// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffTool/MaterialDiffPanel.h"

#include "Components/MeshComponent.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphDiffControl.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "MaterialDomain.h"
#include "MaterialEditorActions.h"
#include "MaterialShared.h"
#include "MaterialEditor/PreviewMaterial.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "SMaterialEditorViewport.h"
#include "SMyBlueprint.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "MaterialDiffPanel"

DEFINE_LOG_CATEGORY_STATIC(LogMaterialEditorDiff, Log, All);

void FMaterialDiffPanel::GeneratePanel(UEdGraph* NewGraph, UEdGraph* OldGraph)
{
	const TSharedPtr<TArray<FDiffSingleResult>> Diff = MakeShared<TArray<FDiffSingleResult>>();
	FGraphDiffControl::DiffGraphs(OldGraph, NewGraph, *Diff);
	GeneratePanel(NewGraph, Diff, TAttribute<int32>{});
}

void FMaterialDiffPanel::GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults, TAttribute<int32> FocusedDiffResult)
{
	if (GraphEditor.IsValid() && GraphEditor.Pin()->GetCurrentGraph() == Graph)
	{
		return;
	}

	// clang-format off
	TSharedPtr<SWidget> Widget = SNew(SBorder)
									 .HAlign(HAlign_Center)
									 .VAlign(VAlign_Center)
									 [
										SNew(STextBlock)
										 .Text(LOCTEXT("MaterialGraphDiffPanelNoGraphTip", "Graph does not exist in this revision"))
									 ];
	// clang-format on

	if (Graph)
	{
		SGraphEditor::FGraphEditorEvents InEvents;
		{
			const auto SelectionChangedHandler = [this](const FGraphPanelSelectionSet& SelectionSet) {
				if (MaterialNodeDetailsView)
				{
					if (SelectionSet.Array().Num() == 1)
					{
						if (UMaterialGraphNode_Base* MaterialNode = Cast<UMaterialGraphNode_Base>(SelectionSet.Array()[0]))
						{
							MaterialNodeDetailsView->SetObject(MaterialNode->GetMaterialNodeOwner());
							MaterialNodeDetailsView->HighlightProperty(PropertyToFocus);
							MaterialNodeDetailsView->SetVisibility(EVisibility::Visible);
							MaterialNodeDetailsView->SetIsPropertyEditingEnabledDelegate(
								FIsPropertyEditingEnabled::CreateLambda([]() { return false; }));
						}
						else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(SelectionSet.Array()[0]))
						{
							MaterialNodeDetailsView->SetObject(CommentNode);
							MaterialNodeDetailsView->SetVisibility(EVisibility::Visible);
						}
						else
						{
							MaterialNodeDetailsView->SetVisibility(EVisibility::Hidden);
						}
					}
					else
					{
						MaterialNodeDetailsView->SetVisibility(EVisibility::Hidden);
					}
				}
			};

			const auto ContextMenuHandler = [this](UEdGraph* CurrentGraph, const UEdGraphNode* InGraphNode, const UEdGraphPin* InGraphPin, FMenuBuilder* MenuBuilder, bool bIsDebugging) {
				if (MenuBuilder)
				{
					MenuBuilder->AddMenuEntry(FGenericCommands::Get().Copy);
					MenuBuilder->AddMenuEntry(FMaterialEditorCommands::Get().SelectDownstreamNodes);
					MenuBuilder->AddMenuEntry(FMaterialEditorCommands::Get().SelectUpstreamNodes);
					if (const UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(InGraphNode))
					{
						if (UMaterialExpression* MaterialExpression = MaterialGraphNode->MaterialExpression)
						{
							// Don't show preview option for bools
							if (!MaterialExpression->IsA(UMaterialExpressionStaticBool::StaticClass()) && !MaterialExpression->IsA(UMaterialExpressionStaticBoolParameter::StaticClass()))
							{
								// Add a preview node option if only one node is selected
								if (MaterialGraphNode->MaterialExpression == PreviewExpression)
								{
									// If we are already previewing the selected node, the menu option should tell the user that this will stop previewing
									MenuBuilder->AddMenuEntry(FMaterialEditorCommands::Get().StopPreviewNode);
								}
								else
								{
									// The menu option should tell the user this node will be previewed.
									MenuBuilder->AddMenuEntry(FMaterialEditorCommands::Get().StartPreviewNode);
								}
							}
						}
					}
					
					return FActionMenuContent(MenuBuilder->MakeWidget());
				}

				return FActionMenuContent();
			};

			InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateLambda(SelectionChangedHandler);
			InEvents.OnCreateNodeOrPinMenu = SGraphEditor::FOnCreateNodeOrPinMenu::CreateLambda(ContextMenuHandler);
		}

		if (!GraphEditorCommands.IsValid())
		{
			GraphEditorCommands = MakeShared<FUICommandList>();

			GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
										   FExecuteAction::CreateRaw(this, &FMaterialDiffPanel::CopySelectedNodes),
										   FCanExecuteAction::CreateRaw(this, &FMaterialDiffPanel::CanCopyNodes));
			GraphEditorCommands->MapAction(FMaterialEditorCommands::Get().SelectDownstreamNodes,
										   FExecuteAction::CreateRaw(this, &FMaterialDiffPanel::SelectDownstreamNodes));
			GraphEditorCommands->MapAction(FMaterialEditorCommands::Get().SelectUpstreamNodes,
										   FExecuteAction::CreateRaw(this, &FMaterialDiffPanel::SelectUpstreamNodes));
			GraphEditorCommands->MapAction(FMaterialEditorCommands::Get().StartPreviewNode,
										   FExecuteAction::CreateRaw(this, &FMaterialDiffPanel::OnPreviewNode));
			GraphEditorCommands->MapAction(FMaterialEditorCommands::Get().StopPreviewNode,
										   FExecuteAction::CreateRaw(this, &FMaterialDiffPanel::OnPreviewNode));
		}

		TSharedRef<SGraphEditor> Editor = SNew(SGraphEditor)
											  .AdditionalCommands(GraphEditorCommands)
											  .GraphToEdit(Graph)
											  .GraphToDiff(nullptr)
											  .DiffResults(DiffResults)
											  .FocusedDiffResult(FocusedDiffResult)
											  .IsEditable(false)
											  .GraphEvents(InEvents);

		GraphEditor = Editor;
		Widget = Editor;
	}

	GraphEditorBox->SetContent(Widget.ToSharedRef());
}

FGraphPanelSelectionSet FMaterialDiffPanel::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection{};
	TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin();

	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}

	return CurrentSelection;
}

void FMaterialDiffPanel::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FMaterialDiffPanel::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if (Node && Node->CanDuplicateNode())
		{
			return true;
		}
	}

	return false;
}

void FMaterialDiffPanel::SelectDownstreamNodes() const
{
	TArray<UMaterialGraphNode*> NodesToCheck;
	TArray<UMaterialGraphNode*> CheckedNodes;
	TArray<UMaterialGraphNode*> NodesToSelect;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
		if (GraphNode)
		{
			NodesToCheck.Add(GraphNode);
		}
	}

	int32 WatchDogWhile = 0;
	static constexpr int32 LimitIteration = 100;
	while (WatchDogWhile++ < LimitIteration && NodesToCheck.Num() > 0)
	{
		UMaterialGraphNode* CurrentNode = NodesToCheck.Last();
		for (UEdGraphPin* Pin : CurrentNode->Pins)
		{
			if (Pin->Direction == EGPD_Output)
			{
				for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
				{
					UMaterialGraphNode* LinkedNode = Cast<UMaterialGraphNode>(Pin->LinkedTo[LinkIndex]->GetOwningNode());
					if (LinkedNode)
					{
						int32 FoundIndex = -1;
						CheckedNodes.Find(LinkedNode, FoundIndex);

						if (FoundIndex < 0)
						{
							NodesToSelect.Add(LinkedNode);
							NodesToCheck.Add(LinkedNode);
						}
					}
				}
			}
		}

		// This graph node has now been examined
		CheckedNodes.Add(CurrentNode);
		NodesToCheck.Remove(CurrentNode);
	}

	if (WatchDogWhile >= LimitIteration)
	{
		UE_LOGF(LogMaterialEditorDiff, Warning, "FMaterialDiffPanel::SelectDownstreamNodes - Infinite loop encounter");
		return;
	}

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin())
	{
		for (int32 Index = 0; Index < NodesToSelect.Num(); ++Index)
		{
			FocusedGraphEd->SetNodeSelection(NodesToSelect[Index], true);
		}
	}
}

void FMaterialDiffPanel::SelectUpstreamNodes() const
{
	TArray<UMaterialGraphNode*> NodesToCheck;
	TArray<UMaterialGraphNode*> CheckedNodes;
	TArray<UMaterialGraphNode*> NodesToSelect;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
		if (GraphNode)
		{
			NodesToCheck.Add(GraphNode);
		}
	}

	int32 WatchDogWhile = 0;
	static constexpr int32 LimitIteration = 100;
	while (WatchDogWhile++ < LimitIteration && NodesToCheck.Num() > 0)
	{
		UMaterialGraphNode* CurrentNode = NodesToCheck.Last();
		for (UEdGraphPin* Pin : CurrentNode->Pins)
		{
			if (Pin->Direction == EGPD_Input)
			{
				for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
				{
					UMaterialGraphNode* LinkedNode = Cast<UMaterialGraphNode>(Pin->LinkedTo[LinkIndex]->GetOwningNode());
					if (LinkedNode)
					{
						int32 FoundIndex = -1;
						CheckedNodes.Find(LinkedNode, FoundIndex);

						if (FoundIndex < 0)
						{
							NodesToSelect.Add(LinkedNode);
							NodesToCheck.Add(LinkedNode);
						}
					}
				}
			}
		}

		// This graph node has now been examined
		CheckedNodes.Add(CurrentNode);
		NodesToCheck.Remove(CurrentNode);
	}

	if (WatchDogWhile >= LimitIteration)
	{
		UE_LOGF(LogMaterialEditorDiff, Warning, "FMaterialDiffPanel::SelectUpstreamNodes - Infinite loop encounter");
		return;
	}

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin())
	{
		for (int32 Index = 0; Index < NodesToSelect.Num(); ++Index)
		{
			FocusedGraphEd->SetNodeSelection(NodesToSelect[Index], true);
		}
	}
}

void FMaterialDiffPanel::OnPreviewNode()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin())
				{
					FocusedGraphEd->NotifyGraphChanged();
				}

				SetPreviewExpression(GraphNode->MaterialExpression);
			}
		}
	}
}

void FMaterialDiffPanel::FocusDiff(UEdGraphPin& Pin)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor.Pin()->JumpToPin(&Pin);
	}
}

void FMaterialDiffPanel::FocusDiff(UEdGraphNode& Node)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor.Pin()->JumpToNode(&Node, false);
	}
}

TSharedRef<SWidget> FMaterialDiffPanel::GetMaterialNodeDetailsViewWidget() const
{
	// If this panel is displaying a null object, return an empty boarder instead of a IDetailsView
	return MaterialNodeDetailsView ? MaterialNodeDetailsView.ToSharedRef() : TSharedRef<SWidget>(SNew(SBorder));
}

void FMaterialDiffPanel::SetViewportToDisplay()
{
	// Determine the preview target: MaterialInstance takes priority over MaterialGraph->Material
	UMaterialInterface* PreviewTarget = nullptr;

	if (MaterialInstance)
	{
		PreviewTarget = MaterialInstance;
	}
	else if (MaterialGraph)
	{
		PreviewTarget = MaterialGraph->Material;
	}

	if (!PreviewTarget)
	{
		return;
	}

	if (PreviewTarget->IsUIMaterial())
	{
		SAssignNew(Preview2DViewport, SMaterialEditorUIPreviewViewport, PreviewTarget);
	}

	SAssignNew(Preview3DViewport, SMaterialEditor3DPreviewViewport)
		.PreviewMaterial(PreviewTarget);

	if (Preview3DViewport->PreviewMeshComponent != nullptr)
	{
		Preview3DViewport->GetViewportClient()->FocusViewportOnBox(Preview3DViewport->PreviewMeshComponent->Bounds.GetBox(), true);
	}

	// See FMaterialEditor::InitMaterialEditor
	if (MaterialGraph->MaterialFunction && MaterialGraph->Material)
	{
		bool bSetPreviewExpression = false;
		UMaterialExpressionFunctionOutput* FirstOutput = nullptr;
		for (int32 ExpressionIndex = MaterialGraph->Material->GetExpressions().Num() - 1; ExpressionIndex >= 0; ExpressionIndex--)
		{
			UMaterialExpression* Expression = MaterialGraph->Material->GetExpressions()[ExpressionIndex];

			// Setup the expression to be used with the preview material instead of the function
			Expression->Function = nullptr;
			Expression->Material = MaterialGraph->Material;

			UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);
			if (FunctionOutput)
			{
				FirstOutput = FunctionOutput;
				if (FunctionOutput->bLastPreviewed)
				{
					bSetPreviewExpression = true;

					// Preview the last output previewed
					SetPreviewExpression(FunctionOutput);
				}
			}
		}

		if (!bSetPreviewExpression && FirstOutput)
		{
			SetPreviewExpression(FirstOutput);
		}
	}
}

TSharedRef<SWidget> FMaterialDiffPanel::GetViewportToDisplay() const
{
	if (Preview2DViewport)
	{
		return Preview2DViewport.ToSharedRef();
	}

	return Preview3DViewport ? Preview3DViewport.ToSharedRef() : TSharedRef<SWidget>(SNew(SBorder));
}

void FMaterialDiffPanel::SetViewportVisibility(bool bShowViewport)
{
	if (Preview2DViewport)
	{
		Preview2DViewport->SetVisibility(bShowViewport ? EVisibility::Visible : EVisibility::Collapsed);
	}

	if (Preview3DViewport)
	{
		Preview3DViewport->SetVisibility(bShowViewport ? EVisibility::Visible : EVisibility::Collapsed);
	}
}

void FMaterialDiffPanel::SetPreviewExpression(UMaterialExpression* NewPreviewExpression)
{
	if (!MaterialGraph)
	{
		return;
	}

	UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(NewPreviewExpression);

	const UMaterial* Material = MaterialGraph->Material;

	if (!NewPreviewExpression || PreviewExpression == NewPreviewExpression)
	{
		if (FunctionOutput)
		{
			FunctionOutput->bLastPreviewed = false;
		}
		// If we are already previewing the selected expression toggle previewing off
		PreviewExpression = nullptr;
		if(ExpressionPreviewMaterial)
		{
			ExpressionPreviewMaterial->GetExpressionCollection().Empty();
		}
		SetPreviewMaterial(MaterialGraph->Material);
		// Recompile the preview material to get changes that might have been made during previewing
		UpdatePreviewMaterial();
	}
	else
	{
		if (ExpressionPreviewMaterial == nullptr)
		{
			// Create the expression preview material if it hasnt already been created
			ExpressionPreviewMaterial = NewObject<UPreviewMaterial>(GetTransientPackage(), NAME_None, RF_Public);
			ExpressionPreviewMaterial->bIsPreviewMaterial = true;
			ExpressionPreviewMaterial->bEnableNewHLSLGenerator = Material->IsUsingNewHLSLGenerator();
			// MODAVI_TODO
			//ExpressionPreviewMaterial->bEnableExecWire = Material->IsUsingControlFlow();
			if (Material->IsUIMaterial())
			{
				ExpressionPreviewMaterial->MaterialDomain = MD_UI;
			}
			else if (Material->IsPostProcessMaterial())
			{
				ExpressionPreviewMaterial->MaterialDomain = MD_PostProcess;
			}
		}

		if (FunctionOutput)
		{
			FunctionOutput->bLastPreviewed = true;
		}
		else
		{
			// Hooking up the output of the break expression doesn't make much sense, preview the expression feeding it instead.
			UMaterialExpressionBreakMaterialAttributes* BreakExpr = Cast<UMaterialExpressionBreakMaterialAttributes>(NewPreviewExpression);
			if (BreakExpr && BreakExpr->GetInput(0) && BreakExpr->GetInput(0)->Expression)
			{
				NewPreviewExpression = BreakExpr->GetInput(0)->Expression;
			}
		}

		// The expression preview material's expressions array must stay up to date before recompiling
		// So that RebuildMaterialFunctionInfo will see all the nested material functions that may need to be updated
		ExpressionPreviewMaterial->AssignExpressionCollection(Material->GetExpressionCollection());

		// The preview window should now show the expression preview material
		SetPreviewMaterial(ExpressionPreviewMaterial);

		// Set the preview expression
		PreviewExpression = NewPreviewExpression;

		// Recompile the preview material
		UpdatePreviewMaterial();
	}
}


void FMaterialDiffPanel::SetPreviewMaterial(UMaterialInterface* InMaterialInterface) const
{
	if (Preview2DViewport)
	{
		Preview2DViewport->SetPreviewMaterial(InMaterialInterface);
	}
	
	if (Preview3DViewport)
	{
		Preview3DViewport->SetPreviewMaterial(InMaterialInterface);
	}
}

void FMaterialDiffPanel::UpdatePreviewMaterial() const
{
	if (!MaterialGraph)
	{
		return;
	}

	if (PreviewExpression && ExpressionPreviewMaterial)
	{
		ExpressionPreviewMaterial->UpdateCachedExpressionData();
		PreviewExpression->ConnectToPreviewMaterial(ExpressionPreviewMaterial, 0);
	}

	const UMaterial* Material = MaterialGraph->Material;

	if (PreviewExpression)
	{
		check(ExpressionPreviewMaterial);

		// The preview material's expressions array must stay up to date before recompiling
		// So that RebuildMaterialFunctionInfo will see all the nested material functions that may need to be updated
		ExpressionPreviewMaterial->AssignExpressionCollection(Material->GetExpressionCollection());
		ExpressionPreviewMaterial->bEnableNewHLSLGenerator = Material->IsUsingNewHLSLGenerator();

		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::SyncWithRenderingThread);
		UpdateContext.AddMaterial(ExpressionPreviewMaterial);

		// If we are previewing an expression, update the expression preview material
		ExpressionPreviewMaterial->PreEditChange(nullptr);
		ExpressionPreviewMaterial->PostEditChange();
	}

	// Reregister all components that use the preview material, since UMaterial::PEC does not reregister components using a bIsPreviewMaterial=true material
	if (Preview3DViewport)
	{
		Preview3DViewport->RefreshViewport();
	}
}

void FMaterialDiffPanel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ExpressionPreviewMaterial);
	Collector.AddReferencedObject(PreviewExpression);
	Collector.AddReferencedObject(MaterialGraph);
	Collector.AddReferencedObject(MaterialInstance);
	Collector.AddReferencedObject(EditorProxy);
}

FString FMaterialDiffPanel::GetReferencerName() const
{
	return TEXT("FMaterialDiffPanel");
}

#undef LOCTEXT_NAMESPACE
