// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffTool/MaterialToDiff.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphDiffControl.h"
#include "MaterialGraph/MaterialGraph.h"
#include "SBlueprintDiff.h"
#include "Widgets/SCompoundWidget.h"
#include "DiffTool/Widgets/SMaterialDiff.h"

#define LOCTEXT_NAMESPACE "MaterialToDiff"

FMaterialToDiff::FMaterialToDiff(SMaterialDiff* InDiffWidget, UMaterialGraph* InOldGraph, UMaterialGraph* InNewGraph, const FRevisionInfo& InOldRevision, const FRevisionInfo& InNewRevision)
	: FoundDiffs(MakeShared<TArray<FDiffSingleResult>>())
	, DiffWidget(InDiffWidget)
	, OldGraph(InOldGraph)
	, NewGraph(InNewGraph)
	, OldRevision(InOldRevision)
	, NewRevision(InNewRevision)
{
	check(InOldGraph || InNewGraph);
	
	BuildDiffSourceArray();
}

FMaterialToDiff::~FMaterialToDiff()
{
}

void FMaterialToDiff::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	if (!DiffListSource.IsEmpty())
	{
		RealDifferencesStartIndex = OutRealDifferences.Num();
	}

	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> Children;
	for (const TSharedPtr<FMaterialDiffResultItem>& Difference : DiffListSource)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> ChildEntry = MakeShared<FBlueprintDifferenceTreeEntry>(
			FOnDiffEntryFocused::CreateSP(DiffWidget, &SMaterialDiff::OnDiffListSelectionChanged, Difference),
			FGenerateDiffEntryWidget::CreateSP(Difference.ToSharedRef(), &FMaterialDiffResultItem::GenerateWidget));
		Children.Push(ChildEntry);
		OutRealDifferences.Push(ChildEntry);
	}

	if (Children.Num() == 0)
	{
		// Make one child informing the user that there are no differences
		Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
	}

	TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
		FOnDiffEntryFocused::CreateSP(DiffWidget, &SMaterialDiff::OnGraphSelectionChanged, TSharedPtr<FMaterialToDiff>(AsShared()), ESelectInfo::Direct),
		FGenerateDiffEntryWidget::CreateSP(AsShared(), &FMaterialToDiff::GenerateCategoryWidget),
		Children);
	OutTreeEntries.Push(Entry);

	// Add comments as children to this category
	FString GraphName;
	const UMaterialGraph* Graph = OldGraph ? OldGraph : NewGraph;
	if (const UEdGraphSchema* Schema = Graph->GetSchema())
	{
		FGraphDisplayInfo DisplayInfo;
		Schema->GetGraphDisplayInformation(*Graph, DisplayInfo);

		GraphName = DisplayInfo.DisplayName.ToString();
	}
	else
	{
		GraphName = Graph->GetFName().ToString();
	}
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, GraphName);
}

UMaterialGraph* FMaterialToDiff::GetOldGraph() const
{
	return OldGraph;
}

UMaterialGraph* FMaterialToDiff::GetNewGraph() const
{
	return NewGraph;
}

FText FMaterialToDiff::GetToolTip()
{
	if (OldGraph && NewGraph)
	{
		if (DiffListSource.Num() > 0)
		{
			return LOCTEXT("ContainsDifferences", "Revisions are different");
		}
		else
		{
			return LOCTEXT("MaterialGraphsIdentical", "Revisions appear to be identical");
		}
	}
	else
	{
		UMaterialGraph* GoodGraph = OldGraph ? OldGraph : NewGraph;
		check(GoodGraph);
		const FRevisionInfo& Revision = NewGraph ? OldRevision : NewRevision;
		FText RevisionText = LOCTEXT("CurrentRevision", "Current Revision");

		if (!Revision.Revision.IsEmpty())
		{
			RevisionText = FText::Format(LOCTEXT("Revision Number", "Revision {0}"), FText::FromString(Revision.Revision));
		}

		return FText::Format(LOCTEXT("MissingGraph", "Graph '{0}' missing from {1}"), FText::FromString(GoodGraph->GetName()), RevisionText);
	}
}

TSharedRef<SWidget> FMaterialToDiff::GenerateCategoryWidget()
{
	const UMaterialGraph* Graph = OldGraph ? OldGraph : NewGraph;
	check(Graph);

	FLinearColor Color = (OldGraph && NewGraph) ? DiffViewUtils::Identical() : FLinearColor(0.3f, 0.3f, 1.f);

	const bool bHasDiffs = DiffListSource.Num() > 0;

	if (bHasDiffs)
	{
		Color = DiffViewUtils::Differs();
	}

	FText GraphName;
	if (const UEdGraphSchema* Schema = Graph->GetSchema())
	{
		FGraphDisplayInfo DisplayInfo;
		Schema->GetGraphDisplayInformation(*Graph, DisplayInfo);

		GraphName = DisplayInfo.DisplayName;
	}
	else
	{
		GraphName = FText::FromName(Graph->GetFName());
	}

	return SNew(SHorizontalBox) 
		   +SHorizontalBox::Slot()
		   [
		       SNew(STextBlock)
		       .ColorAndOpacity(Color)
		       .Text(GraphName)
		       .ToolTipText(GetToolTip())
		    ] 
		    +DiffViewUtils::Box(OldGraph != nullptr, Color) 
		    +DiffViewUtils::Box(NewGraph != nullptr, Color);
}

void FMaterialToDiff::BuildDiffSourceArray()
{
	FoundDiffs->Empty();
	FGraphDiffControl::DiffGraphs(OldGraph, NewGraph, *FoundDiffs);

	Algo::SortBy(*FoundDiffs, &FDiffSingleResult::Diff);

	DiffListSource.Empty();
	for (const FDiffSingleResult& Diff : *FoundDiffs)
	{
		TSharedRef<FMaterialDiffResultItem> DiffResultItem = MakeShared<FMaterialDiffResultItem>(Diff);

		if (Diff.Diff == EDiffType::NODE_PROPERTY && Diff.Object1 && Diff.Object2)
		{
			for (TFieldIterator<FProperty> PropertyIt(Diff.Object1->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				// We use the DisplayString containing the changed property's name
				if (PropertyIt->GetName().Equals(Diff.DisplayString.ToString()))
				{
					DiffResultItem->Property.AddProperty(FPropertyInfo(*PropertyIt));

					// We found the property, we keep it in memory and we can use the correct format for properties of MaterialGraph
					DiffResultItem->Result.DisplayString = FText::Format(LOCTEXT("Diff_MaterialGraphNodePropertyFormat", "Property Changed: {0} in {1}"), 
						                                                 FText::FromString(PropertyIt->GetName()),
						                                                 FText::FromString(Diff.Object1->GetName()));

					break;
				}
			}
		}

		DiffListSource.Add(DiffResultItem);
	}
}

#undef LOCTEXT_NAMESPACE