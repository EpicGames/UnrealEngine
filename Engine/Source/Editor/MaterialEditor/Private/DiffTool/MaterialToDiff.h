// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DiffControl.h"

class SMaterialDiff;
class UMaterialGraph;

struct FMaterialDiffResultItem;

enum class EAssetEditorCloseReason : uint8;

/** Category list item for a MaterialGraph **/
struct FMaterialToDiff : public TSharedFromThis<FMaterialToDiff>, IDiffControl
{
	FMaterialToDiff(SMaterialDiff* DiffWidget, UMaterialGraph* OldGraph, UMaterialGraph* NewGraph, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision);
	virtual ~FMaterialToDiff() override;

	/** Add widgets to the differences tree */
	virtual void GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences) override;

	/** Get old(left) graph*/
	UMaterialGraph* GetOldGraph() const;

	/** Get new(right) graph*/
	UMaterialGraph* GetNewGraph() const;

	/** Source for list view */
	TArray<TSharedPtr<FMaterialDiffResultItem>> DiffListSource;
	TSharedPtr<TArray<FDiffSingleResult>> FoundDiffs;

	/** Index of the first item in RealDifferences that was generated by this graph */
	int32 RealDifferencesStartIndex = INDEX_NONE;

private:
	/** Get tooltip for category */
	FText GetToolTip();

	/** Generate Widget for category list */
	TSharedRef<SWidget> GenerateCategoryWidget();

	/** Build up the Diff Source Array*/
	void BuildDiffSourceArray();

	/** Diff widget */
	SMaterialDiff* DiffWidget;

	/** The old graph(left)*/
	UMaterialGraph* OldGraph;

	/** The new graph(right)*/
	UMaterialGraph* NewGraph;

	/** Description of Old and New graph*/
	FRevisionInfo OldRevision;
	FRevisionInfo NewRevision;
};

