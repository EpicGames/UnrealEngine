// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffTool/Widgets/SMaterialDiff.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "GraphDiffControl.h"
#include "IDetailsView.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialFunction.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SBlueprintDiff.h"
#include "SDetailsDiff.h"
#include "SlateOptMacros.h"
#include "SMaterialEditorViewport.h"
#include "SMyBlueprint.h"
#include "SourceControlHelpers.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "DiffTool/MaterialToDiff.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOverlay.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "MaterialEditorInstanceDetailCustomization.h"
#include "Materials/MaterialInstanceConstant.h"

#define LOCTEXT_NAMESPACE "SMaterialGraphDiff"

static const FName GraphMode = FName(TEXT("GraphMode"));

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMaterialDiff::Construct(const FArguments& InArgs)
{
	check(InArgs._OldMaterialGraph || InArgs._NewMaterialGraph || InArgs._OldMaterialInstance || InArgs._NewMaterialInstance);

	if (InArgs._OldMaterialInstance || InArgs._NewMaterialInstance)
	{
		PanelOld.MaterialInstance = InArgs._OldMaterialInstance;
		PanelNew.MaterialInstance = InArgs._NewMaterialInstance;

		// Create UMaterialEditorInstanceConstant proxies for each side so that ALL parameter types
		// (including static switches and component masks) are visible in the details view.
		// This is the same proxy object the Material Instance Editor uses to display parameters.
		auto CreateEditorProxy = [](UMaterialInstance* MI) -> UMaterialEditorInstanceConstant*
		{
			UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(MI);
			if (!MIC)
			{
				return nullptr;
			}

			UMaterialEditorInstanceConstant* Proxy = NewObject<UMaterialEditorInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);
			Proxy->SetSourceInstance(MIC);
			return Proxy;
		};

		PanelOld.EditorProxy = CreateEditorProxy(PanelOld.MaterialInstance);
		PanelNew.EditorProxy = CreateEditorProxy(PanelNew.MaterialInstance);

		PanelOld.RevisionInfo = InArgs._OldRevision;
		PanelNew.RevisionInfo = InArgs._NewRevision;

		// Build resolved material graphs from each instance's parent material.
		// Each side gets its own duplicated parent so expression objects are independent.
		auto BuildResolvedGraph = [](UMaterial* ParentCopy, UMaterialInstance* Instance) -> UMaterialGraph*
		{
			UMaterialGraph* Graph = CastChecked<UMaterialGraph>(
				FBlueprintEditorUtils::CreateNewGraph(
					GetTransientPackage(),
					MakeUniqueObjectName(GetTransientPackage(), UMaterialGraph::StaticClass()),
					UMaterialGraph::StaticClass(),
					UMaterialGraphSchema::StaticClass()));
			Graph->Material = ParentCopy;
			Graph->MaterialFunction = nullptr;
			Graph->RebuildGraph();

			// Apply static switch overrides from the instance
			if (Instance)
			{
				FStaticParameterSet StaticParams = Instance->GetStaticParameters();

				TMap<FName, bool> SwitchOverrides;
				for (const FStaticSwitchParameter& Param : StaticParams.StaticSwitchParameters)
				{
					if (Param.bOverride)
					{
						SwitchOverrides.Add(Param.ParameterInfo.Name, Param.Value);
					}
				}

				for (UEdGraphNode* Node : Graph->Nodes)
				{
					if (UMaterialGraphNode* MatNode = Cast<UMaterialGraphNode>(Node))
					{
						if (UMaterialExpressionStaticSwitchParameter* SwitchExpr = Cast<UMaterialExpressionStaticSwitchParameter>(MatNode->MaterialExpression))
						{
							if (const bool* Override = SwitchOverrides.Find(SwitchExpr->ParameterName))
							{
								SwitchExpr->DefaultValue = *Override;
							}
						}
					}
				}
			}

			return Graph;
		};

		if (UMaterial* OldParent = PanelOld.MaterialInstance ? PanelOld.MaterialInstance->GetMaterial() : nullptr)
		{
			PanelOld.MaterialGraph = BuildResolvedGraph(DuplicateObject<UMaterial>(OldParent, GetTransientPackage()), PanelOld.MaterialInstance);
		}
		if (UMaterial* NewParent = PanelNew.MaterialInstance ? PanelNew.MaterialInstance->GetMaterial() : nullptr)
		{
			PanelNew.MaterialGraph = BuildResolvedGraph(DuplicateObject<UMaterial>(NewParent, GetTransientPackage()), PanelNew.MaterialInstance);
		}

		// Only display the graph diff if we have two material instances
		if (PanelOld.MaterialInstance && PanelNew.MaterialInstance)
		{
			const bool bSameParent = PanelOld.MaterialInstance->GetMaterial() == PanelNew.MaterialInstance->GetMaterial();
			const bool bHasStaticPermutation = PanelOld.MaterialInstance->bHasStaticPermutationResource || PanelNew.MaterialInstance->bHasStaticPermutationResource;
			bShowGraphDiff = bSameParent && bHasStaticPermutation;
		}
	}
	else
	{
		PanelOld.MaterialGraph = InArgs._OldMaterialGraph;
		PanelNew.MaterialGraph = InArgs._NewMaterialGraph;
		PanelOld.RevisionInfo = InArgs._OldRevision;
		PanelNew.RevisionInfo = InArgs._NewRevision;

		PanelOld.bShowAssetName = InArgs._ShowAssetNames;
		PanelNew.bShowAssetName = InArgs._ShowAssetNames;

		bShowGraphDiff = true;
	}

	UObject* OldAsset = nullptr;
	if (PanelOld.EditorProxy)
	{
		OldAsset = PanelOld.EditorProxy;
	}
	else if (PanelOld.MaterialGraph && PanelOld.MaterialGraph->MaterialFunction)
	{
		OldAsset = PanelOld.MaterialGraph->MaterialFunction;
	}
	else if (PanelOld.MaterialGraph)
	{
		OldAsset = PanelOld.MaterialGraph->Material;
	}

	UObject* NewAsset = nullptr;
	if (PanelNew.EditorProxy)
	{
		NewAsset = PanelNew.EditorProxy;
	}
	else if (PanelNew.MaterialGraph && PanelNew.MaterialGraph->MaterialFunction)
	{
		NewAsset = PanelNew.MaterialGraph->MaterialFunction;
	}
	else if(PanelNew.MaterialGraph)
	{
		NewAsset = PanelNew.MaterialGraph->Material;
	}

	if (InArgs._ParentWindow.IsValid())
	{
		WeakParentWindow = InArgs._ParentWindow;

		AssetEditorCloseDelegate = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().AddSP(this, &SMaterialDiff::OnCloseAssetEditor);
	}

	FToolBarBuilder NavToolBarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SMaterialDiff::PrevDiff),
			FCanExecuteAction::CreateSP(this, &SMaterialDiff::HasPrevDiff)),
		NAME_None,
		LOCTEXT("PrevDiffLabel", "Prev"),
		LOCTEXT("PrevDiffTooltip", "Go to previous difference"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff")
	);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SMaterialDiff::NextDiff),
			FCanExecuteAction::CreateSP(this, &SMaterialDiff::HasNextDiff)),
		NAME_None,
		LOCTEXT("NextDiffLabel", "Next"),
		LOCTEXT("NextDiffTooltip", "Go to next difference"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff")
	);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SMaterialDiff::ToggleViewport)),
		NAME_None,
		LOCTEXT("ToggleViewportLabel", "Toggle Viewport"),
		LOCTEXT("ToggleViewportTooltip", "Show/Hide a preview Viewport for the Materials diffed"),
		TAttribute<FSlateIcon>(this, &SMaterialDiff::GetViewportImage)
	);

	FToolBarBuilder GraphToolbarBuilder(TSharedPtr< const FUICommandList >(), FMultiBoxCustomization::None);
	GraphToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SMaterialDiff::OnToggleLockView)), 
		NAME_None,
		LOCTEXT("LockMaterialGraphsLabel", "Lock/Unlock"),
		LOCTEXT("LockMaterialGraphsTooltip", "Force all graph views to change together, or allow independent scrolling/zooming"),
		TAttribute<FSlateIcon>(this, &SMaterialDiff::GetLockViewImage)
	);
	GraphToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SMaterialDiff::OnToggleSplitViewMode)),
		NAME_None,
		LOCTEXT("SplitMaterialGraphsModeLabel", "Vertical/Horizontal"),
		LOCTEXT("SplitMaterialGraphsModeLabelTooltip", "Toggles the split view of Material Graphs between vertical and horizontal"), 
		TAttribute<FSlateIcon>(this, &SMaterialDiff::GetSplitViewModeImage)
	);

	DifferencesTreeView = DiffTreeView::CreateTreeView(&PrimaryDifferencesList);

	GenerateDifferencesList();

	const auto TextBlock = [](FText Text) -> TSharedRef<SWidget>
	{
		return SNew(SBox)
		.Padding(FMargin(4.0f, 10.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.Text(Text)
		];
	};

	TopRevisionInfoWidget =
		SNew(SSplitter)
		.Visibility(EVisibility::HitTestInvisible)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
		]
		+ SSplitter::Slot()
		.Value(.8f)
		[
			SNew(SSplitter)
			.PhysicalSplitterHandleSize(10.0f)
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(PanelOld.MaterialGraph, PanelOld.RevisionInfo, FText()))
			]
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(PanelNew.MaterialGraph, PanelNew.RevisionInfo, FText()))
			]
		];

	GraphToolBarWidget = 
		SNew(SSplitter)
		.Visibility(EVisibility::HitTestInvisible)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
		]
		+ SSplitter::Slot()
		.Value(.8f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				GraphToolbarBuilder.MakeWidget()
			]	
		];

	// Clang-format off
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Docking.Tab", ".ContentAreaBrush"))
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			[
				TopRevisionInfoWidget.ToSharedRef()		
			]
			+ SOverlay::Slot()
			.Padding(0.f, 6.f, 0.f, 4.f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(0.f, 2.f, 0.f, 5.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.IsEnabled(bShowGraphDiff)
						.OnClicked_Lambda([this]()->FReply { SetCurrentWidgetIndex(0); return FReply::Handled(); })
						[
							SNew(STextBlock)
							.Text(FText::FromString("Graph"))
						]
					]
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.OnClicked_Lambda([this]()->FReply { SetCurrentWidgetIndex(1); return FReply::Handled(); })
						[
							SNew(STextBlock)
							.Text(FText::FromString("Properties"))
						]
					]
				]
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(WidgetSwitcher, SWidgetSwitcher)
					.WidgetIndex(this, &SMaterialDiff::GetCurrentWidgetIndex)
					+ SWidgetSwitcher::Slot()
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						.VAlign(VAlign_Top)
						.Padding(0.0f, 6.0f, 0.0f, 4.0f)
						[
							GraphToolBarWidget.ToSharedRef()
						]
						+SOverlay::Slot()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 2.0f, 0.0f, 2.0f)
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.Padding(4.f)
								.AutoWidth()
								[
									NavToolBarBuilder.MakeWidget()
								]
								+SHorizontalBox::Slot()
								[
									SNew(SSpacer)
								]
							]
							+SVerticalBox::Slot()
							.FillHeight(1.f)
							[
								SNew(SSplitter)
								+SSplitter::Slot()
								.Value(.2f)
								[
									SNew(SBorder)
									.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
									[
										DifferencesTreeView.ToSharedRef()
									]
								]
								+SSplitter::Slot()
								.Value(.8f)
								[
									SAssignNew(ModeContents, SBox)
								]
							]
						]
					]
					+ SWidgetSwitcher::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						[
							SNew(SDetailsDiff)
							.OldAsset(OldAsset)
							.NewAsset(NewAsset)
							.OldRevision(PanelOld.RevisionInfo)
							.NewRevision(PanelNew.RevisionInfo)
							.ShowAssetNames(true)
							.ParentWindow(WeakParentWindow.Pin())
							.OnCustomizeDetailsWidget_Raw(this, &SMaterialDiff::AddMaterialInstanceDetailCustomization)
						]
					]
				]
				
			]
		]
	];
	// clang-format on

	// Default to Properties tab for instances, Graph tab for materials
	CurrentWidgetIndex = (PanelOld.MaterialInstance || PanelNew.MaterialInstance) ? 1 : 0;

	HandleGraphChanged(FString{});
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMaterialDiff::SetCurrentWidgetIndex(int32 Index)
{
	CurrentWidgetIndex = Index;
}


int32 SMaterialDiff::GetCurrentWidgetIndex() const
{
	return CurrentWidgetIndex;
}

SMaterialDiff::~SMaterialDiff()
{
	if (AssetEditorCloseDelegate.IsValid())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseDelegate);
	}
}

void SMaterialDiff::OnGraphChanged(FMaterialToDiff* Diff)
{
	if (PanelNew.GraphEditor.IsValid() && PanelNew.GraphEditor.Pin()->GetCurrentGraph() == Diff->GetNewGraph())
	{
		FocusOnMaterialGraphRevisions(Diff);
	}
}

void SMaterialDiff::OnGraphSelectionChanged(TSharedPtr<FMaterialToDiff> Item, ESelectInfo::Type SelectionType)
{
	if (!Item.IsValid())
	{
		return;
	}

	FocusOnMaterialGraphRevisions(Item.Get());
}

void SMaterialDiff::OnDiffListSelectionChanged(TSharedPtr<FMaterialDiffResultItem> TheDiff)
{
	check(!TheDiff->Result.OwningObjectPath.IsEmpty());

	FocusOnMaterialGraphRevisions(FindGraphToDiffEntry(TheDiff->Result.OwningObjectPath));
	FDiffSingleResult Result = TheDiff->Result;

	const auto SafeClearSelection = [](TWeakPtr<SGraphEditor> GraphEditor) {
		TSharedPtr<SGraphEditor> GraphEditorPtr = GraphEditor.Pin();
		if (GraphEditorPtr.IsValid())
		{
			GraphEditorPtr->ClearSelectionSet();
		}
	};

	SafeClearSelection(PanelNew.GraphEditor);
	SafeClearSelection(PanelOld.GraphEditor);

	if (Result.Pin1)
	{
		GetDiffPanelForNode(*Result.Pin1->GetOwningNode()).FocusDiff(*Result.Pin1);
		if (Result.Pin2)
		{
			GetDiffPanelForNode(*Result.Pin2->GetOwningNode()).FocusDiff(*Result.Pin2);
		}
	}
	else if (Result.Node1)
	{
		GetDiffPanelForNode(*Result.Node1).FocusDiff(*Result.Node1);
		if (Result.Node2)
		{
			GetDiffPanelForNode(*Result.Node2).FocusDiff(*Result.Node2);
		}
	}

	PanelOld.PropertyToFocus = TheDiff->Property;
	PanelNew.PropertyToFocus = TheDiff->Property;
}

TSharedRef<SWidget> SMaterialDiff::DefaultEmptyPanel()
{
	return SNew(SHorizontalBox)
		   +SHorizontalBox::Slot()
		   .HAlign(HAlign_Center)
		   .VAlign(VAlign_Center)
		   [
			   SNew(STextBlock)
			   .Text(LOCTEXT("MaterialGraphDiffGraphsToolTip", "Select Material Graph to Diff"))
		   ];
}

TSharedPtr<SWindow> SMaterialDiff::CreateDiffWindow(FText WindowTitle, TObjectPtr<UMaterialGraph> OldMaterialGraph, TObjectPtr<UMaterialGraph> NewMaterialGraph, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision)
{
	// sometimes we're comparing different revisions of one single asset (other
	// times we're comparing two completely separate assets altogether)
	bool bIsSingleAsset = !OldMaterialGraph || !NewMaterialGraph || (OldMaterialGraph->GetName() == NewMaterialGraph->GetName());

	TSharedPtr<SWindow> Window = SNew(SWindow)
									 .Title(WindowTitle)
									 .ClientSize(FVector2D(1000, 800));

	Window->SetContent(SNew(SMaterialDiff)
						   .OldMaterialGraph(OldMaterialGraph)
						   .NewMaterialGraph(NewMaterialGraph)
						   .OldRevision(OldRevision)
						   .NewRevision(NewRevision)
						   .ShowAssetNames(!bIsSingleAsset)
						   .ParentWindow(Window));

	// Make this window a child of the modal window if we've been spawned while one is active.
	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveModalWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}

	return Window;
}

TSharedPtr<SWindow> SMaterialDiff::CreateDiffWindow(TObjectPtr<UMaterialGraph> OldMaterialGraph, TObjectPtr<UMaterialGraph> NewMaterialGraph, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* ObjectClass)
{
	check(OldMaterialGraph || NewMaterialGraph);

	// sometimes we're comparing different revisions of one single asset (other
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !OldMaterialGraph ||
		!NewMaterialGraph ||
		(NewMaterialGraph->MaterialFunction && OldMaterialGraph->MaterialFunction && NewMaterialGraph->MaterialFunction.GetName() == OldMaterialGraph->MaterialFunction.GetName()) ||
		(NewMaterialGraph->Material.GetName() == OldMaterialGraph->Material.GetName());

	FText WindowTitle = FText::Format(LOCTEXT("NamelessMaterialGraphDiff", "{0} Diff"), ObjectClass->GetDisplayNameText());
	// if we're diffing one asset against itself
	if (bIsSingleAsset)
	{
		// identify the assumed single asset in the window's title
		FString MaterialGraphName{};
		if (NewMaterialGraph)
		{
			MaterialGraphName = NewMaterialGraph->MaterialFunction ? NewMaterialGraph->MaterialFunction.GetName() : NewMaterialGraph->Material.GetName();
		}
		else
		{
			MaterialGraphName = OldMaterialGraph->MaterialFunction ? OldMaterialGraph->MaterialFunction.GetName() : OldMaterialGraph->Material.GetName();
		}
		WindowTitle = FText::Format(LOCTEXT("NamedMaterialGraphDiff", "{0} - {1} Diff"), FText::FromString(MaterialGraphName), ObjectClass->GetDisplayNameText());
	}

	return CreateDiffWindow(WindowTitle, OldMaterialGraph, NewMaterialGraph, OldRevision, NewRevision);
}

void SMaterialDiff::NextDiff()
{
	DiffTreeView::HighlightNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

void SMaterialDiff::PrevDiff()
{
	DiffTreeView::HighlightPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

bool SMaterialDiff::HasNextDiff() const
{
	return DiffTreeView::HasNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences);
}

bool SMaterialDiff::HasPrevDiff() const
{
	return DiffTreeView::HasPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences);
}

void SMaterialDiff::ToggleViewport()
{
	bShowViewport = !bShowViewport;

	PanelOld.SetViewportVisibility(bShowViewport);
	PanelNew.SetViewportVisibility(bShowViewport);
}

FSlateIcon SMaterialDiff::GetViewportImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");
}

FMaterialToDiff* SMaterialDiff::FindGraphToDiffEntry(const FString& GraphPath)
{
	if (MaterialGraphToDiff)
	{
		FString SearchGraphPath = MaterialGraphToDiff->GetOldGraph() ? FGraphDiffControl::GetGraphPath(MaterialGraphToDiff->GetOldGraph()) : FGraphDiffControl::GetGraphPath(MaterialGraphToDiff->GetNewGraph());
		if (SearchGraphPath.Equals(GraphPath, ESearchCase::CaseSensitive))
		{
			return MaterialGraphToDiff.Get();
		}
	}

	return nullptr;
}

void SMaterialDiff::FocusOnMaterialGraphRevisions(FMaterialToDiff* Diff)
{
	if(!Diff)
	{
		return;
	}

	UEdGraph* Graph = Diff->GetOldGraph() ? Diff->GetOldGraph() : Diff->GetNewGraph();

	FString GraphPath = FGraphDiffControl::GetGraphPath(Graph);

	HandleGraphChanged(GraphPath);

	ResetGraphEditors();
}

void SMaterialDiff::OnToggleLockView()
{
	bLockViews = !bLockViews;
	ResetGraphEditors();
}

void SMaterialDiff::OnToggleSplitViewMode()
{
	bVerticalSplitGraphMode = !bVerticalSplitGraphMode;

	if (SSplitter* DiffGraphSplitterPtr = DiffGraphSplitter.Get())
	{
		DiffGraphSplitterPtr->SetOrientation(bVerticalSplitGraphMode ? Orient_Horizontal : Orient_Vertical);
	}
}

FSlateIcon SMaterialDiff::GetLockViewImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), bLockViews ? "Icons.Lock" : "Icons.Unlock");
}

FSlateIcon SMaterialDiff::GetSplitViewModeImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), bVerticalSplitGraphMode ? "BlueprintDif.VerticalDiff" : "BlueprintDif.HorizontalDiff");
}

void SMaterialDiff::ResetGraphEditors()
{
	if (PanelOld.GraphEditor.IsValid() && PanelNew.GraphEditor.IsValid())
	{
		if (bLockViews)
		{
			PanelOld.GraphEditor.Pin()->LockToGraphEditor(PanelNew.GraphEditor);
			PanelNew.GraphEditor.Pin()->LockToGraphEditor(PanelOld.GraphEditor);
		}
		else
		{
			PanelOld.GraphEditor.Pin()->UnlockFromGraphEditor(PanelNew.GraphEditor);
			PanelNew.GraphEditor.Pin()->UnlockFromGraphEditor(PanelOld.GraphEditor);
		}
	}
}

FMaterialDiffPanel& SMaterialDiff::GetDiffPanelForNode(UEdGraphNode& Node)
{
	TSharedPtr<SGraphEditor> OldGraphEditorPtr = PanelOld.GraphEditor.Pin();
	if (OldGraphEditorPtr.IsValid() && Node.GetGraph() == OldGraphEditorPtr->GetCurrentGraph())
	{
		return PanelOld;
	}

	TSharedPtr<SGraphEditor> NewGraphEditorPtr = PanelNew.GraphEditor.Pin();
	if (NewGraphEditorPtr.IsValid() && Node.GetGraph() == NewGraphEditorPtr->GetCurrentGraph())
	{
		return PanelNew;
	}

	ensureMsgf(false, TEXT("Looking for node %s but it cannot be found in provided panels"), *Node.GetName());
	static FMaterialDiffPanel Default {};
	return Default;
}

void SMaterialDiff::HandleGraphChanged(const FString& GraphPath)
{
	SetCurrentMode(GraphMode);

	TSharedPtr<TArray<FDiffSingleResult>> DiffResults;
	int32 RealDifferencesStartIndex = INDEX_NONE;
	
	UEdGraph* NewGraph = MaterialGraphToDiff->GetNewGraph();
	UEdGraph* OldGraph = MaterialGraphToDiff->GetOldGraph();
	DiffResults = MaterialGraphToDiff->FoundDiffs;
	RealDifferencesStartIndex = MaterialGraphToDiff->RealDifferencesStartIndex;

	const TAttribute<int32> FocusedDiffResult = TAttribute<int32>::CreateLambda([this, RealDifferencesStartIndex]() {
		int32 FocusedDiffResult = INDEX_NONE;
		if (RealDifferencesStartIndex != INDEX_NONE)
		{
			FocusedDiffResult = DiffTreeView::CurrentDifference(DifferencesTreeView.ToSharedRef(), RealDifferences) - RealDifferencesStartIndex;
		}

		// find selected index in all the graphs, and subtract the index of the first entry in this graph
		return FocusedDiffResult;
	});
	
	// we could avoid regenerating the panels if the graph was unchanged ala the blueprint diff tool..
	if (!PanelOld.GraphEditor.IsValid())
	{
		PanelOld.GeneratePanel(OldGraph, DiffResults, FocusedDiffResult);
	}

	if (!PanelNew.GraphEditor.IsValid())
	{
		PanelNew.GeneratePanel(NewGraph, DiffResults, FocusedDiffResult);
	}

	ResetGraphEditors();
}

void SMaterialDiff::GenerateDifferencesList()
{
	PrimaryDifferencesList.Empty();
	RealDifferences.Empty();

	const auto CreateDetailsView = []() {
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
		DetailsViewArgs.bShowSectionSelector = true;
		DetailsViewArgs.bAllowFavoriteSystem = true;
		DetailsViewArgs.bShowScrollBar = true;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		return PropertyModule.CreateDetailView(DetailsViewArgs);
	};

	if (PanelOld.MaterialGraph)
	{
		PanelOld.MaterialNodeDetailsView = CreateDetailsView();
	}
	if (PanelNew.MaterialGraph)
	{
		PanelNew.MaterialNodeDetailsView = CreateDetailsView();
	}
	
	ModePanels.Add(GraphMode, GenerateMaterialGraphPanel());

	MaterialGraphToDiff = MakeShared<FMaterialToDiff>(this, 
		                                             PanelOld.MaterialGraph, 
		                                             PanelNew.MaterialGraph,
													 PanelOld.RevisionInfo,
													 PanelNew.RevisionInfo);
	MaterialGraphToDiff->EnableComments(DifferencesTreeView.ToWeakPtr());
	MaterialGraphToDiff->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	DifferencesTreeView->RebuildList();
}

void SMaterialDiff::OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason)
{
	if (CloseReason == EAssetEditorCloseReason::CloseAllAssetEditors)
	{
		// Tell our window to close and set our selves to collapsed to try and stop it from ticking
		SetVisibility(EVisibility::Collapsed);
	
		if (AssetEditorCloseDelegate.IsValid())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseDelegate);
		}
	
		if (WeakParentWindow.IsValid())
		{
			WeakParentWindow.Pin()->RequestDestroyWindow();
		}
	}
}

SMaterialDiff::FMaterialGraphDiffControl SMaterialDiff::GenerateMaterialGraphPanel()
{
	FMaterialGraphDiffControl MaterialGraphPanel{};

	PanelOld.SetViewportToDisplay();
	PanelNew.SetViewportToDisplay();

	MaterialGraphPanel.Widget = SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.FillHeight(1.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Value(0.6f)
			[
				SAssignNew(DiffGraphSplitter, SSplitter)
				.PhysicalSplitterHandleSize(10.0f)
				.Orientation(bVerticalSplitGraphMode ? Orient_Horizontal : Orient_Vertical)
				+SSplitter::Slot()
				[
				   GenerateMaterialGraphWidgetForPanel(PanelOld)
				]
				+SSplitter::Slot()
				[
				   GenerateMaterialGraphWidgetForPanel(PanelNew)
				]
			]
			+SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				+ SSplitter::Slot()
				[
					PanelOld.GetViewportToDisplay()
				]
				+SSplitter::Slot()
				[
					PanelOld.GetMaterialNodeDetailsViewWidget()
				]
				+SSplitter::Slot()
				[
					PanelNew.GetMaterialNodeDetailsViewWidget()
				]
				+SSplitter::Slot()
				[
					PanelNew.GetViewportToDisplay()
				]
			]
		]
	];

	return MaterialGraphPanel;
}

TSharedRef<SOverlay> SMaterialDiff::GenerateMaterialGraphWidgetForPanel(FMaterialDiffPanel& OutDiffPanel) const
{
	return SNew(SOverlay) 
	+ SOverlay::Slot() // Graph slot
	[
		SAssignNew(OutDiffPanel.GraphEditorBox, SBox)
		.HAlign(HAlign_Fill)
		[
			DefaultEmptyPanel()
		]
	] 
	+SOverlay::Slot() // Revision info slot
	.VAlign(VAlign_Bottom)
	.HAlign(HAlign_Right)
	.Padding(FMargin(20.0f, 10.0f))
	[
		GenerateRevisionInfoWidgetForPanel(OutDiffPanel.OverlayGraphRevisionInfo, DiffViewUtils::GetPanelLabel(OutDiffPanel.MaterialGraph, OutDiffPanel.RevisionInfo, FText()))
	];
}

TSharedRef<SBox> SMaterialDiff::GenerateRevisionInfoWidgetForPanel(TSharedPtr<SWidget>& OutGeneratedWidget, const FText& InRevisionText) const
{
	return SAssignNew(OutGeneratedWidget,SBox)
	.Padding(FMargin(4.0f, 10.0f))
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
		.Text(InRevisionText)
		.ShadowColorAndOpacity(FColor::Black)
		.ShadowOffset(FVector2D(1.4f, 1.4f))
	];
}

void SMaterialDiff::SetCurrentMode(FName NewMode)
{
	CurrentMode = NewMode;

	if (FMaterialGraphDiffControl* FoundControl = ModePanels.Find(NewMode))
	{
		ModeContents->SetContent(FoundControl->Widget.ToSharedRef());
	}
	else
	{
		ensureMsgf(false, TEXT("Diff panel does not support mode %s"), *NewMode.ToString());
	}

	OnModeChanged(NewMode);
}

void SMaterialDiff::OnModeChanged(const FName& InNewViewMode) const
{
	UpdateTopSectionVisibility(InNewViewMode);
}

void SMaterialDiff::UpdateTopSectionVisibility(const FName& InNewViewMode) const
{
	SSplitter* GraphToolBarPtr = GraphToolBarWidget.Get();
	SSplitter* TopRevisionInfoWidgetPtr = TopRevisionInfoWidget.Get();

	if (!GraphToolBarPtr || !TopRevisionInfoWidgetPtr)
	{
		return;
	}

	if (InNewViewMode == GraphMode)
	{
		GraphToolBarPtr->SetVisibility(EVisibility::Visible);
		TopRevisionInfoWidgetPtr->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		GraphToolBarPtr->SetVisibility(EVisibility::Collapsed);
		TopRevisionInfoWidgetPtr->SetVisibility(EVisibility::HitTestInvisible);
	}
}

void SMaterialDiff::AddMaterialInstanceDetailCustomization(const TSharedRef<IDetailsView>& DetailsView)
{
	if (!PanelOld.EditorProxy && !PanelNew.EditorProxy)
	{
		return;
	}

	// Determine which panel this details view belongs to by checking which EditorProxy it is displaying.
	// SDetailsDiff calls this callback twice: once for the Old side and once for the New side.
	UMaterialEditorInstanceConstant* ProxyForThisView = nullptr;
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailsView->GetSelectedObjects();
	if (SelectedObjects.Num() > 0 && SelectedObjects[0].IsValid())
	{
		UObject* DisplayedObject = SelectedObjects[0].Get();
		if (DisplayedObject == PanelOld.EditorProxy)
		{
			ProxyForThisView = PanelOld.EditorProxy;
		}
		else if (DisplayedObject == PanelNew.EditorProxy)
		{
			ProxyForThisView = PanelNew.EditorProxy;
		}
	}

	if (!ProxyForThisView)
	{
		return;
	}

	FOnGetDetailCustomizationInstance LayoutMICDetails = FOnGetDetailCustomizationInstance::CreateStatic(
		&FMaterialInstanceParameterDetails::MakeInstance,
		ProxyForThisView,
		static_cast<SMaterialLayersFunctionsInstanceWrapper*>(nullptr),
		FGetShowHiddenParameters());

	DetailsView->RegisterInstancedCustomPropertyLayout(UMaterialEditorInstanceConstant::StaticClass(), LayoutMICDetails);
	DetailsView->ForceRefresh();
}

TSharedPtr<SWindow> SMaterialDiff::CreateDiffWindow(FText WindowTitle, TObjectPtr<UMaterialInstance> OldMaterialInstance, TObjectPtr<UMaterialInstance> NewMaterialInstance, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision)
{
	TSharedPtr<SWindow> Window = SNew(SWindow)
									 .Title(WindowTitle)
									 .ClientSize(FVector2D(1000, 800));

	Window->SetContent(SNew(SMaterialDiff)
						   .OldMaterialInstance(OldMaterialInstance)
						   .NewMaterialInstance(NewMaterialInstance)
						   .OldRevision(OldRevision)
						   .NewRevision(NewRevision)
						   .ParentWindow(Window));

	// Make this window a child of the modal window if we've been spawned while one is active.
	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveModalWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}

	return Window;
}

TSharedPtr<SWindow> SMaterialDiff::CreateDiffWindow(TObjectPtr<UMaterialInstance> OldMaterialInstance, TObjectPtr<UMaterialInstance> NewMaterialInstance, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* ObjectClass)
{
	check(OldMaterialInstance || NewMaterialInstance);

	// sometimes we're comparing different revisions of one single asset (other
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !OldMaterialInstance ||
		!NewMaterialInstance ||
		(NewMaterialInstance->GetName() == OldMaterialInstance->GetName());

	FText WindowTitle = FText::Format(LOCTEXT("NamelessMaterialInstanceDiff", "{0} Diff"), ObjectClass->GetDisplayNameText());
	// if we're diffing one asset against itself
	if (bIsSingleAsset)
	{
		// identify the assumed single asset in the window's title
		FString MaterialInstanceName = OldMaterialInstance ? OldMaterialInstance->GetName() : NewMaterialInstance->GetName();
		WindowTitle = FText::Format(LOCTEXT("NamedMaterialInstanceDiff", "{0} - {1} Diff"), FText::FromString(MaterialInstanceName), ObjectClass->GetDisplayNameText());
	}

	return CreateDiffWindow(WindowTitle, OldMaterialInstance, NewMaterialInstance, OldRevision, NewRevision);
}

#undef LOCTEXT_NAMESPACE
