// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffTool/Widgets/SMaterialRevisionMenu.h"

#include "AssetToolsModule.h"
#include "DiffUtils.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Reply.h"
#include "IAssetTypeActions.h"
#include "ISourceControlModule.h"
#include "ISourceControlRevision.h"
#include "ISourceControlState.h"
#include "Layout/Visibility.h"
#include "MaterialEditorContext.h"
#include "MaterialEditor.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMaterialRevisionMenu"

namespace ESourceControlQueryState
{
	enum Type
	{
		NotQueried,
		QueryInProgress,
		Queried,
	};
}

SMaterialRevisionMenu::~SMaterialRevisionMenu()
{
	// Cancel any operation if this widget is destroyed while in progress
	if (SourceControlQueryState == ESourceControlQueryState::QueryInProgress)
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if (SourceControlQueryOp.IsValid() && SourceControlProvider.CanCancelOperation(SourceControlQueryOp.ToSharedRef()))
		{
			SourceControlProvider.CancelOperation(SourceControlQueryOp.ToSharedRef());
		}
	}
}

void SMaterialRevisionMenu::Construct(const FArguments& InArgs, UObject const* MaterialObj)
{
	bIncludeLocalRevision = InArgs._bIncludeLocalRevision;
	OnRevisionSelected = InArgs._OnRevisionSelected;

	SourceControlQueryState = ESourceControlQueryState::NotQueried;

	ChildSlot
	[
		SAssignNew(MenuBox, SVerticalBox) 
		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.Visibility(this, &SMaterialRevisionMenu::GetInProgressVisibility)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Content()
			[
				SNew(SHorizontalBox) 
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SThrobber)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialGraphDiffMenuOperationInProgress", "Updating history..."))
				] 
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Visibility(this, &SMaterialRevisionMenu::GetCancelButtonVisibility)
					.OnClicked(this, &SMaterialRevisionMenu::OnCancelButtonClicked)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MaterialGraphDiffMenuCancelButton", "Cancel"))
					]
				]
			]
		]
	];

	if (MaterialObj)
	{
		Filename = SourceControlHelpers::PackageFilename(MaterialObj->GetPathName());
		
		// make sure the history info is up to date
		SourceControlQueryOp = ISourceControlOperation::Create<FUpdateStatus>();
		SourceControlQueryOp->SetUpdateHistory(true);
		ISourceControlModule::Get().GetProvider().Execute(SourceControlQueryOp.ToSharedRef(), Filename, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SMaterialRevisionMenu::OnSourceControlQueryComplete));

		SourceControlQueryState = ESourceControlQueryState::QueryInProgress;
	}
}

static void OnDiffRevisionPicked(const FRevisionInfo& RevisionInfo, TWeakObjectPtr<UObject> MaterialObjWeak)
{
	UObject* MaterialObj = MaterialObjWeak.Get();
	if (!MaterialObj)
	{
		return;
	}
		
	// Get the SCC state
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	const FString Filename = SourceControlHelpers::PackageFilename(MaterialObj->GetPathName());
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Filename, EStateCacheUsage::Use);

	if (!SourceControlState.IsValid())
	{
		return;
	}

	for (int32 HistoryIndex = 0; HistoryIndex != SourceControlState->GetHistorySize(); ++HistoryIndex)
	{
		TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);

		if (!Revision.IsValid() || Revision->GetRevision() != RevisionInfo.Revision)
		{
			continue;
		}

		// Get the revision of this package from source control
		if (UPackage* PreviousTempPkg = DiffUtils::LoadPackageForDiff(Revision))
		{
			FString PreviousAssetName = FPaths::GetBaseFilename(Filename, true);
			if (UObject* PreviousMaterial = FindObject<UObject>(PreviousTempPkg, *PreviousAssetName))
			{
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
				FRevisionInfo OldRevision{ Revision->GetRevision(), Revision->GetCheckInIdentifier(), Revision->GetDate() };
				FRevisionInfo CurrentRevision{ TEXT(""), Revision->GetCheckInIdentifier(), Revision->GetDate() };

				AssetToolsModule.Get().DiffAssets(PreviousMaterial, MaterialObj, OldRevision, CurrentRevision);
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("SourceControl.HistoryWindow", "UnableToLoadMaterialGraphs", "Unable to load MaterialGraphs to diff. Please retry."));
		}

		break;
	}
}

void SMaterialRevisionMenu::MakeDiffMenu(UToolMenu* Menu)
{
	FToolMenuSection& DiffSection = Menu->AddSection("SourceControl");
	DiffSection.AddDynamicEntry("SourceControlCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			UMaterialEditorMenuContext* SubMenuContext = InSection.FindContext<UMaterialEditorMenuContext>();
			if (SubMenuContext && SubMenuContext->MaterialEditor.IsValid())
			{
				TWeakPtr<IMaterialEditor> MaterialEditorWeak = SubMenuContext->MaterialEditor;
				
				const auto RevisionMenu = [MaterialEditorWeak]() -> TSharedRef<SWidget>
				{
					TSharedPtr<IMaterialEditor> MaterialEditorStrong = MaterialEditorWeak.Pin();
					if (MaterialEditorStrong && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
					{
						const TArray<UObject*>* ObjectsBeingEdited = MaterialEditorStrong->GetObjectsCurrentlyBeingEdited();
						UObject* MaterialObj = nullptr;
						if (ObjectsBeingEdited)
						{
							for (UObject* Obj : *ObjectsBeingEdited)
							{
								if (Obj &&
									!(Obj->HasAnyFlags(RF_Transient) || Obj->GetPackage() == GetTransientPackage()) &&
									(Obj->IsA<UMaterialInterface>() || Obj->IsA<UMaterialFunction>()))
								{
									MaterialObj = Obj;
									break;
								}
							}
						}

						if (MaterialObj)
						{
							TWeakObjectPtr<UObject> WeakMaterialObj = MaterialObj;
							return SNew(SMaterialRevisionMenu, MaterialObj).OnRevisionSelected_Static(&OnDiffRevisionPicked, WeakMaterialObj);
						}
					}

					FMenuBuilder MenuBuilder(true, nullptr);
					MenuBuilder.AddMenuEntry(LOCTEXT("SourceControlDisabled", "Revision control is disabled"),
												FText(),
												FSlateIcon(),
												FUIAction());

					return MenuBuilder.MakeWidget();
				};

	 			FToolMenuEntry& DiffEntry = InSection.AddEntry(FToolMenuEntry::InitComboButton(
					"Diff",
					FUIAction(),
					FOnGetContent::CreateLambda(RevisionMenu),
					LOCTEXT("Diff", "Diff"),
					LOCTEXT("MaterialEditorDiffToolTip", "Diff against previous revisions"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDiff.ToolbarIcon")
				));
				DiffEntry.StyleNameOverride = "CalloutToolbar";
			}
		}));
}

EVisibility SMaterialRevisionMenu::GetInProgressVisibility() const
{
	return (SourceControlQueryState == ESourceControlQueryState::QueryInProgress) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SMaterialRevisionMenu::GetCancelButtonVisibility() const
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	return SourceControlQueryOp.IsValid() && SourceControlProvider.CanCancelOperation(SourceControlQueryOp.ToSharedRef()) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SMaterialRevisionMenu::OnCancelButtonClicked() const
{
	if (SourceControlQueryOp.IsValid())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		SourceControlProvider.CancelOperation(SourceControlQueryOp.ToSharedRef());
	}

	return FReply::Handled();
}

void SMaterialRevisionMenu::OnSourceControlQueryComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	check(SourceControlQueryOp == InOperation);

	// Add pop-out menu for each revision
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("AddDiffRevision", LOCTEXT("Revisions", "Revisions"));
	if (bIncludeLocalRevision)
	{
		FText const ToolTipText = LOCTEXT("LocalRevisionToolTip", "The current copy you have saved to disk (locally)");

		FOnRevisionSelected OnRevisionSelectedDelegate = OnRevisionSelected;
		auto OnMenuItemSelected = [OnRevisionSelectedDelegate]() {
			OnRevisionSelectedDelegate.ExecuteIfBound(FRevisionInfo::InvalidRevision());
		};

		MenuBuilder.AddMenuEntry(LOCTEXT("LocalRevision", "Local"), ToolTipText, FSlateIcon(), FUIAction(FExecuteAction::CreateLambda(OnMenuItemSelected)));
	}

	if (InResult == ECommandResult::Succeeded)
	{
		// get the cached state
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Filename, EStateCacheUsage::Use);

		if (SourceControlState.IsValid() && SourceControlState->GetHistorySize() > 0)
		{
			// Figure out the highest revision # (so we can label it "Depot")
			int32 LatestRevision = 0;
			for (int32 HistoryIndex = 0; HistoryIndex != SourceControlState->GetHistorySize(); ++HistoryIndex)
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);
				if (Revision.IsValid() && Revision->GetRevisionNumber() > LatestRevision)
				{
					LatestRevision = Revision->GetRevisionNumber();
				}
			}

			for (int32 HistoryIndex = 0; HistoryIndex != SourceControlState->GetHistorySize(); ++HistoryIndex)
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);
				if (Revision.IsValid())
				{
					FInternationalization& I18N = FInternationalization::Get();

					FText Label = FText::Format(LOCTEXT("RevisionNumber", "Revision {0}"), FText::AsNumber(Revision->GetRevisionNumber(), NULL, I18N.GetInvariantCulture()));

					FFormatNamedArguments Args;
					Args.Add(TEXT("CheckInNumber"), FText::AsNumber(Revision->GetCheckInIdentifier(), NULL, I18N.GetInvariantCulture()));
					Args.Add(TEXT("Revision"), FText::FromString(Revision->GetRevision()));
					Args.Add(TEXT("UserName"), FText::FromString(Revision->GetUserName()));
					Args.Add(TEXT("DateTime"), FText::AsDate(Revision->GetDate()));
					Args.Add(TEXT("ChangelistDescription"), FText::FromString(Revision->GetDescription()));
					FText ToolTipText;
					if (ISourceControlModule::Get().GetProvider().UsesChangelists())
					{
						ToolTipText = FText::Format(LOCTEXT("ChangelistToolTip", "CL #{CheckInNumber} {UserName} \n{DateTime} \n{ChangelistDescription}"), Args);
					}
					else
					{
						ToolTipText = FText::Format(LOCTEXT("RevisionToolTip", "{Revision} {UserName} \n{DateTime} \n{ChangelistDescription}"), Args);
					}

					if (LatestRevision == Revision->GetRevisionNumber())
					{
						Label = LOCTEXT("Depot", "Depot");
					}

					FRevisionInfo RevisionInfo{ Revision->GetRevision(), Revision->GetCheckInIdentifier(), Revision->GetDate() };
					FOnRevisionSelected OnRevisionSelectedDelegate = OnRevisionSelected;
					auto OnMenuItemSelected = [RevisionInfo, OnRevisionSelectedDelegate]() 
					{
						OnRevisionSelectedDelegate.ExecuteIfBound(RevisionInfo);
					};
					MenuBuilder.AddMenuEntry(TAttribute<FText>(Label), ToolTipText, FSlateIcon(), FUIAction(FExecuteAction::CreateLambda(OnMenuItemSelected)));
				}
			}
		}
		else if (!bIncludeLocalRevision)
		{
			// Show 'empty' item in toolbar
			MenuBuilder.AddMenuEntry(LOCTEXT("NoRevisionHistory", "No revisions found"),
									 FText(),
									 FSlateIcon(),
									 FUIAction());
		}
	}
	else if (!bIncludeLocalRevision)
	{
		// Show 'empty' item in toolbar
		MenuBuilder.AddMenuEntry(LOCTEXT("NoRevisionHistory", "No revisions found"),
								 FText(),
								 FSlateIcon(),
								 FUIAction());
	}

	MenuBuilder.EndSection();

	MenuBox->AddSlot()
	[
		MenuBuilder.MakeWidget(nullptr, 500)
	];

	SourceControlQueryOp.Reset();
	SourceControlQueryState = ESourceControlQueryState::Queried;
}

#undef LOCTEXT_NAMESPACE
