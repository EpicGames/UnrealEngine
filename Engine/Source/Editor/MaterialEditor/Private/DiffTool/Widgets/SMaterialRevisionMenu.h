// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "Widgets/SCompoundWidget.h"

class FReply;
class FUpdateStatus;
class SVerticalBox;
class UToolMenu;

struct EVisibility;
struct FRevisionInfo;

class SMaterialRevisionMenu : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnRevisionSelected, const FRevisionInfo&)

public : 
SLATE_BEGIN_ARGS(SMaterialRevisionMenu) : _bIncludeLocalRevision(false) {}
	SLATE_ARGUMENT(bool, bIncludeLocalRevision)
	SLATE_EVENT(FOnRevisionSelected, OnRevisionSelected)
SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UObject const* MaterialObj);
	virtual ~SMaterialRevisionMenu();

	// helper function to add the diff menu to a UToolMenu:
	static void MakeDiffMenu(UToolMenu* Menu);

private:
	/** Delegate used to determine the visibility 'in progress' widgets */
	EVisibility GetInProgressVisibility() const;

	/** Delegate used to determine the visibility of the cancel button */
	EVisibility GetCancelButtonVisibility() const;

	/** Delegate used to cancel a source control operation in progress */
	FReply OnCancelButtonClicked() const;

	/** Callback for when the source control operation is complete */
	void OnSourceControlQueryComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/** Whether or not we should include the local revision to the menu */
	bool bIncludeLocalRevision;

	/** Called when we select one Revision from the list in the menu */
	FOnRevisionSelected OnRevisionSelected;

	/** The name of the file we want revision info for */
	FString Filename;

	/** The box we are using to display our menu */
	TSharedPtr<SVerticalBox> MenuBox;

	/** The source control operation in progress */
	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> SourceControlQueryOp;

	/** The state of the SCC query */
	uint32 SourceControlQueryState;
};
