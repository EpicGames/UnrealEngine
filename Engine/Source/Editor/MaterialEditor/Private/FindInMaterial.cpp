// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInMaterial.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "Materials/MaterialExpressionComment.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "FindInMaterial"

//////////////////////////////////////////////////////////////////////////
// FFindInMaterialResult

FFindInMaterialResult::FFindInMaterialResult(const FString& InValue)
:Value(InValue), DuplicationIndex(0), Class(NULL), Pin(), GraphNode(NULL)
{
}

FFindInMaterialResult::FFindInMaterialResult(const FString& InValue, TSharedPtr<FFindInMaterialResult>& InParent, UClass* InClass, int InDuplicationIndex)
: Parent(InParent), Value(InValue), DuplicationIndex(InDuplicationIndex), Class(InClass), Pin(), GraphNode(NULL)
{
}

FFindInMaterialResult::FFindInMaterialResult(const FString& InValue, TSharedPtr<FFindInMaterialResult>& InParent, UEdGraphPin* InPin)
: Parent(InParent), Value(InValue), DuplicationIndex(0), Class(nullptr), Pin(InPin), GraphNode(NULL)
{
}

FFindInMaterialResult::FFindInMaterialResult(const FString& InValue, TSharedPtr<FFindInMaterialResult>& InParent, UEdGraphNode* InNode)
: Parent(InParent), Value(InValue), DuplicationIndex(0), Class(InNode->GetClass()), Pin(), GraphNode(InNode)
{
	if (GraphNode.IsValid())
	{
		CommentText = GraphNode->NodeComment;
	}
}

FReply FFindInMaterialResult::OnClick(TWeakPtr<class FMaterialEditor> MaterialEditor)
{
	if (GraphNode.IsValid())
	{
		MaterialEditor.Pin()->JumpToNode(GraphNode.Get());
	}
	else if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		MaterialEditor.Pin()->JumpToNode(ResolvedPin->GetOwningNode());
	}
	return FReply::Handled();
}

FText FFindInMaterialResult::GetCategory() const
{
	if (Class == nullptr && Pin.Get())
	{
		return LOCTEXT("PinCategory", "Pin");
	}

	return LOCTEXT("NodeCategory", "Node");
}

TSharedRef<SWidget> FFindInMaterialResult::CreateIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = NULL;
	if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		if (ResolvedPin->PinType.IsArray())
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.ArrayPinIcon"));
		}
		else if (ResolvedPin->PinType.bIsReference)
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.RefPinIcon"));
		}
		else
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
		}
		const UEdGraphSchema* Schema = ResolvedPin->GetSchema();
			IconColor = Schema->GetPinTypeColor(ResolvedPin->PinType);
		}
	else if (GraphNode.IsValid())
	{
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.NodeGlyph"));
	}

	return 	SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText(GetCategory());
}

FString FFindInMaterialResult::GetCommentText() const
{
	return CommentText;
}

FString FFindInMaterialResult::GetValueText() const
{
	FString DisplayVal;

	if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		if ((!ResolvedPin->DefaultValue.IsEmpty() || !ResolvedPin->AutogeneratedDefaultValue.IsEmpty() || ResolvedPin->DefaultObject || !ResolvedPin->DefaultTextValue.IsEmpty())
			&& (ResolvedPin->Direction == EGPD_Input)
			&& (ResolvedPin->LinkedTo.Num() == 0))
		{
			if (ResolvedPin->DefaultObject)
			{
				const AActor* DefaultActor = Cast<AActor>(ResolvedPin->DefaultObject);
				DisplayVal = FString::Printf(TEXT("(%s)"), DefaultActor ? *DefaultActor->GetActorLabel() : *ResolvedPin->DefaultObject->GetName());
			}
			else if (!ResolvedPin->DefaultValue.IsEmpty())
			{
				DisplayVal = FString::Printf(TEXT("(%s)"), *ResolvedPin->DefaultValue);
			}
			else if (!ResolvedPin->AutogeneratedDefaultValue.IsEmpty())
			{
				DisplayVal = FString::Printf(TEXT("(%s)"), *ResolvedPin->AutogeneratedDefaultValue);
			}
			else if (!ResolvedPin->DefaultTextValue.IsEmpty())
			{
				DisplayVal = FString::Printf(TEXT("(%s)"), *ResolvedPin->DefaultTextValue.ToString());
			}
		}
	}

	return DisplayVal;
}

//////////////////////////////////////////////////////////////////////////
// SFindInMaterial

void SFindInMaterial::Construct(const FArguments& InArgs, TSharedPtr<FMaterialEditor> InMaterialEditor)
{
	MaterialEditorPtr = InMaterialEditor;

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SAssignNew(SearchTextField, SSearchBox)
				.HintText(LOCTEXT("GraphSearchHint", "Search"))
				.OnTextChanged(this, &SFindInMaterial::OnSearchTextChanged)
				.OnTextCommitted(this, &SFindInMaterial::OnSearchTextCommitted)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.f, 4.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(TreeView, STreeViewType)
				.TreeItemsSource(&ItemsFound)
				.OnGenerateRow(this, &SFindInMaterial::OnGenerateRow)
				.OnGetChildren(this, &SFindInMaterial::OnGetChildren)
				.OnSelectionChanged(this, &SFindInMaterial::OnTreeSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SFindInMaterial::OnTreeSelectionDoubleClick)
				.SelectionMode(ESelectionMode::Multi)
			]
		]
	];
}

void SFindInMaterial::FocusForUse()
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
}

void SFindInMaterial::PopulateSearchItems(const TArray<UEdGraphNode*>& Nodes)
{
	if (Nodes.Num() == 0)
	{
		return;
	}

	// Reset to default state.
	SearchTextField->SetText(FText::GetEmpty());
	RootSearchResult.Reset();
	RootSearchResult = FSearchResult(new FFindInMaterialResult(FString("MaterialTreeRoot")));
	ItemsFound.Empty();

	for (UEdGraphNode* Node : Nodes)
	{
		const FString NodeName = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
		const FString NodeType = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();
		ItemsFound.Add(FSearchResult(new FFindInMaterialResult(NodeName == NodeType ? NodeName : NodeName + " - " + NodeType, RootSearchResult, Node)));
	}

	// Refresh the tree and expand items.
	TreeView->RequestTreeRefresh();

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		TreeView->SetItemExpansion(*It, true);
	}
}

void SFindInMaterial::OnSearchTextChanged(const FText& Text)
{
	SearchValue = Text.ToString();
}

void SFindInMaterial::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		InitiateSearch();
	}
}

void SFindInMaterial::InitiateSearch()
{
	TArray<FString> Tokens;
	if (SearchValue.Contains("\"") && SearchValue.ParseIntoArray(Tokens, TEXT("\""), true) > 0)
	{
		for (auto &TokenIt : Tokens)
		{
			// we have the token, we don't need the quotes anymore, they'll just confused the comparison later on
			TokenIt = TokenIt.TrimQuotes();
			// We remove the spaces as all later comparison strings will also be de-spaced
			TokenIt = TokenIt.Replace(TEXT(" "), TEXT(""));
		}

		// due to being able to handle multiple quoted blocks like ("Make Epic" "Game Now") we can end up with
		// and empty string between (" ") blocks so this simply removes them
		struct FRemoveMatchingStrings
		{
			bool operator()(const FString& RemovalCandidate) const
			{
				return RemovalCandidate.IsEmpty();
			}
		};
		Tokens.RemoveAll(FRemoveMatchingStrings());
	}
	else
	{
		// unquoted search equivalent to a match-any-of search
		SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);
	}

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		TreeView->SetItemExpansion(*It, false);
	}
	ItemsFound.Empty();
	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchValue);
		MatchTokens(Tokens);
	}

	// Insert a fake result to inform user if none found
	if (ItemsFound.Num() == 0)
	{
		ItemsFound.Add(FSearchResult(new FFindInMaterialResult(LOCTEXT("MaterialSearchNoResults", "No Results found").ToString())));
	}

	TreeView->RequestTreeRefresh();

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		TreeView->SetItemExpansion(*It, true);
	}
}

void SFindInMaterial::MatchTokens(const TArray<FString> &Tokens)
{
	RootSearchResult.Reset();

	UEdGraph* Graph = MaterialEditorPtr.Pin()->Material->MaterialGraph;
	MatchTokensInGraph(Graph, Tokens);
}

void SFindInMaterial::MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens)
{
	if (Graph == NULL)
	{
		return;
	}

	RootSearchResult = FSearchResult(new FFindInMaterialResult(FString("MaterialTreeRoot")));

	for (auto It(Graph->Nodes.CreateConstIterator()); It; ++It)
	{
		UEdGraphNode* Node = *It;

		const FString NodeName = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
		const FString NodeType = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();
		FSearchResult NodeResult(new FFindInMaterialResult(NodeName == NodeType ? NodeName : NodeName + " - " + NodeType, RootSearchResult, Node));

		FString NodeSearchString = NodeName + NodeType + Node->NodeComment;
		NodeSearchString = NodeSearchString.Replace(TEXT(" "), TEXT(""));

		bool bNodeMatchesSearch = StringMatchesSearchTokens(Tokens, NodeSearchString);

		// Use old Material Expression search functions too
		if (!bNodeMatchesSearch)
		{
			bool bMatchesAllTokens = true;
			if (UMaterialGraphNode* MatNode = Cast<UMaterialGraphNode>(Node))
			{
				for (int32 Index = 0; Index < Tokens.Num(); ++Index)
				{
					if (!MatNode->MaterialExpression->DeepSearchQuery(*Tokens[Index], NodeResult))
					{
						bMatchesAllTokens = false;
						break;
					}
				}
			}
			else if (UMaterialGraphNode_Comment* MatComment = Cast<UMaterialGraphNode_Comment>(Node))
			{
				for (int32 Index = 0; Index < Tokens.Num(); ++Index)
				{
					if (!MatComment->MaterialExpressionComment->MatchesSearchQuery(*Tokens[Index]))
					{
						bMatchesAllTokens = false;
						break;
					}
				}
			}
			else
			{
				bMatchesAllTokens = false;
			}
			if (bMatchesAllTokens)
			{
				bNodeMatchesSearch = true;
			}
		}

		for (TArray<UEdGraphPin*>::TIterator PinIt(Node->Pins); PinIt; ++PinIt)
		{
			UEdGraphPin* Pin = *PinIt;
			if (Pin && Pin->PinFriendlyName.CompareTo(FText::FromString(TEXT(" "))) != 0)
			{
				FText PinName = Pin->GetSchema()->GetPinDisplayName(Pin);
				FString PinSearchString = Pin->PinName.ToString() + Pin->PinFriendlyName.ToString() + Pin->DefaultValue + Pin->PinType.PinCategory.ToString() + Pin->PinType.PinSubCategory.ToString() + (Pin->PinType.PinSubCategoryObject.IsValid() ? Pin->PinType.PinSubCategoryObject.Get()->GetFullName() : TEXT(""));
				PinSearchString = PinSearchString.Replace(TEXT(" "), TEXT(""));
				if (StringMatchesSearchTokens(Tokens, PinSearchString))
				{
					FSearchResult PinResult(new FFindInMaterialResult(PinName.ToString(), NodeResult, Pin));
					NodeResult->Children.Add(PinResult);
				}
			}
		}

		if ((NodeResult->Children.Num() > 0) || bNodeMatchesSearch)
		{
			ItemsFound.Add(NodeResult);
		}
	}

	for (const UEdGraph* Subgraph : Graph->SubGraphs)
	{
		MatchTokensInGraph(Subgraph, Tokens);
	}
}

TSharedRef<ITableRow> SFindInMaterial::OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FString CommentText = InItem->GetCommentText();

	return SNew(STableRow< TSharedPtr<FFindInMaterialResult> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				InItem->CreateIcon()
			]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(InItem->bEmbeddedViaNode ? FText::Format(LOCTEXT("MaterialResultSearchValueEmbeddedFmt", "Is found inside of {0}"), FText::FromString(InItem->Value)) : FText::FromString(InItem->Value))
					.HighlightText(HighlightText)
					.ToolTipText(FText::Format(LOCTEXT("MaterialResultSearchToolTipFmt", "{0} : {1}"), InItem->GetCategory(), FText::FromString(InItem->Value)))
				]
			+ SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(InItem->GetValueText()))
					.HighlightText(HighlightText)
				]
			+ SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(CommentText.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("NodeCommentFmt", "Node Comment:[{0}]"), FText::FromString(CommentText)))
					.ColorAndOpacity(FLinearColor::Yellow)
					.HighlightText(HighlightText)
				]
		];
}

void SFindInMaterial::OnGetChildren(FSearchResult InItem, TArray< FSearchResult >& OutChildren)
{
	OutChildren += InItem->Children;
}

void SFindInMaterial::OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		Item->OnClick(MaterialEditorPtr);
	}
}

void SFindInMaterial::OnTreeSelectionDoubleClick(FSearchResult Item)
{
	if (Item.IsValid())
	{
		Item->OnClick(MaterialEditorPtr);
	}
}

bool SFindInMaterial::StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString)
{
	bool bFoundAllTokens = true;
	//search the entry for each token, it must have all of them to pass
	for (auto TokIT(Tokens.CreateConstIterator()); TokIT; ++TokIT)
	{
		const FString& Token = *TokIT;
		if (!ComparisonString.Contains(Token))
		{
			bFoundAllTokens = false;
			break;
		}
	}
	return bFoundAllTokens;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
