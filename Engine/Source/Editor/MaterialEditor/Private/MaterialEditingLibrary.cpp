// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditingLibrary.h"
#include "Engine/Texture.h"
#include "Editor.h"
#include "MaterialEditor.h"
#include "MaterialInstanceEditor.h"
#include "MaterialEditorUtilities.h"
#include "MaterialShared.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionCollectionTransform.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionStaticComponentMaskParameter.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "MaterialStatsCommon.h"
#include "Particles/ParticleSystemComponent.h"
#include "EditorSupportDelegates.h"
#include "Misc/RuntimeErrors.h"
#include "SceneTypes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DebugViewModeHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ShaderCompiler.h"
#include "UObject/UObjectIterator.h"
#include "MaterialCachedData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialEditingLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogMaterialEditingLibrary, Log, All);

/** Util to find expression  */
static FExpressionInput* GetExpressionInputByName(UMaterialExpression* Expression, const FName InputName)
{
	check(Expression);
	FExpressionInput* Result = nullptr;

	// Return first input if no name specified
	if (InputName.IsNone())
	{
		return Expression->GetInput(0);
	}

	// Get all inputs. Get name of each input, see if its the one we want
	for (FExpressionInputIterator It{ Expression }; It; ++It)
	{
		FName TestName;
		if (UMaterialExpressionMaterialFunctionCall* FuncCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			// If a function call, don't want to compare string with type postfix
			TestName = FuncCall->GetInputNameWithType(It.Index, false);
		}
		else
		{
			const FName ExpressionInputName = Expression->GetInputName(It.Index);
			TestName = UMaterialGraphNode::GetShortenPinName(ExpressionInputName);
		}

		if (TestName == InputName)
		{
			return It.Input;
		}
	}

	return nullptr;
}

static int32 GetExpressionOutputIndexByName(UMaterialExpression* Expression, const FName OutputName)
{
	check(Expression);
	
	int32 Result = INDEX_NONE;

	if (Expression->Outputs.Num() == 0)
	{
		// leave as INDEX_NONE
	}
	// Return first output if no name specified
	else if (OutputName.IsNone())
	{
		Result = 0;
	}
	else
	{
		// Iterate over outputs and look for name match
		for (int OutIdx = 0; OutIdx < Expression->Outputs.Num(); OutIdx++)
		{
			bool bFoundMatch = false;

			FExpressionOutput& Output = Expression->Outputs[OutIdx];
			// If output name is no empty - see if it matches
			if(!Output.OutputName.IsNone())
			{
				if (OutputName == Output.OutputName)
				{
					bFoundMatch = true;
				}
			}
			// if it is empty we look for R/G/B/A
			else
			{
				if (Output.MaskR && !Output.MaskG && !Output.MaskB && !Output.MaskA && OutputName == TEXT("R"))
				{
					bFoundMatch = true;
				}
				else if (!Output.MaskR && Output.MaskG && !Output.MaskB && !Output.MaskA && OutputName == TEXT("G"))
				{
					bFoundMatch = true;
				}
				else if (!Output.MaskR && !Output.MaskG && Output.MaskB && !Output.MaskA && OutputName == TEXT("B"))
				{
					bFoundMatch = true;
				}
				else if (!Output.MaskR && !Output.MaskG && !Output.MaskB && Output.MaskA && OutputName == TEXT("A"))
				{
					bFoundMatch = true;
				}
			}

			// Got a match, remember the index, exit iteration
			if (bFoundMatch)
			{
				Result = OutIdx;
				break;
			}
		}
	}

	return Result;
}

namespace MaterialEditingLibraryImpl
{
	struct FMaterialExpressionLayoutInfo
	{
		static const int32 LayoutWidth = 260;

		UMaterialExpression* Connected = nullptr;
		int32 Column = 0;
		int32 Row = 0;
	};

	void LayoutMaterialExpression( UMaterialExpression* MaterialExpression, UMaterialExpression* ConnectedExpression, TMap< UMaterialExpression*, FMaterialExpressionLayoutInfo >& MaterialExpressionsToLayout, int32 Row, int32 Depth )
	{
		if ( !MaterialExpression )
		{
			return;
		}

		FMaterialExpressionLayoutInfo LayoutInfo;

		if ( MaterialExpressionsToLayout.Contains( MaterialExpression ) )
		{
			LayoutInfo = MaterialExpressionsToLayout[ MaterialExpression ];
		}

		LayoutInfo.Row = FMath::Max( LayoutInfo.Row, Row );

		if ( Depth > LayoutInfo.Column )
		{
			LayoutInfo.Connected = ConnectedExpression;
		}

		LayoutInfo.Column = FMath::Max( LayoutInfo.Column, Depth );

		MaterialExpressionsToLayout.Add( MaterialExpression ) = MoveTemp( LayoutInfo );

		for (FExpressionInputIterator It{ MaterialExpression }; It; ++It)
		{
			LayoutMaterialExpression( It->Expression, MaterialExpression, MaterialExpressionsToLayout, Row, Depth + 1 );
		}
	}

	void LayoutMaterialExpressions( UObject* MaterialOrMaterialFunction )
	{
		if ( !MaterialOrMaterialFunction )
		{
			return;
		}

		TMap< UMaterialExpression*, FMaterialExpressionLayoutInfo > MaterialExpressionsToLayout;

		if ( UMaterial* Material = Cast< UMaterial >( MaterialOrMaterialFunction ) )
		{
			for ( int32 MaterialPropertyIndex = 0; MaterialPropertyIndex < MP_MAX; ++MaterialPropertyIndex )
			{
				FExpressionInput* ExpressionInput = Material->GetExpressionInputForProperty( EMaterialProperty(MaterialPropertyIndex) );
		
				if ( ExpressionInput  )
				{
					LayoutMaterialExpression( ExpressionInput->Expression, nullptr, MaterialExpressionsToLayout, MaterialPropertyIndex, 0 );
				}
			}
		}
		else if ( UMaterialFunction* MaterialFunction = Cast< UMaterialFunction >( MaterialOrMaterialFunction ) )
		{
			TArray< FFunctionExpressionInput > Inputs;
			TArray< FFunctionExpressionOutput > Outputs;
			
			MaterialFunction->GetInputsAndOutputs( Inputs, Outputs );

			int32 InputIndex = 0;

			if ( Inputs.Num() > 0 )
			{
				for ( FFunctionExpressionInput& FunctionExpressionInput : Inputs )
				{
					LayoutMaterialExpression( FunctionExpressionInput.ExpressionInput, nullptr, MaterialExpressionsToLayout, ++InputIndex, 0 );
				}
			}
			else
			{
				for ( FFunctionExpressionOutput& FunctionExpressionOutput : Outputs )
				{
					LayoutMaterialExpression( FunctionExpressionOutput.ExpressionOutput, nullptr, MaterialExpressionsToLayout, ++InputIndex, 0 );
				}
			}
		}

		TMap< int32, TMap< int32, bool > > UsedColumnRows;

		TMap< int32, int32 > ColumnsHeights;

		for ( TMap< UMaterialExpression*, FMaterialExpressionLayoutInfo >::TIterator It = MaterialExpressionsToLayout.CreateIterator(); It; ++It )
		{
			UMaterialExpression* MaterialExpression = It->Key;
			FMaterialExpressionLayoutInfo& LayoutInfo = It->Value;

			if ( !UsedColumnRows.Contains( LayoutInfo.Column ) )
			{
				UsedColumnRows.Add( LayoutInfo.Column );
			}

			while ( UsedColumnRows[ LayoutInfo.Column ].Contains( LayoutInfo.Row ) )
			{
				++LayoutInfo.Row;
			}

			UsedColumnRows[ LayoutInfo.Column ].Add( LayoutInfo.Row ) = true;

			if ( !ColumnsHeights.Contains( LayoutInfo.Column ) )
			{
				ColumnsHeights.Add( LayoutInfo.Column ) = 0;
			}

			int32& ColumnHeight = ColumnsHeights[ LayoutInfo.Column ];

			MaterialExpression->MaterialExpressionEditorX = -FMaterialExpressionLayoutInfo::LayoutWidth * ( LayoutInfo.Column + 1 );

			int32 ConnectedHeight = LayoutInfo.Connected ? LayoutInfo.Connected->MaterialExpressionEditorY : 0;
			MaterialExpression->MaterialExpressionEditorY = FMath::Max( ColumnHeight, ConnectedHeight );

			ColumnHeight = MaterialExpression->MaterialExpressionEditorY + MaterialExpression->GetHeight() + ME_STD_HPADDING;
		}
	}

	IMaterialEditor* FindMaterialEditorForAsset(UObject* InAsset)
	{
		if (IAssetEditorInstance* AssetEditorInstance = (InAsset != nullptr) ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InAsset, false) : nullptr)
		{
			// Ensure this is not a UMaterialInstanceDynamic, as that doesn't use IMaterialEditor as its editor
			if (!InAsset->IsA(UMaterialInstanceDynamic::StaticClass()))
			{
				return static_cast<IMaterialEditor*>(AssetEditorInstance);
			}
		}

		return nullptr;
	}

	void NotifyMaterialEditorOfChange(UMaterialInterface* InAsset)
	{
		if (!InAsset)
		{
			return;
		}

		if (IMaterialEditor* MaterialEditor = FindMaterialEditorForAsset(InAsset))
		{
			MaterialEditor->NotifyExternalMaterialChange();
		}
	}

	FMaterialInstanceEditor* FindMaterialInstanceEditorForAsset(UObject* InAsset)
	{
		if (IAssetEditorInstance* AssetEditorInstance = (InAsset != nullptr) ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InAsset, false) : nullptr)
		{
			// Ensure this is not a UMaterialInstanceDynamic, as that doesn't use FMaterialInstanceEditor as its editor
			if (!InAsset->IsA(UMaterialInstanceDynamic::StaticClass()))
			{
				return static_cast<FMaterialInstanceEditor*>(AssetEditorInstance);
			}
		}

		return nullptr;
	}

	static bool DoesEditedAssetDependOn(UObject* EditedAsset, UObject* ChangedAsset)
	{
		if (!EditedAsset || !ChangedAsset)
		{
			return false;
		}

		if (UMaterialInterface* EditedMaterial = Cast<UMaterialInterface>(EditedAsset))
		{
			if (UMaterialFunctionInterface* ChangedFunction = Cast<UMaterialFunctionInterface>(ChangedAsset))
			{
				TArray<UMaterialFunctionInterface*> DependentFunctions;
				EditedMaterial->GetDependentFunctions(DependentFunctions);
				return DependentFunctions.Contains(ChangedFunction);
			}

			if (UMaterialInterface* ChangedMaterial = Cast<UMaterialInterface>(ChangedAsset))
			{
				for (UMaterialInterface* Current = EditedMaterial; Current; )
				{
					UMaterialInstance* AsInstance = Cast<UMaterialInstance>(Current);
					if (!AsInstance)
					{
						break;
					}
					if (AsInstance->Parent == ChangedMaterial)
					{
						return true;
					}
					Current = AsInstance->Parent;
				}
			}
			return false;
		}

		if (UMaterialFunctionInterface* EditedFunction = Cast<UMaterialFunctionInterface>(EditedAsset))
		{
			if (UMaterialFunctionInterface* ChangedFunction = Cast<UMaterialFunctionInterface>(ChangedAsset))
			{
				TArray<UMaterialFunctionInterface*> DependentFunctions;
				EditedFunction->GetDependentFunctions(DependentFunctions);
				return DependentFunctions.Contains(ChangedFunction);
			}
		}

		return false;
	}

	// Refreshes the editor for ChangedAsset itself (if open) plus any open editors whose edited asset depends on it.
	static void RefreshAssetEditorAndDependents(UObject* ChangedAsset)
	{
		if (!ChangedAsset || !GEditor)
		{
			return;
		}

		UAssetEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (!Subsystem)
		{
			return;
		}

		TArray<UObject*> AssetsToRefresh;
		AssetsToRefresh.Add(ChangedAsset);

		for (UObject* EditedAsset : Subsystem->GetAllEditedAssets())
		{
			if (!EditedAsset || EditedAsset == ChangedAsset)
			{
				continue;
			}
			if (DoesEditedAssetDependOn(EditedAsset, ChangedAsset))
			{
				AssetsToRefresh.Add(EditedAsset);
			}
		}

		for (UObject* Asset : AssetsToRefresh)
		{
			if (Asset->IsA(UMaterialInstanceConstant::StaticClass()))
			{
				if (FMaterialInstanceEditor* InstanceEditor = FindMaterialInstanceEditorForAsset(Asset))
				{
					InstanceEditor->RebuildMaterialInstanceEditor();
					InstanceEditor->Refresh();
				}
			}
			else if (IMaterialEditor* MaterialEditor = FindMaterialEditorForAsset(Asset))
			{
				MaterialEditor->RefreshGraphFromOriginal();
			}
		}
	}

}

void UMaterialEditingLibrary::RebuildMaterialInstanceEditors(UMaterial* BaseMaterial)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialEditingLibrary::RebuildMaterialInstanceEditors)

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();

	TSet<FMaterialInstanceEditor*> InstanceEditorsToUpdate;

	for (int32 AssetIdx = 0; AssetIdx < EditedAssets.Num(); AssetIdx++)
	{
		UObject* EditedAsset = EditedAssets[AssetIdx];

		UMaterialInstance* SourceInstance = Cast<UMaterialInstance>(EditedAsset);
		if (!SourceInstance)
		{
			// Check to see if the EditedAssets are from material instance editor
			UMaterialEditorInstanceConstant* EditorInstance = Cast<UMaterialEditorInstanceConstant>(EditedAsset);
			if (EditorInstance && EditorInstance->SourceInstance)
			{
				SourceInstance = EditorInstance->SourceInstance;
			}
		}

		if (SourceInstance != nullptr)
		{
			UMaterial* MICOriginalMaterial = SourceInstance->GetMaterial();
			if (MICOriginalMaterial == BaseMaterial)
			{
				if (FMaterialInstanceEditor* MaterialInstanceEditor = MaterialEditingLibraryImpl::FindMaterialInstanceEditorForAsset(SourceInstance))
				{
					InstanceEditorsToUpdate.Add(MaterialInstanceEditor);
				}
			}
		}
	}

	// Update the unique list of instance editors.
	for (FMaterialInstanceEditor* MaterialInstanceEditor : InstanceEditorsToUpdate)
	{
		MaterialInstanceEditor->RebuildMaterialInstanceEditor();
	}
}

void UMaterialEditingLibrary::RefreshMaterialEditor(UMaterial* Material)
{
	MaterialEditingLibraryImpl::RefreshAssetEditorAndDependents(Material);
}

void UMaterialEditingLibrary::RefreshMaterialFunctionEditor(UMaterialFunction* MaterialFunction)
{
	MaterialEditingLibraryImpl::RefreshAssetEditorAndDependents(MaterialFunction);
}

void UMaterialEditingLibrary::RefreshMaterialInstanceEditor(UMaterialInstanceConstant* Instance)
{
	MaterialEditingLibraryImpl::RefreshAssetEditorAndDependents(Instance);
}

void UMaterialEditingLibrary::RebuildMaterialInstanceEditors(UMaterialFunction* BaseFunction)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialEditingLibrary::RebuildMaterialInstanceEditors)

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();

	TSet<FMaterialInstanceEditor*> InstanceEditorsToUpdate;

	for (int32 AssetIdx = 0; AssetIdx < EditedAssets.Num(); AssetIdx++)
	{
		UObject* EditedAsset = EditedAssets[AssetIdx];

		UMaterialFunctionInstance* FunctionInstance = Cast<UMaterialFunctionInstance>(EditedAsset);	
		UMaterialInstance* SourceInstance = Cast<UMaterialInstance>(EditedAsset);
	
		if (FunctionInstance)
		{
			// Update function instances that are children of this material function	
			if (BaseFunction && BaseFunction == FunctionInstance->GetBaseFunction())
			{
				if (FMaterialInstanceEditor* MaterialInstanceEditor = MaterialEditingLibraryImpl::FindMaterialInstanceEditorForAsset(EditedAsset))
				{
					InstanceEditorsToUpdate.Add(MaterialInstanceEditor);
				}
			}
		}
		else
		{
			if (!SourceInstance)
			{
				// Check to see if the EditedAssets are from material instance editor
				UMaterialEditorInstanceConstant* EditorInstance = Cast<UMaterialEditorInstanceConstant>(EditedAsset);
				if (EditorInstance && EditorInstance->SourceInstance)
				{
					SourceInstance = EditorInstance->SourceInstance;
				}
			}

			// Ensure the material instance is valid and not a UMaterialInstanceDynamic, as that doesn't use FMaterialInstanceEditor as its editor
			if (SourceInstance != nullptr && !SourceInstance->IsA(UMaterialInstanceDynamic::StaticClass()))
			{
				TArray<UMaterialFunctionInterface*> DependentFunctions;
				SourceInstance->GetDependentFunctions(DependentFunctions);

				if (BaseFunction && (DependentFunctions.Contains(BaseFunction) || DependentFunctions.Contains(BaseFunction->ParentFunction)))
				{
					if (FMaterialInstanceEditor* MaterialInstanceEditor = MaterialEditingLibraryImpl::FindMaterialInstanceEditorForAsset(EditedAsset))
					{
						InstanceEditorsToUpdate.Add(MaterialInstanceEditor);
					}
				}
			}
		}
	}

	// Update the unique list of instance editors.
	for (FMaterialInstanceEditor* MaterialInstanceEditor : InstanceEditorsToUpdate)
	{
		MaterialInstanceEditor->RebuildMaterialInstanceEditor();
	}
}

TArray<UMaterialExpression*> UMaterialEditingLibrary::GetMaterialFunctionExpressions(const UMaterialFunction* MaterialFunction)
{
	if (MaterialFunction)
	{
		return TArray<UMaterialExpression*>(MaterialFunction->GetExpressions());
	}

	return {};
}

TArray<UMaterialExpression*> UMaterialEditingLibrary::GetMaterialExpressions(const UMaterial* Material)
{
	if (Material)
	{
		return TArray<UMaterialExpression*>(Material->GetExpressions());
	}

	return {};
}

int32 UMaterialEditingLibrary::GetNumMaterialExpressions(const UMaterial* Material)
{
	int32 Result = 0;
	if (Material)
	{
		Result = Material->GetExpressions().Num();
	}
	return Result;
}

void UMaterialEditingLibrary::DeleteAllMaterialExpressions(UMaterial* Material)
{
	if (Material)
	{
		for (UMaterialExpression* Expression : Material->GetExpressions())
		{
			DeleteMaterialExpression(Material, Expression);
		}
	}
}

/** Util to iterate over list of expressions, and break any links to specified expression */
static void BreakLinksToExpression(TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions, UMaterialExpression* Expression)
{
	// Need to find any other expressions which are connected to this one, and break link
	for (UMaterialExpression* TestExp : Expressions)
	{
		// Don't check myself, though that shouldn't really matter...
		if (TestExp != Expression)
		{
			for (FExpressionInputIterator It{ TestExp }; It; ++It)
			{
				if (It->Expression == Expression)
				{
					It->Expression = nullptr;
				}
			}
		}
	}
}

void UMaterialEditingLibrary::DeleteMaterialExpression(UMaterial* Material, UMaterialExpression* Expression)
{
	if (Material && Expression && Expression->GetOuter() == Material)
	{
		// Break any links to this expression
		BreakLinksToExpression(Material->GetExpressions(), Expression);

		// Check material parameter inputs, to make sure expression is not connected to it
		for (int32 InputIndex = 0; InputIndex < MP_MAX; InputIndex++)
		{
			FExpressionInput* Input = Material->GetExpressionInputForProperty((EMaterialProperty)InputIndex);
			if (Input && Input->Expression == Expression)
			{
				Input->Expression = nullptr;
			}
		}

		Material->RemoveExpressionParameter(Expression);

		Material->GetExpressionCollection().RemoveExpression(Expression);

		Expression->MarkAsGarbage();

		Material->MarkPackageDirty();

		MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Material);
	}
}

void UMaterialEditingLibrary::DeleteUnusedExpressions(UMaterial* Material)
{
	if (!ensureAsRuntimeWarning(Material != nullptr))
	{
		return;
	}
	TArray<UMaterialExpression*> Unused;
	UMaterialGraph::GetUnusedMaterialExpressions(Material, Unused);
	for (UMaterialExpression* Expression : Unused)
	{
		DeleteMaterialExpression(Material, Expression);
	}
}


UMaterialExpression* UMaterialEditingLibrary::CreateMaterialExpression(UMaterial* Material, TSubclassOf<UMaterialExpression> ExpressionClass, int32 NodePosX, int32 NodePosY)
{
	return CreateMaterialExpressionEx(Material, nullptr, ExpressionClass, nullptr, NodePosX, NodePosY);
}

UMaterialExpression* UMaterialEditingLibrary::DuplicateMaterialExpression(UMaterial* Material, UMaterialFunction* MaterialFunction, UMaterialExpression* Expression)
{
	UMaterialExpression* NewExpression = nullptr;
	if (Material || MaterialFunction)
	{
		UObject* ExpressionOuter = Material;
		if (MaterialFunction)
		{
			ExpressionOuter = MaterialFunction;
		}

		NewExpression = DuplicateObject(Expression, ExpressionOuter);

		if (Material)
		{
			Material->GetExpressionCollection().AddExpression(NewExpression);
			NewExpression->Material = Material;
		}

		if (MaterialFunction && !Material)
		{
			MaterialFunction->GetExpressionCollection().AddExpression(NewExpression);
		}

		// Create a GUID for the node
		NewExpression->UpdateMaterialExpressionGuid(true, true);

		if (Material)
		{
			Material->AddExpressionParameter(NewExpression, Material->EditorParameters);
		}

		NewExpression->MarkPackageDirty();
	}
	return NewExpression;
}

UMaterialExpression* UMaterialEditingLibrary::CreateMaterialExpressionInFunction(UMaterialFunction* MaterialFunction, TSubclassOf<UMaterialExpression> ExpressionClass, int32 NodePosX, int32 NodePosY)
{
	return CreateMaterialExpressionEx(nullptr, MaterialFunction, ExpressionClass, nullptr, NodePosX, NodePosY);
}


UMaterialExpression* UMaterialEditingLibrary::CreateMaterialExpressionEx(UMaterial* Material, UMaterialFunction* MaterialFunction, TSubclassOf<UMaterialExpression> ExpressionClass,
	UObject* SelectedAsset, int32 NodePosX, int32 NodePosY, bool bAllowMarkingPackageDirty)
{
	UMaterialExpression* NewExpression = nullptr;
	if (Material || MaterialFunction)
	{
		UObject* ExpressionOuter = Material;
		if (MaterialFunction)
		{
			ExpressionOuter = MaterialFunction;
		}

		NewExpression = NewObject<UMaterialExpression>(ExpressionOuter, ExpressionClass.Get(), NAME_None, RF_Transactional);

		if (Material)
		{
			Material->GetExpressionCollection().AddExpression(NewExpression);
			NewExpression->Material = Material;
		}

		if (MaterialFunction && !Material)
		{
			MaterialFunction->GetExpressionCollection().AddExpression(NewExpression);
		}

		NewExpression->MaterialExpressionEditorX = NodePosX;
		NewExpression->MaterialExpressionEditorY = NodePosY;

		// Create a GUID for the node
		NewExpression->UpdateMaterialExpressionGuid(true, bAllowMarkingPackageDirty);

		if (SelectedAsset)
		{
			// If the user is adding a texture, automatically assign the currently selected texture to it.
			UMaterialExpressionTextureBase* METextureBase = Cast<UMaterialExpressionTextureBase>(NewExpression);
			if (METextureBase)
			{
				if (UTexture* SelectedTexture = Cast<UTexture>(SelectedAsset))
				{
					METextureBase->Texture = SelectedTexture;
				}
				METextureBase->AutoSetSampleType();
			}

			UMaterialExpressionMaterialFunctionCall* MEMaterialFunction = Cast<UMaterialExpressionMaterialFunctionCall>(NewExpression);
			if (MEMaterialFunction)
			{
				MEMaterialFunction->SetMaterialFunction(Cast<UMaterialFunction>(SelectedAsset));
			}

			UMaterialExpressionCollectionParameter* MECollectionParameter = Cast<UMaterialExpressionCollectionParameter>(NewExpression);
			if (MECollectionParameter)
			{
				MECollectionParameter->Collection = Cast<UMaterialParameterCollection>(SelectedAsset);
			}

			UMaterialExpressionCollectionTransform* MECollectionTransform = Cast<UMaterialExpressionCollectionTransform>(NewExpression);
			if (MECollectionTransform)
			{
				MECollectionTransform->Collection = Cast<UMaterialParameterCollection>(SelectedAsset);
			}
		}

		UMaterialExpressionFunctionInput* FunctionInput = Cast<UMaterialExpressionFunctionInput>(NewExpression);
		if (FunctionInput)
		{
			FunctionInput->ConditionallyGenerateId(true);
			FunctionInput->ValidateName();
		}

		UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(NewExpression);
		if (FunctionOutput)
		{
			FunctionOutput->ConditionallyGenerateId(true);
			FunctionOutput->ValidateName();
		}

		NewExpression->UpdateParameterGuid(true, bAllowMarkingPackageDirty);

		if (NewExpression->HasAParameterName())
		{
			NewExpression->ValidateParameterName(false);
		}

		UMaterialExpressionComponentMask* ComponentMaskExpression = Cast<UMaterialExpressionComponentMask>(NewExpression);
		// Setup defaults for the most likely use case
		// Can't change default properties as that will affect existing content
		if (ComponentMaskExpression)
		{
			ComponentMaskExpression->R = true;
			ComponentMaskExpression->G = true;
		}

		UMaterialExpressionStaticComponentMaskParameter* StaticComponentMaskExpression = Cast<UMaterialExpressionStaticComponentMaskParameter>(NewExpression);
		// Setup defaults for the most likely use case
		// Can't change default properties as that will affect existing content
		if (StaticComponentMaskExpression)
		{
			StaticComponentMaskExpression->DefaultR = true;
		}

		// Setup defaults for the most likely use case
		// Can't change default properties as that will affect existing content
		UMaterialExpressionTransformPosition* PositionTransform = Cast<UMaterialExpressionTransformPosition>(NewExpression);
		if (PositionTransform)
		{
			PositionTransform->TransformSourceType = TRANSFORMPOSSOURCE_Local;
			PositionTransform->TransformType = TRANSFORMPOSSOURCE_World;
		}

		// Make sure the dynamic parameters are named based on existing ones
		UMaterialExpressionDynamicParameter* DynamicExpression = Cast<UMaterialExpressionDynamicParameter>(NewExpression);
		if (DynamicExpression)
		{
			DynamicExpression->UpdateDynamicParameterProperties();
		}

		if (Material)
		{
			Material->AddExpressionParameter(NewExpression, Material->EditorParameters);
		}

		if (bAllowMarkingPackageDirty)
		{
			NewExpression->MarkPackageDirty();
		}

		MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Material);
	}
	return NewExpression;
}

bool UMaterialEditingLibrary::SetMaterialUsage(UMaterial* Material, EMaterialUsage Usage, bool& bNeedsRecompile)
{
	bool bResult = false;
	bNeedsRecompile = false;
	if (Material)
	{
		bResult = Material->SetMaterialUsage(Usage);
	}
	return bResult;
}

bool UMaterialEditingLibrary::HasMaterialUsage(UMaterialInterface* Material, EMaterialUsage Usage)
{
	return Material ? Material->GetUsageByFlag(Usage) : false;
}

void UMaterialEditingLibrary::SetBaseMaterialUsage(UMaterial* Material, EMaterialUsage Usage, bool bValue)
{
	if (Material == nullptr)
	{
		return;
	}

	if (bValue == Material->GetUsageByFlag(Usage))
	{
		return;
	}

	Material->SetUsageByFlag(Usage, bValue);
	RecompileMaterialInternal( { Material }, nullptr);

	FEditorSupportDelegates::MaterialUsageFlagsChanged.Broadcast(Material, Usage);
}

bool UMaterialEditingLibrary::HasMaterialUsageOverride(UMaterialInstanceConstant* MaterialInstance, EMaterialUsage Usage)
{
	return MaterialInstance ? (MaterialInstance->BasePropertyOverrides.bOverride_UsageFlags & (1u << Usage)) != 0 : false;
}

void UMaterialEditingLibrary::SetMaterialUsageOverride(UMaterialInstanceConstant* MaterialInstance, EMaterialUsage Usage, bool bOverride, bool bValue)
{
	if (MaterialInstance == nullptr)
	{
		return;
	}
		
	FMaterialInstanceBasePropertyOverrides BasePropertyOverrides = MaterialInstance->BasePropertyOverrides;
		
	if (bOverride)
	{
		BasePropertyOverrides.bOverride_UsageFlags |= 1u << Usage;
	}
	else
	{
		BasePropertyOverrides.bOverride_UsageFlags &= ~(1u << Usage);

		// Get value from parent.		
		if (UMaterialInterface* Parent = MaterialInstance->Parent)
		{
			bValue = Parent->GetUsageByFlag(Usage);
		}
	}

	if (bValue)
	{
		BasePropertyOverrides.UsageFlags |= 1u << Usage;
	}
	else
	{
		BasePropertyOverrides.UsageFlags &= ~(1u << Usage);
	}

	if (BasePropertyOverrides.bOverride_UsageFlags == MaterialInstance->BasePropertyOverrides.bOverride_UsageFlags &&
		BasePropertyOverrides.UsageFlags == MaterialInstance->BasePropertyOverrides.UsageFlags)
	{
		return;
	}
		
	{
		FMaterialInstanceParameterUpdateContext UpdateContext(MaterialInstance);
		UpdateContext.SetBasePropertyOverrides(BasePropertyOverrides);
	}
			
	MaterialInstance->MarkPackageDirty();
		
	if (UMaterial* Material = MaterialInstance->GetMaterial())
	{
		RebuildMaterialInstanceEditors(Material);
	}
}

bool UMaterialEditingLibrary::ConnectMaterialProperty(UMaterialExpression* FromExpression, FString FromOutputName, EMaterialProperty Property)
{
	bool bResult = false;
	if (FromExpression)
	{
		// Get material that owns this expression
		UMaterial* Material = Cast<UMaterial>(FromExpression->GetOuter());
		if (Material)
		{
			FExpressionInput* Input = Material->GetExpressionInputForProperty(Property);
			int32 FromIndex = GetExpressionOutputIndexByName(FromExpression, *FromOutputName);
			if (Input && FromIndex != INDEX_NONE)
			{
				Input->Connect(FromIndex, FromExpression);
				bResult = true;

				MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Material);
			}
		}
	}
	return bResult;
}

bool UMaterialEditingLibrary::ConnectMaterialExpressions(UMaterialExpression* FromExpression, FString FromOutputName, UMaterialExpression* ToExpression, FString ToInputName)
{
	bool bResult = false;
	if (FromExpression && ToExpression)
	{
		FExpressionInput* Input = GetExpressionInputByName(ToExpression, *ToInputName);
		int32 FromIndex = GetExpressionOutputIndexByName(FromExpression, *FromOutputName);
		if (Input && FromIndex != INDEX_NONE)
		{
			Input->Connect(FromIndex, FromExpression);
			bResult = true;
			MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Cast<UMaterial>(FromExpression->GetOuter()));
		}
	}
	return bResult;
}

bool UMaterialEditingLibrary::DisconnectMaterialExpressions(UMaterialExpression* ToExpression, FString ToInputName)
{
	if (!ToExpression)
	{
		return false;
	}
	FExpressionInput* Input = GetExpressionInputByName(ToExpression, *ToInputName);
	if (!Input || !Input->IsConnected())
	{
		return false;
	}
	Input->Expression = nullptr;
	Input->OutputIndex = 0;

	MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Cast<UMaterial>(ToExpression->GetOuter()));

	return true;
}

bool UMaterialEditingLibrary::DisconnectMaterialProperty(UMaterial* Material, EMaterialProperty Property)
{
	if (!Material)
	{
		return false;
	}
	FExpressionInput* Input = Material->GetExpressionInputForProperty(Property);
	if (!Input || !Input->IsConnected())
	{
		return false;
	}
	Input->Expression = nullptr;
	Input->OutputIndex = 0;

	MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Material);

	return true;
}


void UMaterialEditingLibrary::RecompileMaterialInternal(const TArray<UMaterial*>& Materials, const FOnItemComplete* OnItemComplete)
{
	// 1. Update all children Material Instances via the update context
	{
		FMaterialUpdateContext UpdateContext;

		for (UMaterial* Material : Materials)
		{
			if (ensureAsRuntimeWarning(Material != nullptr))
			{
				UpdateContext.AddMaterial(Material);

				// Propagate the change to this material
				Material->PreEditChange(nullptr);
				Material->PostEditChange();

				Material->MarkPackageDirty();

				// Force particle components to update their view relevance.
				for (TObjectIterator<UParticleSystemComponent> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
				{
					It->bIsViewRelevanceDirty = true;
				}

				// Update parameter names on any child material instances
				for (TObjectIterator<UMaterialInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
				{
					if (It->Parent == Material)
					{
						It->UpdateParameterNames();
					}
				}

				if (OnItemComplete)
				{
					OnItemComplete->Execute();
				}
			}
		}
	}

	// 2. Once the MIs are updated, update the opened MI Editors.
	for (UMaterial* Material : Materials)
	{
		if (Material)
		{
			UMaterialEditingLibrary::RebuildMaterialInstanceEditors(Material);
		}
	}
}

TArray<FString> UMaterialEditingLibrary::RecompileMaterial(UMaterial* Material)
{
	if (!ensureAsRuntimeWarning(Material != nullptr))
	{
		return { TEXT("Cannot compile a null material.") };
	}

	RecompileMaterialInternal( { Material }, nullptr);

	// Notify any open Material Editor so it refreshes its graph view
	MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Material);

	// Update the world's viewports
	FEditorDelegates::RefreshEditor.Broadcast();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

	// Update streaming data
	FMaterialEditorUtilities::BuildTextureStreamingData(Material);

	if (const FMaterialResource* Resource = Material->GetMaterialResource(GMaxRHIShaderPlatform))
	{
		return Resource->GetCompileErrors();
	}

	return {};
}

void UMaterialEditingLibrary::RecompileMaterials(TArray<UMaterial*>& Materials, const FOnItemComplete& OnItemComplete)
{
	if (Materials.Num() == 0)
	{
		return;
	}

	RecompileMaterialInternal(Materials, &OnItemComplete);

	// Update the world's viewports
	FEditorDelegates::RefreshEditor.Broadcast();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

	// Update streaming data
	for (UMaterial* Material : Materials)
	{
		if (Material != nullptr)
		{
			FMaterialEditorUtilities::BuildTextureStreamingData(Material);
		}
	}
}

void UMaterialEditingLibrary::LayoutMaterialExpressions(UMaterial* Material)
{
	MaterialEditingLibraryImpl::LayoutMaterialExpressions( Material );
	MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Material);
}

float UMaterialEditingLibrary::GetMaterialDefaultScalarParameterValue(UMaterial* Material, FName ParameterName)
{
	float Result = 0.f;
	if (Material)
	{
		Material->GetScalarParameterDefaultValue(ParameterName, Result);
	}
	return Result;
}

UTexture* UMaterialEditingLibrary::GetMaterialDefaultTextureParameterValue(UMaterial* Material, FName ParameterName)
{
	UTexture* Result = nullptr;
	if (Material)
	{
		Material->GetTextureParameterDefaultValue(ParameterName, Result);
	}
	return Result;
}

FLinearColor UMaterialEditingLibrary::GetMaterialDefaultVectorParameterValue(UMaterial* Material, FName ParameterName)
{
	FLinearColor Result = FLinearColor::Black;
	if (Material)
	{
		Material->GetVectorParameterDefaultValue(ParameterName, Result);
	}
	return Result;
}

bool UMaterialEditingLibrary::GetMaterialDefaultStaticSwitchParameterValue(UMaterial* Material, FName ParameterName)
{
	bool bResult = false;
	if (Material)
	{
		FGuid OutGuid;
		Material->GetStaticSwitchParameterDefaultValue(ParameterName, bResult, OutGuid);
	}
	return bResult;
}

TSet<UObject*> UMaterialEditingLibrary::GetMaterialSelectedNodes(UMaterial* Material)
{
	if (IMaterialEditor* MaterialEditor = MaterialEditingLibraryImpl::FindMaterialEditorForAsset(Material))
	{
		TSet<UObject*> SelectedMaterialObjects;
		for (const FFieldVariant SelectedNode : MaterialEditor->GetSelectedNodes())
		{
			check(SelectedNode.IsUObject());
			SelectedMaterialObjects.Add(SelectedNode.ToUObject());
		}
		return SelectedMaterialObjects;
	}

	return TSet<UObject*>();
}

UMaterialExpression* UMaterialEditingLibrary::GetMaterialPropertyInputNode(UMaterial* Material, EMaterialProperty Property)
{
	if (Material)
	{
		FExpressionInput*  ExpressionInput = Material->GetExpressionInputForProperty(Property);
		return ExpressionInput->Expression;
	}

	return nullptr;
}

static FString GetExpressionOutputName(const FExpressionOutput& Output)
{
	if (!Output.OutputName.IsNone())
	{
		return Output.OutputName.ToString();
	}
	else if (Output.Mask)
	{
		if (Output.MaskR && !Output.MaskG && !Output.MaskB && !Output.MaskA)
		{
			return TEXT("R");
		}
		else if (!Output.MaskR && Output.MaskG && !Output.MaskB && !Output.MaskA)
		{
			return TEXT("G");
		}
		else if (!Output.MaskR && !Output.MaskG && Output.MaskB && !Output.MaskA)
		{
			return TEXT("B");
		}
		else if (!Output.MaskR && !Output.MaskG && !Output.MaskB && Output.MaskA)
		{
			return TEXT("A");
		}
	}
	return FString();
}

FString UMaterialEditingLibrary::GetMaterialPropertyInputNodeOutputName(UMaterial* Material, EMaterialProperty Property)
{
	if (Material)
	{
		FExpressionInput* ExpressionInput = Material->GetExpressionInputForProperty(Property);
		if (ExpressionInput->OutputIndex != INDEX_NONE
			&& ExpressionInput->Expression
			&& ExpressionInput->OutputIndex < ExpressionInput->Expression->Outputs.Num())
		{
			FExpressionOutput& Output = ExpressionInput->Expression->Outputs[ExpressionInput->OutputIndex];
			return GetExpressionOutputName(Output);
		}
	}
	return FString();
}

TArray<FString> UMaterialEditingLibrary::GetMaterialExpressionInputNames(UMaterialExpression* MaterialExpression)
{
	TArray<FString> InputNames;

	for (FExpressionInputIterator It{ MaterialExpression }; It; ++It)
	{
		FName Name;
		if (UMaterialExpressionMaterialFunctionCall* FuncCall = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression))
		{
			// If a function call, don't want to compare string with type postfix
			Name = FuncCall->GetInputNameWithType(It.Index, false);
		}
		else
		{
			const FName ExpressionInputName = MaterialExpression->GetInputName(It.Index);
			Name = UMaterialGraphNode::GetShortenPinName(ExpressionInputName);
		}


		InputNames.Add(Name.ToString());
	}

	return InputNames;
}

TArray<int32> UMaterialEditingLibrary::GetMaterialExpressionInputTypes(UMaterialExpression* MaterialExpression)
{
	TArray<int32> InputTypes;

	for (FExpressionInputIterator It{ MaterialExpression }; It; ++It)
	{
		UMaterialExpression* Expression = It->Expression;
		if (Expression != nullptr)
		{
			InputTypes.Add(Expression->GetOutputValueType(It->OutputIndex));
		}
		else
		{
			InputTypes.Add(MaterialExpression->GetInputValueType(It.Index));
		}
	}
	return InputTypes;
}

TArray<FString> UMaterialEditingLibrary::GetMaterialExpressionOutputNames(UMaterialExpression* MaterialExpression)
{
	TArray<FString> OutputNames;

	if (MaterialExpression)
	{
		for (const FExpressionOutput& Output : MaterialExpression->GetOutputs())
		{
			OutputNames.Add(GetExpressionOutputName(Output));
		}
	}

	return OutputNames;
}

TArray<UMaterialExpression*> UMaterialEditingLibrary::GetInputsForMaterialExpression(UMaterial* Material, UMaterialExpression* MaterialExpression)
{
	TArray<UMaterialExpression*> MaterialExpressions;
	if (Material)
	{
		for (FExpressionInputIterator It{ MaterialExpression }; It; ++It)
		{
			MaterialExpressions.Add(It->Expression);
		}
	}

	return MaterialExpressions;
}

TArray<UMaterialExpression*> UMaterialEditingLibrary::GetInputsForMaterialFunctionExpression(UMaterialFunction* MaterialFunction, UMaterialExpression* MaterialExpression)
{
	TArray<UMaterialExpression*> MaterialExpressions;
	if (MaterialFunction)
	{
		for (FExpressionInputIterator It{ MaterialExpression }; It; ++It)
		{
			MaterialExpressions.Add(It->Expression);
		}
	}

	return MaterialExpressions;
}

bool UMaterialEditingLibrary::GetInputNodeOutputNameForMaterialExpression(UMaterialExpression* MaterialExpression, UMaterialExpression* InputNode, FString& OutputName)
{
	OutputName = TEXT("");
	for (FExpressionInputIterator It{ MaterialExpression }; It; ++It)
	{
		if (It->Expression == InputNode)
		{
			if(It->OutputIndex != INDEX_NONE && It->OutputIndex < InputNode->Outputs.Num())
			{
				FExpressionOutput& Output = InputNode->Outputs[It->OutputIndex];
				OutputName = GetExpressionOutputName(Output);
				return true;
			}
		}
	}
	return false;
}

void UMaterialEditingLibrary::GetMaterialExpressionNodePosition(UMaterialExpression* MaterialExpression, int32& NodePosX, int32& NodePosY)
{
	if (MaterialExpression)
	{
		NodePosX = MaterialExpression->MaterialExpressionEditorX;
		NodePosY = MaterialExpression->MaterialExpressionEditorY;
	}
}

TArray<UTexture*> UMaterialEditingLibrary::GetUsedTextures(UMaterial* Material)
{
	return UMaterialEditingLibrary::GetMaterialUsedTextures(Material);
}

TArray<UTexture*> UMaterialEditingLibrary::GetMaterialUsedTextures(UMaterialInterface* MaterialInterface)
{
	TArray<UTexture*> OutTextures;
	MaterialInterface->GetUsedTextures(OutTextures, GetCurrentMaterialQualityLevelChecked());
	return OutTextures;
}

//////////////////////////////////////////////////////////////////////////

int32 UMaterialEditingLibrary::GetNumMaterialExpressionsInFunction(const UMaterialFunction* MaterialFunction)
{
	int32 Result = 0;
	if (MaterialFunction)
	{
		Result = MaterialFunction->GetExpressions().Num();
	}
	return Result;
}

void UMaterialEditingLibrary::DeleteAllMaterialExpressionsInFunction(UMaterialFunction* MaterialFunction)
{
	if (MaterialFunction)
	{
		for (UMaterialExpression* Expression : MaterialFunction->GetExpressions())
		{
			DeleteMaterialExpressionInFunction(MaterialFunction, Expression);
		}
	}
}


void UMaterialEditingLibrary::DeleteMaterialExpressionInFunction(UMaterialFunction* MaterialFunction, UMaterialExpression* Expression)
{
	if (MaterialFunction && Expression && Expression->GetOuter() == MaterialFunction)
	{
		// Break any links to this expression
		BreakLinksToExpression(MaterialFunction->GetExpressions(), Expression);

		MaterialFunction->GetExpressionCollection().RemoveExpression(Expression);

		Expression->MarkAsGarbage();

		MaterialFunction->MarkPackageDirty();
	}
}

void UMaterialEditingLibrary::UpdateMaterialFunctionInternal(FMaterialUpdateContext& UpdateContext, UMaterialFunctionInterface* MaterialFunction, UMaterial* PreviewMaterial)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialEditingLibrary::UpdateMaterialFunction)

	// mark the function as changed
	MaterialFunction->ForceRecompileForRendering(UpdateContext, PreviewMaterial);
	MaterialFunction->MarkPackageDirty();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateAllMaterialInstances)

		// Go through all function instances in memory and recompile them if they are children
		for (TObjectIterator<UMaterialFunctionInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterialFunctionInstance* FunctionInstance = *It;

			TArray<UMaterialFunctionInterface*> Functions;
			FunctionInstance->GetDependentFunctions(Functions);
			if (Functions.Contains(MaterialFunction))
			{
				FunctionInstance->UpdateParameterSet();
				FunctionInstance->ForceRecompileForRendering(UpdateContext, PreviewMaterial);

				// ForceRecompileForRendering will update StateId, so need to mark the package as dirty
				FunctionInstance->MarkPackageDirty();
			}
		}
	}

	// Notify material editor for any materials that we are updating
	for (UMaterialInterface* CurrentMaterial : UpdateContext.GetUpdatedMaterials())
	{
		MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(CurrentMaterial);
	}

	UMaterialFunction* BaseFunction = MaterialFunction->GetBaseFunction();
	UMaterialEditingLibrary::RebuildMaterialInstanceEditors(BaseFunction);
}

void UMaterialEditingLibrary::UpdateMaterialFunction(UMaterialFunctionInterface* MaterialFunction, UMaterial* PreviewMaterial)
{
	if (MaterialFunction == nullptr)
	{
		return;
	}
	
	{
		// Create a material update context so we can safely update materials using this function.
		FMaterialUpdateContext UpdateContext;
	
		UpdateMaterialFunctionInternal(UpdateContext, MaterialFunction, PreviewMaterial);
	}

	// Update the world's viewports	
	FEditorDelegates::RefreshEditor.Broadcast();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void UMaterialEditingLibrary::UpdateMaterialFunctions(TArray<UMaterialFunctionInterface*>& MaterialFunctions, const FOnItemComplete& OnItemComplete)
{
	if (MaterialFunctions.Num() == 0)
	{
		return;
	}

	{
		// Create a material update context so we can safely update materials using this function.
		FMaterialUpdateContext UpdateContext;

		for (UMaterialFunctionInterface* MaterialFunction : MaterialFunctions)
		{
			if (MaterialFunction != nullptr)
			{
				UpdateMaterialFunctionInternal(UpdateContext, MaterialFunction, nullptr);
				OnItemComplete.Execute();
			}
		}
	}

	// Update the world's viewports	
	FEditorDelegates::RefreshEditor.Broadcast();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void UMaterialEditingLibrary::LayoutMaterialFunctionExpressions(UMaterialFunction* MaterialFunction)
{
	MaterialEditingLibraryImpl::LayoutMaterialExpressions( MaterialFunction );
}

void UMaterialEditingLibrary::SetMaterialInstanceParent(UMaterialInstanceConstant* Instance, UMaterialInterface* NewParent)
{
	if (Instance)
	{
		FMaterialUpdateContext UpdateContext;
		Instance->SetParentEditorOnly(NewParent);
	}
}

void UMaterialEditingLibrary::ClearAllMaterialInstanceParameters(UMaterialInstanceConstant* Instance)
{
	if (Instance)
	{
		Instance->ClearParameterValuesEditorOnly();
	}
}


float UMaterialEditingLibrary::GetMaterialInstanceScalarParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	float Result = 0.f;
	if (Instance)
	{
		Instance->GetScalarParameterValue(FHashedMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Result);
	}
	return Result;
}

bool UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, float Value, EMaterialParameterAssociation Association)
{
	bool bResult = false;
	if (Instance)
	{
		Instance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Value);
		UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
	}
	return bResult;
}


UTexture* UMaterialEditingLibrary::GetMaterialInstanceTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	UTexture* Result = nullptr;
	if (Instance)
	{
		Instance->GetTextureParameterValue(FHashedMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Result);
	}
	return Result;
}

bool UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, UTexture* Value, EMaterialParameterAssociation Association)
{
	bool bResult = false;
	if (Instance)
	{
		Instance->SetTextureParameterValueEditorOnly(FMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Value);
		UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
	}
	return bResult;
}


URuntimeVirtualTexture* UMaterialEditingLibrary::GetMaterialInstanceRuntimeVirtualTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	URuntimeVirtualTexture* Result = nullptr;
	if (Instance)
	{
		Instance->GetRuntimeVirtualTextureParameterValue(FHashedMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Result);
	}
	return Result;
}

bool UMaterialEditingLibrary::SetMaterialInstanceRuntimeVirtualTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, URuntimeVirtualTexture* Value, EMaterialParameterAssociation Association)
{
	bool bResult = false;
	if (Instance)
	{
		Instance->SetRuntimeVirtualTextureParameterValueEditorOnly(FMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Value);
		UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
	}
	return bResult;
}


USparseVolumeTexture* UMaterialEditingLibrary::GetMaterialInstanceSparseVolumeTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	USparseVolumeTexture* Result = nullptr;
	if (Instance)
	{
		Instance->GetSparseVolumeTextureParameterValue(FHashedMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Result);
	}
	return Result;
}

bool UMaterialEditingLibrary::SetMaterialInstanceSparseVolumeTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, USparseVolumeTexture* Value, EMaterialParameterAssociation Association)
{
	bool bResult = false;
	if (Instance)
	{
		Instance->SetSparseVolumeTextureParameterValueEditorOnly(FMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Value);
		UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
	}
	return bResult;
}


FLinearColor UMaterialEditingLibrary::GetMaterialInstanceVectorParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	FLinearColor Result = FLinearColor::Black;
	if (Instance)
	{
		Instance->GetVectorParameterValue(FHashedMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Result);
	}
	return Result;
}

bool UMaterialEditingLibrary::SetMaterialInstanceVectorParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, FLinearColor Value, EMaterialParameterAssociation Association)
{
	bool bResult = false;
	if (Instance)
	{
		Instance->SetVectorParameterValueEditorOnly(FMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Value);
		UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
	}
	return bResult;
}


bool UMaterialEditingLibrary::GetMaterialInstanceStaticSwitchParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	bool bResult = false;
	if (Instance)
	{
		FGuid OutGuid;
		Instance->GetStaticSwitchParameterValue(FHashedMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), bResult, OutGuid);
	}
	return bResult;
}

bool UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, bool Value, EMaterialParameterAssociation Association, bool bUpdateMaterialInstance)
{
	bool bResult = false;
	if (Instance)
	{
		Instance->SetStaticSwitchParameterValueEditorOnly(FMaterialParameterInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE), Value);

		// The material instance editor window puts MaterialLayersParameters into our StaticParameters, if we don't do this, our settings could get wiped out on first launch of the material editor.
		// If there's ever a cleaner and more isolated way of populating MaterialLayersParameters, we should do that instead.
		UMaterialEditorInstanceConstant* MaterialEditorInstance = NewObject<UMaterialEditorInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);
		MaterialEditorInstance->SetSourceInstance(Instance);

		if (bUpdateMaterialInstance)
		{
			UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
		}
	}
	return bResult;
}

bool UMaterialEditingLibrary::IsMaterialInstanceParameterOverridden(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association)
{
	if (!Instance)
	{
		return false;
	}

	const FMaterialParameterInfo ParamInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE);
	FMaterialParameterMetadata OutResult;

	for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
	{
		const EMaterialParameterType Type = static_cast<EMaterialParameterType>(TypeIndex);
		if (Instance->GetParameterOverrideValue(Type, ParamInfo, OutResult))
		{
			return true;
		}
	}

	return false;
}

bool UMaterialEditingLibrary::SetMaterialInstanceParameterOverride(UMaterialInstanceConstant* Instance, FName ParameterName, bool bOverride, EMaterialParameterAssociation Association)
{
	if (!Instance)
	{
		return false;
	}

	const FMaterialParameterInfo ParamInfo(ParameterName, Association, Association == EMaterialParameterAssociation::LayerParameter ? 0 : INDEX_NONE);
	FMaterialParameterMetadata Meta;
	EMaterialParameterType FoundType = EMaterialParameterType::None;

	for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
	{
		const EMaterialParameterType Type = static_cast<EMaterialParameterType>(TypeIndex);
		if (Instance->GetParameterValue(Type, ParamInfo, Meta))
		{
			FoundType = Type;
			break;
		}
	}

	if (FoundType == EMaterialParameterType::None)
	{
		UE_LOGF(LogMaterialEditingLibrary, Warning, "SetMaterialInstanceParameterOverride: Parameter '%ls' not found in material hierarchy.", *ParameterName.ToString());
		return false;
	}

	// Already in the desired state — nothing to do.
	if (bOverride == Meta.bOverride)
	{
		return true;
	}


	if (bOverride)
	{
		// Enabling override: set the parameter on this instance using the inherited value from parent.
		switch (FoundType)
		{
		case EMaterialParameterType::Scalar:				Instance->SetScalarParameterValueEditorOnly(ParamInfo, Meta.Value.AsScalar()); break;
		case EMaterialParameterType::Vector:				Instance->SetVectorParameterValueEditorOnly(ParamInfo, Meta.Value.AsLinearColor()); break;
		case EMaterialParameterType::DoubleVector:			Instance->SetDoubleVectorParameterValueEditorOnly(ParamInfo, Meta.Value.AsVector4d()); break;
		case EMaterialParameterType::Texture:				Instance->SetTextureParameterValueEditorOnly(ParamInfo, Meta.Value.Texture); break;
		case EMaterialParameterType::TextureCollection:		Instance->SetTextureCollectionParameterValueEditorOnly(ParamInfo, Meta.Value.TextureCollection); break;
		case EMaterialParameterType::Font:					Instance->SetFontParameterValueEditorOnly(ParamInfo, Meta.Value.Font.Value, Meta.Value.Font.Page); break;
		case EMaterialParameterType::RuntimeVirtualTexture: Instance->SetRuntimeVirtualTextureParameterValueEditorOnly(ParamInfo, Meta.Value.RuntimeVirtualTexture); break;
		case EMaterialParameterType::SparseVolumeTexture:	Instance->SetSparseVolumeTextureParameterValueEditorOnly(ParamInfo, Meta.Value.SparseVolumeTexture); break;
		case EMaterialParameterType::StaticSwitch:			Instance->SetStaticSwitchParameterValueEditorOnly(ParamInfo, Meta.Value.AsStaticSwitch()); break;
		case EMaterialParameterType::ParameterCollection:	Instance->SetParameterCollectionParameterValueEditorOnly(ParamInfo, Meta.Value.ParameterCollection); break;
		case EMaterialParameterType::StaticComponentMask:	Instance->SetStaticComponentMaskParameterValueEditorOnly(ParamInfo, Meta.Value.AsStaticComponentMask()); break;
		default:
			UE_LOGF(LogMaterialEditingLibrary, Warning, "SetMaterialInstanceParameterOverride: Unsupported parameter type for '%ls'.", *ParameterName.ToString());
			return false;
		}
	}
	else
	{
		bool bRemoved = false;

		// Disabling override: remove the parameter from the appropriate array.
		auto RemoveParameterByInfo = [&ParamInfo](auto& ParameterArray) -> bool
		{
			const int32 Index = ParameterArray.IndexOfByPredicate([&ParamInfo](const auto& Param)
			{
				return Param.ParameterInfo == ParamInfo;
			});

			if (Index != INDEX_NONE)
			{
				ParameterArray.RemoveAt(Index);
				return true;
			}
			return false;
		};

		auto RemoveStaticParameter = [&ParamInfo](auto& StaticParameterArray) -> bool
		{
			for (auto& Param : StaticParameterArray)
			{
				if (Param.ParameterInfo == ParamInfo)
				{
					Param.bOverride = false;
					return true;
				}
			}
			return false;
		};

		switch (FoundType)
		{
		case EMaterialParameterType::Scalar:				bRemoved = RemoveParameterByInfo(Instance->ScalarParameterValues); break;
		case EMaterialParameterType::Vector:				bRemoved = RemoveParameterByInfo(Instance->VectorParameterValues); break;
		case EMaterialParameterType::DoubleVector:			bRemoved = RemoveParameterByInfo(Instance->DoubleVectorParameterValues); break;
		case EMaterialParameterType::Texture:				bRemoved = RemoveParameterByInfo(Instance->TextureParameterValues); break;
		case EMaterialParameterType::TextureCollection:		bRemoved = RemoveParameterByInfo(Instance->TextureCollectionParameterValues); break;
		case EMaterialParameterType::ParameterCollection:	bRemoved = RemoveParameterByInfo(Instance->ParameterCollectionParameterValues); break;
		case EMaterialParameterType::Font:					bRemoved = RemoveParameterByInfo(Instance->FontParameterValues); break;
		case EMaterialParameterType::RuntimeVirtualTexture: bRemoved = RemoveParameterByInfo(Instance->RuntimeVirtualTextureParameterValues); break;
		case EMaterialParameterType::SparseVolumeTexture:	bRemoved = RemoveParameterByInfo(Instance->SparseVolumeTextureParameterValues); break;
		case EMaterialParameterType::StaticSwitch:			bRemoved = RemoveStaticParameter(Instance->StaticParametersRuntime.StaticSwitchParameters); break;
		case EMaterialParameterType::StaticComponentMask:	bRemoved = RemoveStaticParameter(Instance->GetEditorOnlyData()->StaticParameters.StaticComponentMaskParameters); break;
		default:
			UE_LOGF(LogMaterialEditingLibrary, Warning, "SetMaterialInstanceParameterOverride: Unsupported parameter type for '%ls'.", *ParameterName.ToString());
		}

		if (!bRemoved)
		{
			return false;
		}
	}

	UMaterialEditingLibrary::UpdateMaterialInstance(Instance);
	if (FMaterialInstanceEditor* MaterialInstanceEditor = MaterialEditingLibraryImpl::FindMaterialInstanceEditorForAsset(Instance))
	{
		// Rebuild the editor's parameter proxies so the UI reflects the override change.
		MaterialInstanceEditor->RebuildMaterialInstanceEditor();
	}

	return true;
}

void UMaterialEditingLibrary::UpdateMaterialInstance(UMaterialInstanceConstant* Instance)
{
	if (Instance)
	{
		Instance->MarkPackageDirty();
		Instance->PreEditChange(nullptr);
		Instance->PostEditChange();

		Instance->UpdateStaticPermutation();
		Instance->UpdateParameterNames();

		// Refresh the material instance editor to reflect parameter value changes in the property window.
		MaterialEditingLibraryImpl::NotifyMaterialEditorOfChange(Instance);

		// update the world's viewports
		FEditorDelegates::RefreshEditor.Broadcast();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}
}

void UMaterialEditingLibrary::GetChildInstances(UMaterialInterface* Parent, TArray< FAssetData>& ChildInstances)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> AssetList;
	TMultiMap<FName, FString> TagsAndValues;
	const FString ParentNameString = FAssetData(Parent).GetExportTextName();
	TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UMaterialInstance, Parent), ParentNameString);
	AssetRegistryModule.Get().GetAssetsByTagValues(TagsAndValues, AssetList);
	
	for (const FAssetData& MatInstRef : AssetList)
	{
		ChildInstances.Add(MatInstRef);
	}
}

void UMaterialEditingLibrary::GetMaterialsReferencingFunction(UMaterialFunction* InFunction, TArray<FAssetData>& OutMaterials)
{
	if (InFunction == nullptr)
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	const FAssetData FunctionAssetData = AssetRegistry.GetAssetByObjectPath(InFunction->GetPathName());
	if (!FunctionAssetData.IsValid())
	{
		return;
	}

	TArray<FName> Visited;
	TArray<FName> ToVisit;
	ToVisit.Push(FunctionAssetData.PackageName);

	while (!ToVisit.IsEmpty())
	{
		const FName FunctionPackageName = ToVisit.Pop(EAllowShrinking::No);

		TArray<FName> ReferencerPackageNames;
		AssetRegistry.GetReferencers(FunctionPackageName, ReferencerPackageNames);

		for (const FName& PackageName : ReferencerPackageNames)
		{
			if (Visited.Contains(PackageName))
			{
				continue;
			}
			Visited.Add(PackageName);

			TArray<FAssetData> AssetDatas;
			AssetRegistry.GetAssetsByPackageName(PackageName, AssetDatas, true);

			for (FAssetData const& AssetData : AssetDatas)
			{
				if (!AssetData.IsValid())
				{
					continue;
				}
				else if (AssetData.IsInstanceOf(UMaterial::StaticClass()))
				{
					OutMaterials.Add(AssetData);
				}
				else if (AssetData.IsInstanceOf(UMaterialFunction::StaticClass()))
				{
					ToVisit.Push(PackageName);
				}
			}
		}
	}
}

void UMaterialEditingLibrary::GetScalarParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames)
{
	ParameterNames.Empty();
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllScalarParameterInfo(MaterialInfo, MaterialGuids);

		for (const FMaterialParameterInfo& Info : MaterialInfo)
		{
			ParameterNames.Add(Info.Name);
		}
	}
}

void UMaterialEditingLibrary::GetVectorParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames)
{
	ParameterNames.Empty();
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllVectorParameterInfo(MaterialInfo, MaterialGuids);

		for (const FMaterialParameterInfo& Info : MaterialInfo)
		{
			ParameterNames.Add(Info.Name);
		}
	}
}

void UMaterialEditingLibrary::GetTextureParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames)
{
	ParameterNames.Empty();
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllTextureParameterInfo(MaterialInfo, MaterialGuids);

		for (const FMaterialParameterInfo& Info : MaterialInfo)
		{
			ParameterNames.Add(Info.Name);
		}
	}
}

void UMaterialEditingLibrary::GetStaticSwitchParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames)
{
	ParameterNames.Empty();
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllStaticSwitchParameterInfo(MaterialInfo, MaterialGuids);

		for (const FMaterialParameterInfo& Info : MaterialInfo)
		{
			ParameterNames.Add(Info.Name);
		}
	}
}

static bool GetParameterSource(UMaterialInterface* Material, const TArray<FMaterialParameterInfo>& Info, const TArray<FGuid>& Guids, const FName& ParameterName, FSoftObjectPath& OutParameterSource)
{
	bool bResult = false;
	for (int32 Index = 0; Index < Info.Num(); ++Index)
	{
		if (Info[Index].Name == ParameterName)
		{
			UMaterial* BaseMaterial = Material->GetBaseMaterial();
			UMaterialExpression* Expression = BaseMaterial->FindExpressionByGUID<UMaterialExpression>(Guids[Index]);
			if (Expression)
			{
				bResult = true;
				OutParameterSource = Expression->GetAssetOwner();
			}
			break;
		}
	}
	return bResult;
}

bool UMaterialEditingLibrary::GetScalarParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource)
{
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllScalarParameterInfo(MaterialInfo, MaterialGuids);
		return GetParameterSource(Material, MaterialInfo, MaterialGuids, ParameterName, ParameterSource);
	}
	return false;
}

bool UMaterialEditingLibrary::GetVectorParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource)
{
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllVectorParameterInfo(MaterialInfo, MaterialGuids);
		return GetParameterSource(Material, MaterialInfo, MaterialGuids, ParameterName, ParameterSource);
	}
	return false;
}

bool UMaterialEditingLibrary::GetTextureParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource)
{
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllTextureParameterInfo(MaterialInfo, MaterialGuids);
		return GetParameterSource(Material, MaterialInfo, MaterialGuids, ParameterName, ParameterSource);
	}
	return false;
}

template <typename AssetType>
static bool RenameInternal(AssetType* Asset, const FName OldGroupName, const FName NewGroupName)
{
	if (!Asset)
	{
		UE_LOGF(LogMaterialEditingLibrary, Warning, "Invalid asset");
		return false;
	}

	if (Asset->HasAnyFlags(RF_ClassDefaultObject))
	{
		UE_LOGF(LogMaterialEditingLibrary, Warning, "Can't rename groups on a Class Default Object.");
		return false;
	}

	if (OldGroupName == NewGroupName)
	{
		UE_LOGF(LogMaterialEditingLibrary, Log, "Old and new group names are identical: %ls. Nothing to do.", *NewGroupName.ToString());
		return false;
	}

	int NumModified = 0;
	for (UMaterialExpression* Expression : Asset->GetExpressions())
	{
		if (!Expression)
		{
			continue;
		}

		// Use the property system to look for the `Group` parameter since many distinct parameter expressions
		// can have a `Group`.
		if (FProperty* Prop = Expression->GetClass()->FindPropertyByName(TEXT("Group")))
		{
			if (FNameProperty* NameProp = CastField<FNameProperty>(Prop))
			{
				FName* CurrentGroupName = NameProp->ContainerPtrToValuePtr<FName>(Expression);
				if (CurrentGroupName && *CurrentGroupName == OldGroupName)
				{
					*CurrentGroupName = NewGroupName;
					UE_LOGF(LogMaterialEditingLibrary, Verbose, "Moved %ls to %ls.", *Expression->GetClass()->GetName(), *NewGroupName.ToString());
					NumModified++;
				}
			}
		}
	}

	if (NumModified > 0)
	{
		UE_LOGF(LogMaterialEditingLibrary, Log, "Moved %d parameters from %ls into group %ls", NumModified, *OldGroupName.ToString(), *NewGroupName.ToString());
	}
	else
	{
		UE_LOGF(LogMaterialEditingLibrary, Log, "No parameters found with the group name: %ls", *OldGroupName.ToString());
	}

	return NumModified > 0;
}

bool UMaterialEditingLibrary::RenameMaterialParameterGroup(UMaterial* Material, const FName OldGroupName, const FName NewGroupName)
{
	if (RenameInternal(Material, OldGroupName, NewGroupName))
	{
		UMaterialEditingLibrary::RecompileMaterial(Material);
		return true;
	}

	return false;
}

bool UMaterialEditingLibrary::RenameMaterialFunctionParameterGroup(UMaterialFunctionInterface* MaterialFunction, const FName OldGroupName, const FName NewGroupName)
{
	if (RenameInternal(MaterialFunction, OldGroupName, NewGroupName))
	{
		UMaterialEditingLibrary::UpdateMaterialFunction(MaterialFunction, nullptr);
		return true;
	}

	return false;
}

bool UMaterialEditingLibrary::GetStaticSwitchParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource)
{
	if (Material)
	{
		TArray<FMaterialParameterInfo> MaterialInfo;
		TArray<FGuid> MaterialGuids;
		Material->GetAllStaticSwitchParameterInfo(MaterialInfo, MaterialGuids);
		return GetParameterSource(Material, MaterialInfo, MaterialGuids, ParameterName, ParameterSource);
	}
	return false;
}

int32 UMaterialEditingLibrary::GetNumShaderTypes(UMaterialInterface* Material)
{
	return UMaterialEditingLibrary::ListShaders(Material).Num();
}

TArray<FDebugShaderInfo> UMaterialEditingLibrary::GetShaderInfoFromTypeInfo(const TArray<FDebugShaderTypeInfo>& InShaderTypeInfos)
{
	TArray<FDebugShaderInfo> OutShaderInfos;

	for (FDebugShaderTypeInfo const& ShaderInfo : InShaderTypeInfos)
	{
		for (FShaderType* ShaderType : ShaderInfo.ShaderTypes)
		{
			if (ShaderType)
			{
				OutShaderInfos.Add({
					ShaderInfo.VFType ? ShaderInfo.VFType->GetFName() : NAME_None,
					ShaderType->GetFName()
					});
			}
		}

		for (FDebugShaderPipelineInfo const& PipelineInfo : ShaderInfo.Pipelines)
		{
			for (FShaderType* ShaderType : PipelineInfo.ShaderTypes)
			{
				if (ShaderType)
				{
					OutShaderInfos.Add({
						ShaderInfo.VFType ? ShaderInfo.VFType->GetFName() : NAME_None,
						ShaderType->GetFName()
						});
				}
			}
		}
	}

	return OutShaderInfos;
}

TArray<FDebugShaderInfo> UMaterialEditingLibrary::ListShaders(UMaterialInterface* Material)
{
	if (!Material)
	{
		return {};
	}

	TArray<FDebugShaderTypeInfo> ValidationShaderTypeInfos = GetShadersUsingCompilationParameters_GameThread(Material);
	const TArray<FDebugShaderInfo> ValidationShaderDebugInfo = GetShaderInfoFromTypeInfo(ValidationShaderTypeInfos);

	return ValidationShaderDebugInfo;
}

FMaterialStatistics UMaterialEditingLibrary::GetStatistics(class UMaterialInterface* Material)
{
	FMaterialStatistics Result;

	FMaterialResource* Resource = Material ? Material->GetMaterialResource(GMaxRHIShaderPlatform) : nullptr;
	if (Resource)
	{
		if (!Resource->IsGameThreadShaderMapComplete())
		{
			Resource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::High);
		}
		Resource->FinishCompilation();

		TArray<FMaterialStatsUtils::FShaderInstructionsInfo> InstructionInfos;
		FMaterialStatsUtils::GetRepresentativeInstructionCounts(InstructionInfos, Resource);
		for (const FMaterialStatsUtils::FShaderInstructionsInfo& Info : InstructionInfos)
		{
			const int32 ShaderType = (int32)Info.ShaderType;
			if (ShaderType >= (int32)ERepresentativeShader::FirstFragmentShader && ShaderType <= (int32)ERepresentativeShader::LastFragmentShader)
			{
				Result.NumPixelShaderInstructions = FMath::Max(Result.NumPixelShaderInstructions, Info.InstructionCount);
			}
			else if (ShaderType >= (int32)ERepresentativeShader::FirstVertexShader && ShaderType <= (int32)ERepresentativeShader::LastVertexShader)
			{
				Result.NumVertexShaderInstructions = FMath::Max(Result.NumVertexShaderInstructions, Info.InstructionCount);
			}
		}

		Result.NumSamplers = Resource->GetSamplerUsage();

		uint32 NumVSTextureSamples = 0, NumPSTextureSamples = 0;
		Resource->GetEstimatedNumTextureSamples(NumVSTextureSamples, NumPSTextureSamples);
		Result.NumVertexTextureSamples = (int32)NumVSTextureSamples;
		Result.NumPixelTextureSamples = (int32)NumPSTextureSamples;

		Result.NumVirtualTextureSamples = Resource->GetEstimatedNumVirtualTextureLookups();

		uint32 UVScalarsUsed, CustomInterpolatorScalarsUsed;
		Resource->GetUserInterpolatorUsage(UVScalarsUsed, CustomInterpolatorScalarsUsed);
		Result.NumUVScalars = (int32)UVScalarsUsed;
		Result.NumInterpolatorScalars = (int32)CustomInterpolatorScalarsUsed;
	}

	return Result;
}

UMaterialInterface* UMaterialEditingLibrary::GetNaniteOverrideMaterial(UMaterialInterface* Material)
{
	return Material->GetNaniteOverride();
}
