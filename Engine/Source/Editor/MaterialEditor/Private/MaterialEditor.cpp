// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor.h"
#include "EngineGlobals.h"
#include "Engine/SkeletalMesh.h"
#include "AI/NavigationSystemBase.h"
#include "Components/MeshComponent.h"
#include "Engine/Engine.h"
#include "EngineModule.h"
#include "Misc/MessageDialog.h"
#include "Misc/UObjectToken.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationBackground.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Styling/AppStyle.h"
#include "EdGraph/EdGraph.h"
#include "GraphEditAction.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "Editor/UnrealEdEngine.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Preferences/MaterialEditorOptions.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "MaterialGraph/MaterialGraphNode_PinBase.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialEditor/PreviewMaterial.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "StaticParameterSet.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/TextureCollection.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "MaterialEditorModule.h"
#include "MaterialEditingLibrary.h"
#include "HAL/PlatformApplicationMisc.h"
#include "MaterialCachedData.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Algo/ForEach.h"
#include "MaterialEditorSettings.h"

#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionCollectionTransform.h"
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionObjectPositionWS.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionParticleSubUV.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticComponentMaskParameter.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "Materials/MaterialExpressionTextureSampleParameterCube.h"
#include "Materials/MaterialExpressionTextureSampleParameter2DArray.h"
#include "Materials/MaterialExpressionTextureSampleParameterCubeArray.h"
#include "Materials/MaterialExpressionTextureSampleParameterSubUV.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionSparseVolumeTextureObject.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionDoubleVectorParameter.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionShadingModel.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "Materials/MaterialParameterCollection.h"

#include "MaterialGraphNode_Knot.h"
#include "MaterialEditorActions.h"
#include "MaterialExpressionClasses.h"
#include "MaterialCompiler.h"
#include "EditorSupportDelegates.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Tabs/MaterialEditorTabFactories.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "SMaterialEditorTitleBar.h"
#include "ScopedTransaction.h"
#include "BusyCursor.h"

#include "PropertyEditorModule.h"
#include "MaterialEditorDetailCustomization.h"
#include "MaterialInstanceEditor.h"

#include "EditorViewportCommands.h"

#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/TokenizedMessage.h"
#include "EdGraphUtilities.h"
#include "SGraphPanel.h"
#include "MaterialEditorUtilities.h"
#include "SMaterialPalette.h"
#include "SMaterialNamedReroutesPanel.h"
#include "FindInMaterial.h"
#include "Misc/FeedbackContext.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Colors/SColorPicker.h"
#include "EditorClassUtils.h"
#include "IDocumentation.h"
#include "Widgets/Docking/SDockTab.h"

#include "IMessageLogListing.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "Framework/Commands/GenericCommands.h"
#include "CanvasTypes.h"
#include "Engine/Selection.h"
#include "Materials/Material.h"
#include "AdvancedPreviewSceneModule.h"
#include "GraphEditorSettings.h"
#include "MaterialLayersFunctionsCustomization.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "SMaterialLayersFunctionsTree.h"
#include "Materials/MaterialExpressionGetMaterialAttributes.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Materials/MaterialExpressionBlendMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialLayerOutput.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionUtils.h"

#include "MaterialStats.h"
#include "MaterialEditorTabs.h"
#include "MaterialEditorModes.h"
#include "Materials/MaterialExpression.h"
#include "SMaterialEditorSubstrateWidget.h"
#include "MaterialEditor/SGraphSubstrateMaterial.h"

#include "SMaterialParametersOverviewWidget.h"
#include "SMaterialEditorCustomPrimitiveDataWidget.h"
#include "SMaterialExpressionsOverviewWidget.h"
#include "IPropertyRowGenerator.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "Widgets/Layout/SScrollBox.h"
#include "UObject/TextProperty.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenus.h"
#include "MaterialEditorHelpers.h"
#include "MaterialEditorContext.h"
#include "UObject/MetaData.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#include "DiffTool/Widgets/SMaterialRevisionMenu.h"

#define LOCTEXT_NAMESPACE "MaterialEditor"

DEFINE_LOG_CATEGORY_STATIC(LogMaterialEditor, Log, All);

static TAutoConsoleVariable<int32> CVarMaterialEdUseDevShaders(
	TEXT("r.MaterialEditor.UseDevShaders"),
	1,
	TEXT("Toggles whether the material editor will use shaders that include extra overhead incurred by the editor. Material editor must be re-opened if changed at runtime."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaterialEdMaxDerivedMaterialInstances(
	TEXT("r.MaterialEditor.MaxDerivedMaterialInstances"),
	-1,
	TEXT("Limits amount of derived material instance shown in platform stats. Use negative number to disable the limit. Material editor must be re-opened if changed at runtime."));
 
static TAutoConsoleVariable<bool> CVarMaterialEdGroupSortIncludeFunctions(
	TEXT("r.MaterialEditor.GroupSortIncludeFunctions"),
	true,
	TEXT("Whether Groups inside used Material Functions should be included in the sorting data."),
	ECVF_Default);

static void InvalidateSubstrateConversionVersion(UMaterial* Material)
{
	// The material has been updated, invalid the AutoConversion version
	if (Substrate::IsSubstrateEnabled())
	{
		if (UMaterialEditorOnlyData* WritableEditorOnly = Material->GetEditorOnlyData())
		{
			WritableEditorOnly->ResetSubstrateConversionVersion();
		}
	}
}

// ~RF_Standalone lets the preview GC once the editor releases its reference.
static UMaterial* DuplicatePreviewMaterial(UMaterial* Source)
{
	UMaterial* Preview = (UMaterial*)StaticDuplicateObject(Source, GetTransientPackage(), NAME_None, ~RF_Standalone, UPreviewMaterial::StaticClass());
	Preview->CancelOutstandingCompilation();
	Preview->bAllowDevelopmentShaderCompile = CVarMaterialEdUseDevShaders.GetValueOnGameThread();
	Preview->bIsPreviewMaterial = true;
	return Preview;
}

// Usage flags are duplicate-transient (prevents accidental new special-engine materials); copy manually.
static void CopyMaterialUsageFlags(const UMaterial* From, UMaterial* To)
{
	for (int32 Usage = 0; Usage < MATUSAGE_MAX; ++Usage)
	{
		const EMaterialUsage UsageEnum = (EMaterialUsage)Usage;
		To->SetUsageByFlag(UsageEnum, From->GetUsageByFlag(UsageEnum));
	}
	To->bUsedAsSpecialEngineMaterial = From->bUsedAsSpecialEngineMaterial;
}

// Restore back-pointers that serializers and duplicators null out.
static void RestoreExpressionBackReferences(UMaterial* Material)
{
	for (UMaterialExpression* MaterialExpression : Material->GetExpressions())
	{
		if (!MaterialExpression)
		{
			continue;
		}
		MaterialExpression->Material = Material;
		if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(MaterialExpression->GraphNode))
		{
			GraphNode->MaterialExpression = MaterialExpression;
		}
	}
}

// Function-edit mode wraps a UMaterialFunction's expressions in a wrapper UMaterial so the material editor can drive them. The expressions must report Material = the
// wrapper and Function = nullptr; dispatch code that keys off Expression->Function would otherwise route them through the function path instead of the material path.
static void RebindExpressionsToMaterial(UMaterial* Material)
{
	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		if (!Expression)
		{
			continue;
		}
		Expression->Function = nullptr;
		Expression->Material = Material;
	}
}

static bool IsSubstrateAutoConvertedAndNotSaved(const UMaterial* Material)
{
	if (Substrate::IsSubstrateEnabled())
	{
		if (const UMaterialEditorOnlyData* EditorOnly = Material->GetEditorOnlyData())
		{
			return EditorOnly->IsSubstrateAutoConverted() && EditorOnly->bIsSubsutrateAutoConvertedAndNotSaved;
		}
	}
	return false;
}

static bool MarkSubstrateAutoConversionSaved(UMaterial* Material)
{
	if (Substrate::IsSubstrateEnabled())
	{
		if (UMaterialEditorOnlyData* EditorOnly = Material->GetEditorOnlyData())
		{
			if (EditorOnly->IsSubstrateAutoConverted() && EditorOnly->bIsSubsutrateAutoConvertedAndNotSaved)
			{
				EditorOnly->bIsSubsutrateAutoConvertedAndNotSaved = false;
			}
		}
	}
	return false;
}

///////////////////////////
// FMatExpressionPreview //
///////////////////////////

FMatExpressionPreview::FMatExpressionPreview()
	: FEditorMaterialResourceBase()
	, FMaterialRenderProxy(TEXT("FMatExpressionPreview"))
	, UnrelatedNodesOpacity(1.0f)
{
	// Register this FMaterial derivative with AddEditorLoadedMaterialResource since it does not have a corresponding UMaterialInterface
	FMaterial::AddEditorLoadedMaterialResource(this);
	SetQualityLevelProperties(GMaxRHIShaderPlatform);
}

FMatExpressionPreview::FMatExpressionPreview(UMaterialExpression* InExpression)
	: FEditorMaterialResourceBase()
	, FMaterialRenderProxy(GetPathNameSafe(InExpression->Material))
	, UnrelatedNodesOpacity(1.0f)
	, Expression(InExpression)
{
	FMaterial::AddEditorLoadedMaterialResource(this);
	FPlatformMisc::CreateGuid(Id);

	check(InExpression->Material && InExpression->Material->GetExpressions().Contains(InExpression));
	SetQualityLevelProperties(GMaxRHIShaderPlatform);

	UMaterial* BaseMaterial = InExpression->Material;
	ReferencedTextures = InExpression->Material->GetDefaultTextures();
	ReferencedTextureCollections = InExpression->Material->GetReferencedTextureCollections();
}

FMatExpressionPreview::~FMatExpressionPreview()
{
}

void FMatExpressionPreview::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ReferencedTextures);
	Collector.AddReferencedObjects(ReferencedTextureCollections);
}

bool FMatExpressionPreview::ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const
{
	if(VertexFactoryType == FindVertexFactoryType(FName(TEXT("FLocalVertexFactory"), FNAME_Find)))
	{
		// we only need the non-light-mapped, base pass, local vertex factory shaders for drawing an opaque Material Tile
		// @todo: Added a FindShaderType by fname or something"

		if (IsMobilePlatform(GetShaderPlatform()))
		{
			if (FCString::Stristr(ShaderType->GetName(), TEXT("BasePassForForwardShadingVSFNoLightMapPolicy")) ||
				FCString::Stristr(ShaderType->GetName(), TEXT("BasePassForForwardShadingPSFNoLightMapPolicy")))
			{
				return true;
			}
		}
		else
		{
			if (FCString::Stristr(ShaderType->GetName(), TEXT("BasePassVSFNoLightMapPolicy")) ||
				FCString::Stristr(ShaderType->GetName(), TEXT("BasePassHSFNoLightMapPolicy")) ||
				FCString::Stristr(ShaderType->GetName(), TEXT("BasePassDSFNoLightMapPolicy")))
			{
				return true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("BasePassPSFNoLightMapPolicy")))
			{
				return true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("Simple")))
			{
				return true;
			}
		}
	}

	return false;
}

int32 FMatExpressionPreview::CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const
{
	// Early out if the compiler wishes to terminate translation.
	if (Compiler->ShouldStopTranslating())
	{
		return INDEX_NONE;
	}

	// needs to be called in this function!!
	Compiler->SetMaterialProperty(Property, OverrideShaderFrequency, bUsePreviousFrameTime);

	if(Substrate::IsSubstrateEnabled())
	{
		// Set the Substrate export mode to material preview
		Compiler->SetSubstrateMaterialExportType(SME_MaterialPreview, ESubstrateMaterialExportContext::SMEC_Opaque, 0);
	}

	int32 Ret = INDEX_NONE;

	if( Property == MP_EmissiveColor && Expression.IsValid())
	{
		// Hardcoding output 0 as we don't have the UI to specify any other output
		const int32 OutputIndex = 0;
		int32 PreviewCodeChunk = INDEX_NONE;
		PreviewCodeChunk = Expression->CompilePreview(Compiler, OutputIndex);

		if (PreviewCodeChunk != INDEX_NONE)
		{
			const EMaterialValueType CompiledType = Compiler->GetType(PreviewCodeChunk);
			CompiledPreviewNumChannels = static_cast<int32>(GetNumComponents(MakeNonLWCType(CompiledType)));
		}
		else
		{
			CompiledPreviewNumChannels = 0;
		}

		// Get back into gamma corrected space, as DrawTile does not do this adjustment.
		Ret = Compiler->Power(Compiler->Max(PreviewCodeChunk, Compiler->Constant(0)), Compiler->Constant(1.f / 2.2f));
	}
	else if (Property == MP_WorldPositionOffset || Property == MP_Displacement)
	{
		//set to 0 to prevent off by 1 pixel errors
		Ret = Compiler->Constant(0.0f);
	}
	else if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
	{
		const int32 TextureCoordinateIndex = Property - MP_CustomizedUVs0;
		Ret = Compiler->TextureCoordinate(TextureCoordinateIndex, false, false);
	}
	else if (Property == MP_ShadingModel)
	{
		FMaterialShadingModelField ShadingModels = Compiler->GetMaterialShadingModels();
		Ret = Compiler->ShadingModel(ShadingModels.GetFirstShadingModel());
	}
	else if (Property == MP_FrontMaterial)
	{
		// No need to compile the front material: when previewing a node, the FrontMaterial is plugged into the emissive color.
		// Then CompilePreview is called, and this is where we convert the Substrate material to a single color for preview.
		// That single color is then scheduled to be output thanks to setting the compiler as SME_MaterialPreview. 
		return Compiler->SubstrateCreateAndRegisterNullMaterial();
	}
	else
	{
		Ret = Compiler->Constant(1.0f);
	}

	// output should always be the right type for this property
	return Compiler->ForceCast(Ret, FMaterialAttributeDefinitionMap::GetValueType(Property), MFCF_ExactMatch);
}

UMaterialInterface* FMatExpressionPreview::GetMaterialInterface() const
{
	if (Expression.IsValid())
	{
		UMaterial* ExprMat = Expression->Material;
		if (ExprMat)
		{
			FMaterialRenderProxy* MatProxy = ExprMat->GetRenderProxy();
			if (MatProxy)
			{
				return MatProxy->GetMaterialInterface();
			}
		}
	}

	return nullptr;
}

void FMatExpressionPreview::NotifyCompilationFinished()
{
	if (Expression.IsValid() && Expression->GraphNode)
	{
		CastChecked<UMaterialGraphNode>(Expression->GraphNode)->bPreviewNeedsUpdate = true;
	}
	FMaterialRenderProxy::CacheUniformExpressions_GameThread(true);
}

bool FMatExpressionPreview::IsUsingNewHLSLGenerator() const
{
	if (Expression.IsValid() && Expression->Material)
	{
		return Expression->Material->IsUsingNewHLSLGenerator();
	}
	return false;
}

bool FMatExpressionPreview::CheckInValidStateForCompilation(FMaterialCompiler* Compiler) const
{
	return Expression.IsValid() && Expression->Material && Expression->Material->CheckInValidStateForCompilation(Compiler);
}

/////////////////////
// FMaterialEditor //
/////////////////////

void FMaterialEditor::RegisterToolbarTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MaterialEditor", "Material Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::PreviewTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_Preview) )
		.SetDisplayName( LOCTEXT("ViewportTab", "Viewport") )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::PropertiesTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_MaterialProperties) )
		.SetDisplayName( LOCTEXT("DetailsTab", "Details") )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::PaletteTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_Palette) )
		.SetDisplayName( LOCTEXT("PaletteTab", "Palette") )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::NamedReroutesTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_NamedReroutes))
		.SetDisplayName(LOCTEXT("NamedReroutesTab", "Named Reroutes"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::FindTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_Find))
		.SetDisplayName(LOCTEXT("FindTab", "Find Results"))
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::PreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_PreviewSettings))
		.SetDisplayName( LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings") )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::ParameterDefaultsTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_ParameterDefaults))
		.SetDisplayName(LOCTEXT("ParametersTab", "Parameters"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::CustomPrimitiveTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_CustomPrimitiveData))
		.SetDisplayName(LOCTEXT("CustomPrimitiveTab", "Custom Primitive Data"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::LayerPropertiesTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_LayerProperties))
		.SetDisplayName(LOCTEXT("LayerPropertiesTab", "Layer Parameters"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Layers"));

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::SubstrateTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_Substrate))
		.SetDisplayName(LOCTEXT("SubstrateTab", "Substrate"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));	// SUBSTRATE_TODO a Substrate icon

	InTabManager->RegisterTabSpawner(FMaterialEditorTabs::MaterialOverviewTabId, FOnSpawnTab::CreateSP(this, &FMaterialEditor::SpawnTab_MaterialExpressionsOverview))
		.SetDisplayName(LOCTEXT("MaterialOverviewTab", "Overview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	MaterialStatsManager->RegisterTabs();

	OnRegisterTabSpawners().Broadcast(InTabManager);
}

void FMaterialEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	DocumentManager->SetTabManager(InTabManager);
	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);
}

void FMaterialEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::PreviewTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::PropertiesTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::PaletteTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::NamedReroutesTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::FindTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::PreviewSettingsTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::ParameterDefaultsTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::CustomPrimitiveTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::LayerPropertiesTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::SubstrateTabId);
	InTabManager->UnregisterTabSpawner(FMaterialEditorTabs::MaterialOverviewTabId);

	MaterialStatsManager->UnregisterTabs();

	OnUnregisterTabSpawners().Broadcast(InTabManager);
}

void FMaterialEditor::InitEditorForMaterial(UMaterial* InMaterial)
{
	check(InMaterial);

	OriginalMaterial = InMaterial;
	MaterialFunction = NULL;
	OriginalMaterialObject = InMaterial;

	ExpressionPreviewMaterial = NULL;

	Material = DuplicatePreviewMaterial(OriginalMaterial);

	// Ensure there are no null entries
	check(Material->GetExpressions().Find(nullptr) == INDEX_NONE);

	TArray<FString> Groups;
	UpdateAndGetMaterialExpressionGroups(&Groups);
}

void FMaterialEditor::InitEditorForMaterialFunction(UMaterialFunction* InMaterialFunction)
{
	check(InMaterialFunction);

	Material = NULL;
	MaterialFunction = InMaterialFunction;
	OriginalMaterialObject = InMaterialFunction;

	ExpressionPreviewMaterial = NULL;

	// Create a temporary material to preview the material function
	Material = NewObject<UMaterial>(GetTransientPackage(), NAME_None, RF_Transactional);
	{
		FArchiveUObject DummyArchive;
		// Hack: serialize the new material with an archive that does nothing so that its material resources are created
		Material->Serialize(DummyArchive);
	}
	Material->SetShadingModel(MSM_Unlit);
	Material->MaterialDomain = InMaterialFunction->PreviewMaterialDomain;

	// Propagate all object flags except for RF_Standalone, otherwise the preview material function won't GC once
	// the material editor releases the reference.
	MaterialFunction = (UMaterialFunction*)StaticDuplicateObject(InMaterialFunction, GetTransientPackage(), NAME_None, ~RF_Standalone, UMaterialFunction::StaticClass()); 
	MaterialFunction->ParentFunction = InMaterialFunction;

	OriginalMaterial = Material;

	TArray<FString> Groups;
	UpdateAndGetMaterialExpressionGroups(&Groups);
}

void FMaterialEditor::InitMaterialEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit )
{
	EditorOptions = NULL;
	bMaterialDirty = false;
	bStatsFromPreviewMaterial = false;

	// Support undo/redo
	Material->SetFlags(RF_Transactional);

	GEditor->RegisterForUndo(this);

	MaterialStatsManager = FMaterialStatsUtils::CreateMaterialStats(this, true, GetDefault<UMaterialEditorSettings>()->bAllowIgnoringCompilationErrors);
	MaterialStatsManager->SetMaterialsDisplayNames({OriginalMaterial->GetName()});
	MaterialStatsManager->GetOldStatsListing()->OnMessageTokenClicked().AddSP(this, &FMaterialEditor::OnMessageLogLinkActivated);

	if (!Material->MaterialGraph)
	{
		Material->MaterialGraph = CastChecked<UMaterialGraph>(FBlueprintEditorUtils::CreateNewGraph(Material, NAME_None, UMaterialGraph::StaticClass(), UMaterialGraphSchema::StaticClass()));
	}
	Material->MaterialGraph->Material = Material;
	Material->MaterialGraph->MaterialFunction = MaterialFunction;
	Material->MaterialGraph->RealtimeDelegate.BindSP(this, &FMaterialEditor::IsToggleRealTimeExpressionsChecked);
	Material->MaterialGraph->MaterialDirtyDelegate.BindSP(this, &FMaterialEditor::SetMaterialDirty);
	Material->MaterialGraph->ToggleCollapsedDelegate.BindSP(this, &FMaterialEditor::ToggleCollapsed);

	CopyMaterialUsageFlags(OriginalMaterial, Material);
	
	NodeQualityLevel = EMaterialQualityLevel::Num;
	NodeFeatureLevel = ERHIFeatureLevel::Num;
	bPreviewStaticSwitches = false;
	bPreviewFeaturesChanged = true;

	// Register our commands. This will only register them if not previously registered
	FGraphEditorCommands::Register();
	FMaterialEditorCommands::Register();
	FMaterialEditorSpawnNodeCommands::Register();

	FEditorSupportDelegates::MaterialUsageFlagsChanged.AddRaw(this, &FMaterialEditor::OnMaterialUsageFlagsChanged);
	FEditorSupportDelegates::NumericParameterDefaultChanged.AddRaw(this, &FMaterialEditor::OnNumericParameterDefaultChanged);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	
	AssetRegistryModule.Get().OnAssetRenamed().AddSP( this, &FMaterialEditor::RenameAssetFromRegistry );

	CreateInternalWidgets();

	// Do setup previously done in SMaterialEditorCanvas
	SetPreviewMaterial(Material);
	Material->bIsPreviewMaterial = true;
	FMaterialEditorUtilities::InitExpressions(Material);

	UpdatePreviewViewportsVisibility();

	BindCommands();
	RegisterToolBar();

	TSharedPtr<FMaterialEditor> ThisPtr(SharedThis(this));
	DocumentManager->Initialize(ThisPtr);

	// Register the document factories
	{
		TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShareable(new FMaterialGraphEditorSummoner(ThisPtr,
			FMaterialGraphEditorSummoner::FOnCreateGraphEditorWidget::CreateSP(this, &FMaterialEditor::CreateGraphEditorWidget)
		));

		// Also store off a reference to the grapheditor factory so we can find all the tabs spawned by it later.
		GraphEditorTabFactoryPtr = GraphEditorFactory;
		DocumentManager->RegisterDocumentFactory(GraphEditorFactory);
	}

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;

	// Add the preview material to the objects being edited, so that we can find this editor from the temporary material graph
	const TSharedRef<FTabManager::FLayout> DummyLayout = FTabManager::NewLayout("NullLayout")->AddArea(FTabManager::NewPrimaryArea());
	TArray< UObject* > ObjectsToEdit;
	ObjectsToEdit.Add(ObjectToEdit);
	ObjectsToEdit.Add(Material);
	ObjectsToEdit.Add(MaterialEditorInstance);
	InitAssetEditor( Mode, InitToolkitHost, MaterialEditorAppIdentifier, DummyLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit, false );
	AddMenuExtender(GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
	AddMenuExtender(MaterialEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	{
		AddApplicationMode(
			FMaterialEditorApplicationModes::StandardMaterialEditorMode,
			MakeShareable(new FMaterialEditorApplicationMode(SharedThis(this))));
		SetCurrentMode(FMaterialEditorApplicationModes::StandardMaterialEditorMode);
	}

	// @todo toolkit world centric editing
	/*if( IsWorldCentricAssetEditor() )
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		SpawnToolkitTab(PreviewTabId, FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(GraphCanvasTabId, FString(), EToolkitTabSpot::Document);
		SpawnToolkitTab(PropertiesTabId, FString(), EToolkitTabSpot::Details);
	}*/
	

	// Load editor settings from disk.
	LoadEditorSettings();

	// Initialize expression previews.
	if (MaterialFunction)
	{
		// Support undo/redo for the material function if it exists
		MaterialFunction->SetFlags(RF_Transactional);

		MaterialFunction->MaterialGraph = Material->MaterialGraph;
		MaterialFunction->EditorMaterial = Material;

		Material->AssignExpressionCollection(MaterialFunction->GetExpressionCollection());
		Material->bEnableNewHLSLGenerator = MaterialFunction->IsUsingNewHLSLGenerator();

		// Ensure there are no null entries
		check(Material->GetExpressions().Find(nullptr) == INDEX_NONE);

		if (Material->GetExpressions().Num() == 0)
		{
			// If this is an empty function, create an output by default and start previewing it
			if (FocusedGraphEdPtr.IsValid())
			{
				check(!bMaterialDirty);
				FVector2D OutputPlacement = FVector2D(200, 300);
				if (GetDefault<UEditorExperimentalSettings>()->bExampleLayersAndBlends)
				{
					switch (MaterialFunction->GetMaterialFunctionUsage())
					{
					case(EMaterialFunctionUsage::MaterialLayer):
					{
						OutputPlacement = FVector2D(300, 269);
						break;
					}
					case(EMaterialFunctionUsage::MaterialLayerBlend):
					{
						OutputPlacement = FVector2D(275, 269);
						break;
					}
					default:
						break;
					}
				}
				UMaterialExpression* Expression;
				if (MaterialFunction->GetMaterialFunctionUsage() == EMaterialFunctionUsage::Default)
				{
					Expression = CreateNewMaterialExpression(UMaterialExpressionFunctionOutput::StaticClass(), OutputPlacement, false, true);
					SetPreviewExpression(Expression);
					// This shouldn't count as having dirtied the material, so reset the flag
					bMaterialDirty = false;
				}
				else
				{
					Expression = CreateNewMaterialExpression(UMaterialExpressionMaterialLayerOutput::StaticClass(), OutputPlacement, false, true);
					Expression->bCollapsed = true;
					SetPreviewExpression(Expression);
					// This shouldn't count as having dirtied the material, so reset the flag
					bMaterialDirty = false;
				}
				// We can check the usage here and add the appropriate inputs too (e.g. Layer==1MA, Blend==2MA)
				if (MaterialFunction->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayer)
				{
					UMaterialExpression* Input = CreateNewMaterialExpression(UMaterialExpressionFunctionInput::StaticClass(), FVector2D(-350, 300), false, true);
					if (Input)
					{
						UMaterialExpressionFunctionInput* BaseAttributesInput = Cast<UMaterialExpressionFunctionInput>(Input);
						BaseAttributesInput->InputType = FunctionInput_MaterialAttributes;
						BaseAttributesInput->InputName = TEXT("Material Attributes");
						BaseAttributesInput->bUsePreviewValueAsDefault = true;
					}

					if (GetDefault<UEditorExperimentalSettings>()->bExampleLayersAndBlends)
					{
						UMaterialExpression* SetMaterialAttributes = CreateNewMaterialExpression(UMaterialExpressionSetMaterialAttributes::StaticClass(), FVector2D(40, 300), false, true);
						if (Input && SetMaterialAttributes)
						{
						UMaterialEditingLibrary::ConnectMaterialExpressions(Input, FString(), SetMaterialAttributes, FString());
							UMaterialEditingLibrary::ConnectMaterialExpressions(SetMaterialAttributes, FString(), Expression, FString());
							bMaterialDirty = true;
						}
					}
				}
				else if (MaterialFunction->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayerBlend)
				{
					// "Top layer" should be below "bottom layer" on the graph, to align with B on blend nodes
					UMaterialExpression* InputTop = CreateNewMaterialExpression(UMaterialExpressionFunctionInput::StaticClass(), FVector2D(-300, 400), false, true);
					if (InputTop)
					{
						UMaterialExpressionFunctionInput* BaseAttributesInput = Cast<UMaterialExpressionFunctionInput>(InputTop);
						BaseAttributesInput->InputType = FunctionInput_MaterialAttributes;
						BaseAttributesInput->InputName = TopMaterialBlendInputName;
						BaseAttributesInput->bUsePreviewValueAsDefault = true;
						BaseAttributesInput->BlendInputRelevance = EBlendInputRelevance::Top;
					}

					UMaterialExpression* InputBottom = CreateNewMaterialExpression(UMaterialExpressionFunctionInput::StaticClass(), FVector2D(-300, 200), false, true);
					if (InputBottom)
					{
						UMaterialExpressionFunctionInput* BaseAttributesInput = Cast<UMaterialExpressionFunctionInput>(InputBottom);
						BaseAttributesInput->InputType = FunctionInput_MaterialAttributes;
						BaseAttributesInput->InputName = BottomMaterialBlendInputName;
						BaseAttributesInput->bUsePreviewValueAsDefault = true;
						BaseAttributesInput->BlendInputRelevance = EBlendInputRelevance::Bottom;
					}
					if (GetDefault<UEditorExperimentalSettings>()->bExampleLayersAndBlends)
					{
						UMaterialExpression* BlendMaterialAttributes = CreateNewMaterialExpression(UMaterialExpressionBlendMaterialAttributes::StaticClass(), FVector2D(40, 300), false, true);
						if (InputTop && InputBottom && BlendMaterialAttributes)
						{
							UMaterialEditingLibrary::ConnectMaterialExpressions(InputBottom, FString(), BlendMaterialAttributes, FString(TEXT("A")));
							UMaterialEditingLibrary::ConnectMaterialExpressions(InputTop, FString(), BlendMaterialAttributes, FString(TEXT("B")));
							UMaterialEditingLibrary::ConnectMaterialExpressions(BlendMaterialAttributes, FString(), Expression, FString());
							bMaterialDirty = true;
						}
					}
				}
			}
		}
		else
		{
			bool bSetPreviewExpression = false;
			UMaterialExpressionFunctionOutput* FirstOutput = NULL;
			for (int32 ExpressionIndex = Material->GetExpressions().Num() - 1; ExpressionIndex >= 0; ExpressionIndex--)
			{
				UMaterialExpression* Expression = Material->GetExpressions()[ExpressionIndex];

				// Setup the expression to be used with the preview material instead of the function
				Expression->Function = NULL;
				Expression->Material = Material;

				UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);
				if (FunctionOutput)
				{
					FirstOutput = FunctionOutput;
					if (FunctionOutput->bLastPreviewed)
					{
						bSetPreviewExpression = true;

						// Preview the last output previewed
						SetPreviewExpression(FunctionOutput);
					}
				}
			}

			if (!bSetPreviewExpression && FirstOutput)
			{
				SetPreviewExpression(FirstOutput);
			}
		}
	}

	// Store the name of this material (for the tutorial widget meta)
	if (OriginalMaterial != nullptr)
	{
		Material->MaterialGraph->OriginalMaterialFullName = OriginalMaterial->GetName();
	}
	Material->MaterialGraph->RebuildGraph();
	RecenterEditor();

	//Make sure the preview material is initialized.
	UpdatePreviewMaterial(true);
	RegenerateCodeView(true);

	ForceRefreshExpressionPreviews();
	if (Generator.IsValid() && MaterialEditorInstance)
	{
		UpdateGenerator();
	}

	// Register for pre/post world cleanup
	OnWorldCleanupDelegateHandle = FWorldDelegates::OnWorldCleanup.AddSP(this, &FMaterialEditor::OnWorldCleanup);
	OnPostWorldCleanupDelegateHandle = FWorldDelegates::OnPostWorldCleanup.AddSP(this, &FMaterialEditor::OnPostWorldCleanup);

	if (OriginalMaterial->bUsedAsSpecialEngineMaterial)
	{
		FSuppressableWarningDialog::FSetupInfo Info(
		NSLOCTEXT("UnrealEd", "Warning_EditingDefaultMaterial", "Editing this Default Material is an advanced workflow.\nDefault Materials must be available as a code fallback at all times, compilation errors are not handled gracefully.  Save your work."),
		NSLOCTEXT("UnrealEd", "Warning_EditingDefaultMaterial_Title", "Warning: Editing Default Material" ), "Warning_EditingDefaultMaterial");
		Info.ConfirmText = NSLOCTEXT("ModalDialogs", "EditingDefaultMaterialOk", "Ok");

		FSuppressableWarningDialog EditingDefaultMaterial( Info );
		EditingDefaultMaterial.ShowModal();	
	}

	if (GetDefault<UEditorExperimentalSettings>()->bExampleLayersAndBlends && bMaterialDirty)
	{
		SaveAsset_Execute();
	}

	// Notify other editors if this material editor has a post process named output, which may affect their preview
	NotifyUserSceneTextureLoadOrUnload();
}

void FMaterialEditor::UpdateGenerator()
{
	if (MaterialEditorInstance && Generator.IsValid())
	{
		MaterialEditorInstance->RegenerateArrays();
		TArray<UObject*> Objects;
		Objects.Add(MaterialEditorInstance);
		Generator->SetObjects(Objects);

		if (MaterialLayersFunctionsInstance.IsValid())
		{
		 	MaterialLayersFunctionsInstance->SetEditorInstance(MaterialEditorInstance);
		}
	}
}

void FMaterialEditor::NavigateTab(FDocumentTracker::EOpenDocumentCause InCause)
{
	OpenDocument(nullptr, InCause);
}

FMaterialEditor::FMaterialEditor()
	: bMaterialDirty(false)
	, bStatsFromPreviewMaterial(false)
	, Material(nullptr)
	, OriginalMaterial(nullptr)
	, ExpressionPreviewMaterial(nullptr)
	, EmptyMaterial(nullptr)
	, PreviewExpression(nullptr)
	, MaterialFunction(nullptr)
	, OriginalMaterialObject(nullptr)
	, EditorOptions(nullptr)
	, ScopedTransaction(nullptr)
	, bAlwaysRefreshAllPreviews(false)
	, bHideUnusedConnectors(false)
	, bLivePreview(true)
	, bIsRealtime(false)
	, bShowBuiltinStats(false)
	, bHideUnrelatedNodes(false)
	, bLockNodeFadeState(false)
	, bFocusWholeChain(false)
	, bSelectFocusedNodes(false)
	, bFocusSelectionInProgress(false)
	, bSelectRegularNode(false)
	, MenuExtensibilityManager(new FExtensibilityManager)
	, ToolBarExtensibilityManager(new FExtensibilityManager)
	, MaterialEditorInstance(nullptr)
{
	DocumentManager = MakeShareable(new FDocumentTracker);
}

FMaterialEditor::~FMaterialEditor()
{
	bDestructing = true;

	if(Material != nullptr)
	{
		Material->SetMarkTextureAsEditorStreamingPool(false);
	}

	if (OnWorldCleanupDelegateHandle.IsValid())
	{
		FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupDelegateHandle);
		OnWorldCleanupDelegateHandle.Reset();
	}

	if (OnPostWorldCleanupDelegateHandle.IsValid())
	{
		FWorldDelegates::OnPostWorldCleanup.Remove(OnPostWorldCleanupDelegateHandle);
		OnPostWorldCleanupDelegateHandle.Reset();
	}

	// Notify other editors if this material editor has a post process named output, which may affect their preview
	NotifyUserSceneTextureLoadOrUnload();

	// Broadcast that this editor is going down to all listeners
	OnMaterialEditorClosed().Broadcast();

	for (const auto& It : OverriddenNumericParametersToRevert)
	{
		ClearNumericParameterDefaultOnDependentMaterials(It.Key, It.Value);
	}

	if (OnGraphChangedDelegateHandle.IsValid() && Material && Material->MaterialGraph)
	{
		Material->MaterialGraph->RemoveOnGraphChangedHandler(OnGraphChangedDelegateHandle);
		OnGraphChangedDelegateHandle.Reset();
	}

	// Unregister this delegate
	FEditorSupportDelegates::MaterialUsageFlagsChanged.RemoveAll(this);
	FEditorSupportDelegates::NumericParameterDefaultChanged.RemoveAll(this);

	// Null out the expression preview material so they can be GC'ed
	ExpressionPreviewMaterial = NULL;

	// Save editor settings to disk.
	SaveEditorSettings();

	MaterialDetailsView.Reset();

	{
		FMaterial::DeferredDeleteArray(ExpressionPreviews);
	}
	
	check( !ScopedTransaction );
	
	GEditor->UnregisterForUndo( this );

	MaterialEditorInstance = NULL;
}

void FMaterialEditor::UpdateAndGetMaterialExpressionGroups(TArray<FString>* OutGroups)
{
	UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();
	if (!EditorOnlyData)
	{
		return;
	}

	UpdateMaterialExpressionGroups();
	for (const FParameterGroupData& GroupData : EditorOnlyData->ParameterGroupData)
	{
		OutGroups->Add(GroupData.GroupName);
	}
}

void FMaterialEditor::UpdateMaterialExpressionGroups()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialInstanceEditor::UpdateMaterialExpressionGroups)

	UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();
	if (!EditorOnlyData)
	{
		return;
	}

	TArray<UMaterialExpression*> ExpressionsToParse = TArray<UMaterialExpression*>(Material->GetExpressions());

	if (CVarMaterialEdGroupSortIncludeFunctions.GetValueOnGameThread())
	{
		TSet<UMaterialExpression*> FunctionsExpressions;

		for (const UMaterialExpression* MaterialExpression : Material->GetExpressions())
		{
			if (const UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression))
			{
				if (FunctionCall->MaterialFunction)
				{
					TArray<UMaterialExpression*> FunctionExpressions;
					FunctionCall->MaterialFunction->GetAllExpressionsOfType<UMaterialExpression>(FunctionExpressions);
					FunctionsExpressions.Append(FunctionExpressions);
				}
			}
		}

		ExpressionsToParse.Append(FunctionsExpressions.Array());
	}

	TSet<FParameterGroupData> UpdatedGroups;
	for (const UMaterialExpression* MaterialExpression : ExpressionsToParse)
	{
		FMaterialParameterMetadata ParameterMeta;
		if (MaterialExpression->GetParameterValue(ParameterMeta))
		{
			if (ParameterMeta.Group.IsNone())
			{
				continue;
			}

			const FString GroupName = ParameterMeta.Group.ToString();

			if (const FParameterGroupData* GroupData = Material->GetSortingGroup(GroupName))
			{
				UpdatedGroups.Add(*GroupData);
			}
			else
			{
				UpdatedGroups.Add(FParameterGroupData(GroupName, 0));
			}
		}
	}

	TArray<FParameterGroupData> UpdatedGroupsArray = UpdatedGroups.Array();
	UpdatedGroupsArray.Sort(); // Sort groups based on name

	// This indirectly gets rid of any groups which might have been removed
	EditorOnlyData->ParameterGroupData = UpdatedGroupsArray;
}

void FMaterialEditor::UpdatePreviewViewportsVisibility()
{
	if( Material->IsUIMaterial() )
	{
		PreviewViewport->SetVisibility(EVisibility::Collapsed);
		PreviewUIViewport->SetVisibility(EVisibility::Visible);
	}
	else
	{
		PreviewViewport->SetVisibility(EVisibility::Visible);
		PreviewUIViewport->SetVisibility(EVisibility::Collapsed);
	}
}

void FMaterialEditor::NotifyUserSceneTextureLoadOrUnload()
{
	if (Material->IsPostProcessMaterial() && !Material->UserSceneTexture.IsNone())
	{
		FMaterialEditorUtilities::RefreshPostProcessPreviewMaterials(Material);
	}
}

static void AssignPinSourceIndices(UEdGraphNode* Node)
{
	int32 NumInputDataPins = 0;
	int32 NumOutputDataPins = 0;
	int32 NumInputExecPins = 0;
	int32 NumOutputExecPins = 0;

	for (UEdGraphPin* Pin : Node->Pins)
	{
		int32 SourceIndex = INDEX_NONE;
		if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
		{
			switch (Pin->Direction)
			{
			case EGPD_Input: SourceIndex = NumInputExecPins++; break;
			case EGPD_Output: SourceIndex = NumOutputExecPins++; break;
			default: checkNoEntry(); break;
			}
		}
		else
		{
			switch (Pin->Direction)
			{
			case EGPD_Input: SourceIndex = NumInputDataPins++; break;
			case EGPD_Output: SourceIndex = NumOutputDataPins++; break;
			default: checkNoEntry(); break;
			}
		}

		if (Pin->SourceIndex == INDEX_NONE)
		{
			Pin->SourceIndex = SourceIndex;
		}
		ensure(Pin->SourceIndex == SourceIndex);
	}
}

void FMaterialEditor::CollapseNodesIntoGraph(UEdGraphNode* InGatewayNode, UMaterialGraphNode* InEntryNode, UMaterialGraphNode* InResultNode, UEdGraph* InSourceGraph, UEdGraph* InDestinationGraph, TSet<UEdGraphNode*>& InCollapsableNodes)
{
	const UMaterialGraphSchema* MaterialSchema = GetDefault<UMaterialGraphSchema>(); // ?

	// Keep track of the statistics of the node positions so the new nodes can be located reasonably well
	float SumNodeX = 0.0f;
	float SumNodeY = 0.0f;
	float MinNodeX = 1e9f;
	float MinNodeY = 1e9f;
	float MaxNodeX = -1e9f;
	float MaxNodeY = -1e9f;
	
	// Move the nodes over, which may create cross-graph references that we need fix up ASAP
	for (TSet<UEdGraphNode*>::TConstIterator NodeIt(InCollapsableNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = *NodeIt;
		Node->Modify();

		// Update stats
		SumNodeX += Node->NodePosX;
		SumNodeY += Node->NodePosY;
		MinNodeX = FMath::Min<float>(MinNodeX, Node->NodePosX);
		MinNodeY = FMath::Min<float>(MinNodeY, Node->NodePosY);
		MaxNodeX = FMath::Max<float>(MaxNodeX, Node->NodePosX);
		MaxNodeY = FMath::Max<float>(MaxNodeY, Node->NodePosY);

		// Move the node over
		InSourceGraph->Nodes.Remove(Node);
		InDestinationGraph->Nodes.Add(Node);
		Node->Rename(/*NewName=*/ NULL, /*NewOuter=*/ InDestinationGraph);

		// Move the sub-graph to the new graph
		if(UMaterialGraphNode_Composite* Composite = Cast<UMaterialGraphNode_Composite>(Node))
		{
			InSourceGraph->SubGraphs.Remove(Composite->BoundGraph);
			InDestinationGraph->SubGraphs.Add(Composite->BoundGraph);
			Composite->BoundGraph->SubgraphExpression = CastChecked<UMaterialGraphNode_Composite>(InGatewayNode)->MaterialExpression;
		}

		// Mark the node's expression as owned by the gateway node's expression
		UMaterialGraphNode* GatewayMaterialNode = CastChecked<UMaterialGraphNode>(InGatewayNode);
		if (UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Node))
		{
			MaterialNode->MaterialExpression->SubgraphExpression = GatewayMaterialNode->MaterialExpression;
		}
		else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(Node))
		{
			CommentNode->MaterialExpressionComment->SubgraphExpression = GatewayMaterialNode->MaterialExpression;
		}

		TArray<UEdGraphPin*> OutputGatewayExecPins;

		// Find cross-graph links
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* LocalPin = Node->Pins[PinIndex];

			bool bIsGatewayPin = false;
			if(LocalPin->LinkedTo.Num())
			{
				for (int32 LinkIndex = 0; LinkIndex < LocalPin->LinkedTo.Num(); ++LinkIndex)
				{
					UEdGraphPin* TrialPin = LocalPin->LinkedTo[LinkIndex];
					if (!InCollapsableNodes.Contains(TrialPin->GetOwningNode()))
					{
						bIsGatewayPin = true;
						break;
					}
				}
			}

			// Thunk cross-graph links thru the gateway
			if (bIsGatewayPin)
			{
				// Local port is either the entry or the result node in the collapsed graph
				// Remote port is the node placed in the source graph
				UMaterialGraphNode* LocalPort = (LocalPin->Direction == EGPD_Input) ? InEntryNode : InResultNode;

				// Add a new pin to the entry/exit node and to the composite node
				UEdGraphPin* LocalPortPin = NULL;
				UEdGraphPin* RemotePortPin = NULL;

				if(LocalPin->LinkedTo[0]->GetOwningNode() != InEntryNode)
				{
					const FName UniquePortName = InGatewayNode->CreateUniquePinName(LocalPin->PinName);

					if(!RemotePortPin && !LocalPortPin)
					{
						FEdGraphPinType PinType = LocalPin->PinType;
						RemotePortPin = InGatewayNode->CreatePin(LocalPin->Direction, PinType, UniquePortName);
						LocalPortPin = LocalPort->CreatePin((LocalPin->Direction == EGPD_Input) ? EGPD_Output : EGPD_Input, PinType, UniquePortName);

						// Create reroute expressions / pins for the pinbase.
						UMaterialGraphNode_Composite* CompositeNode = CastChecked<UMaterialGraphNode_Composite>(InGatewayNode);
						UMaterialExpression* GatewayRerouteExpression = UMaterialEditingLibrary::CreateMaterialExpressionEx(Material, MaterialFunction, UMaterialExpressionReroute::StaticClass());
						UMaterialExpressionReroute* GatewayReroutePin = CastChecked<UMaterialExpressionReroute>(GatewayRerouteExpression);
						UMaterialExpressionPinBase* PinBase = CastChecked<UMaterialExpressionPinBase>(LocalPort->MaterialExpression);

						GatewayRerouteExpression->SubgraphExpression = CompositeNode->MaterialExpression;
						PinBase->ReroutePins.Add(FCompositeReroute{UniquePortName, decltype(FCompositeReroute::Expression)(GatewayReroutePin)});
						PinBase->Modify();
					}
				}

				check(LocalPortPin);
				check(RemotePortPin);

				LocalPin->Modify();

				// Route the links
				for (int32 LinkIndex = 0; LinkIndex < LocalPin->LinkedTo.Num(); ++LinkIndex)
				{
					UEdGraphPin* RemotePin = LocalPin->LinkedTo[LinkIndex];
					RemotePin->Modify();

					if (!InCollapsableNodes.Contains(RemotePin->GetOwningNode()) && RemotePin->GetOwningNode() != InEntryNode && RemotePin->GetOwningNode() != InResultNode)
					{
						// Fix up the remote pin
						RemotePin->LinkedTo.Remove(LocalPin);
						RemotePin->MakeLinkTo(RemotePortPin);

						// The Entry Node only supports a single link, so if we made links above
						// we need to break them now, to make room for the new link.
						if (LocalPort == InEntryNode)
						{
							LocalPortPin->BreakAllPinLinks();
						}

						// Fix up the local pin
						LocalPin->LinkedTo.Remove(RemotePin);
						--LinkIndex;
						LocalPin->MakeLinkTo(LocalPortPin);
					}
				}
			}
		}
	}

	// Reposition the newly created nodes
	const int32 NumNodes = InCollapsableNodes.Num();
	const float CenterX = NumNodes == 0 ? SumNodeX : SumNodeX / NumNodes;
	const float CenterY = NumNodes == 0 ? SumNodeY : SumNodeY / NumNodes;
	const float MinusOffsetX = 160.0f; //@TODO: Random magic numbers
	const float PlusOffsetX = 300.0f;

	// Put the gateway node at the center of the empty space in the old graph
	InGatewayNode->NodePosX = CenterX;
	InGatewayNode->NodePosY = CenterY;
	InGatewayNode->SnapToGrid(SNodePanel::GetSnapGridSize());

	if (UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(InGatewayNode))
	{
		CompositeNode->MaterialExpression->MaterialExpressionEditorX = InGatewayNode->NodePosX;
		CompositeNode->MaterialExpression->MaterialExpressionEditorY = InGatewayNode->NodePosY;
	}

	// Put the entry and exit nodes on either side of the nodes in the new graph
	if (NumNodes != 0)
	{
		InEntryNode->NodePosX = MinNodeX - MinusOffsetX;
		InEntryNode->NodePosY = CenterY;
		InEntryNode->SnapToGrid(SNodePanel::GetSnapGridSize());
		InEntryNode->MaterialExpression->MaterialExpressionEditorX = InEntryNode->NodePosX;
		InEntryNode->MaterialExpression->MaterialExpressionEditorY = InEntryNode->NodePosY;

		InResultNode->NodePosX = MaxNodeX + PlusOffsetX;
		InResultNode->NodePosY = CenterY;
		InResultNode->SnapToGrid(SNodePanel::GetSnapGridSize());
		InResultNode->MaterialExpression->MaterialExpressionEditorX = InResultNode->NodePosX;
		InResultNode->MaterialExpression->MaterialExpressionEditorY = InResultNode->NodePosY;
	}

	AssignPinSourceIndices(InGatewayNode);
	AssignPinSourceIndices(InEntryNode);
	AssignPinSourceIndices(InResultNode);
}

void FMaterialEditor::CollapseNodes(TSet<UEdGraphNode*>& InCollapsableNodes)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd.IsValid())
	{
		return;
	}

	UEdGraph* SourceGraph = FocusedGraphEd->GetCurrentGraph();
	SourceGraph->Modify();

	// Create the composite node that will serve as the gateway into the subgraph
	UMaterialGraphNode_Composite* GatewayNode = NULL;
	{
		GatewayNode = Cast<UMaterialGraphNode_Composite>(FMaterialGraphSchemaAction_NewComposite::SpawnNode(SourceGraph, FVector2D(0, 0)));
		GatewayNode->bCanRenameNode = true;
		check(GatewayNode);
	}

	UEdGraph* DestinationGraph = GatewayNode->BoundGraph;
	UMaterialExpressionComposite* CompositeExpression = CastChecked<UMaterialExpressionComposite>(GatewayNode->MaterialExpression);

	CollapseNodesIntoGraph(GatewayNode, 
		Cast<UMaterialGraphNode>(CompositeExpression->InputExpressions->GraphNode),
		Cast<UMaterialGraphNode>(CompositeExpression->OutputExpressions->GraphNode),
		SourceGraph, 
		DestinationGraph, 
		InCollapsableNodes);
	AssignPinSourceIndices(GatewayNode);

	UpdateMaterialAfterGraphChange();

	// Now that the expressions are updated, reconstruct the nodes
	// Need to do this to prevent user from copying a freshly collapsed node not built from its expression
	GatewayNode->ReconstructNode();
	Cast<UMaterialGraphNode>(CompositeExpression->InputExpressions->GraphNode)->ReconstructNode();
	Cast<UMaterialGraphNode>(CompositeExpression->OutputExpressions->GraphNode)->ReconstructNode();
}

void FMaterialEditor::ExpandNode(UEdGraphNode* InNodeToExpand, UEdGraph* InSourceGraph, TSet<UEdGraphNode*>& OutExpandedNodes)
{
	UEdGraph* DestinationGraph = InNodeToExpand->GetGraph();
	UEdGraph* SourceGraph = InSourceGraph;
	check(SourceGraph);

	// Mark all edited objects so they will appear in the transaction record if needed.
	DestinationGraph->Modify();
	SourceGraph->Modify();
	InNodeToExpand->Modify();

	UEdGraphNode* Entry = nullptr;
	UEdGraphNode* Result = nullptr;

	const bool bIsCollapsedGraph = InNodeToExpand->IsA<UMaterialGraphNode_Composite>();

	MoveNodesToGraph(MutableView(SourceGraph->Nodes), DestinationGraph, OutExpandedNodes, &Entry, &Result, bIsCollapsedGraph);
	CollapseGatewayNode(InNodeToExpand, Entry, Result, &OutExpandedNodes);

	bool bPreviewExpressionDeleted = false;

	if (Entry)
	{
		UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(Cast<UMaterialGraphNode>(Entry)->MaterialExpression);
		PinBase->DeleteReroutePins();
		Material->GetExpressionCollection().RemoveExpression(PinBase);
		PinBase->MarkAsGarbage();
		Entry->DestroyNode();
		bPreviewExpressionDeleted |= PinBase == PreviewExpression;
	}

	if (Result)
	{
		UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(Cast<UMaterialGraphNode>(Result)->MaterialExpression);
		PinBase->DeleteReroutePins();
		Material->GetExpressionCollection().RemoveExpression(PinBase);
		PinBase->MarkAsGarbage();
		Result->DestroyNode();
		bPreviewExpressionDeleted |= PinBase == PreviewExpression;
	}

	// Make sure any subgraphs get propagated appropriately
	if (SourceGraph->SubGraphs.Num() > 0)
	{
		DestinationGraph->SubGraphs.Append(SourceGraph->SubGraphs);
		SourceGraph->SubGraphs.Empty();
	}

	// Remove the gateway node and source graph
	UMaterialExpression* CompositeExpression = Cast<UMaterialGraphNode>(InNodeToExpand)->MaterialExpression;
	CompositeExpression->Modify();
	Material->GetExpressionCollection().RemoveExpression(CompositeExpression);
	CompositeExpression->MarkAsGarbage();
	InNodeToExpand->DestroyNode();
	bPreviewExpressionDeleted |= CompositeExpression == PreviewExpression;

	if (bPreviewExpressionDeleted)
	{
		// The preview expression was deleted.  Null out our reference to it and reset to the normal preview material
		SetPreviewExpression(nullptr);
		RegenerateCodeView();
		UpdatePreviewMaterial();
	}
}

void FMaterialEditor::MoveNodesToAveragePos(TSet<UEdGraphNode*>& AverageNodes, FVector2D SourcePos, bool bExpandedNodesNeedUniqueGuid) const
{
	if (AverageNodes.Num() > 0)
	{
		FVector2D AvgNodePosition(0.0f, 0.0f);

		for (TSet<UEdGraphNode*>::TIterator It(AverageNodes); It; ++It)
		{
			UEdGraphNode* Node = *It;
			AvgNodePosition.X += Node->NodePosX;
			AvgNodePosition.Y += Node->NodePosY;
		}

		float InvNumNodes = 1.0f / float(AverageNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;

		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		for (UEdGraphNode* ExpandedNode : AverageNodes)
		{
			ExpandedNode->NodePosX = (ExpandedNode->NodePosX - AvgNodePosition.X) + SourcePos.X;
			ExpandedNode->NodePosY = (ExpandedNode->NodePosY - AvgNodePosition.Y) + SourcePos.Y;

			ExpandedNode->SnapToGrid(SNodePanel::GetSnapGridSize());

			if (bExpandedNodesNeedUniqueGuid)
			{
				ExpandedNode->CreateNewGuid();
			}

			//Add expanded node to selection
			FocusedGraphEd->SetNodeSelection(ExpandedNode, true);
		}
	}
}

void FMaterialEditor::MoveNodesToGraph(TArray<UEdGraphNode*>& SourceNodes, UEdGraph* DestinationGraph, TSet<UEdGraphNode*>& OutExpandedNodes, UEdGraphNode** OutEntry, UEdGraphNode** OutResult, const bool bIsCollapsedGraph)
{
	// Move the nodes over, remembering any that are boundary nodes
	while (SourceNodes.Num())
	{
		UEdGraphNode* Node = SourceNodes.Pop();
		UEdGraph* OriginalGraph = Node->GetGraph();

		Node->Modify();
		OriginalGraph->Modify();
		Node->Rename(/*NewName=*/ nullptr, /*NewOuter=*/ DestinationGraph, REN_DontCreateRedirectors);

		// Remove the node from the original graph
		OriginalGraph->RemoveNode(Node, false);

		// We do not check CanPasteHere when determining CanCollapseNodes, unlike CanCollapseSelectionToFunction/Macro,
		// so when expanding a collapsed graph we don't want to check the CanPasteHere function:
		if (!bIsCollapsedGraph && !Node->CanPasteHere(DestinationGraph))
		{
			Node->BreakAllNodeLinks();
			continue;
		}

		// Successfully added the node to the graph, we may need to remove flags
		if (Node->HasAllFlags(RF_Transient) && !DestinationGraph->HasAllFlags(RF_Transient))
		{
			Node->SetFlags(RF_Transactional);
			Node->ClearFlags(RF_Transient);
			TArray<UObject*> Subobjects;
			GetObjectsWithOuter(Node, Subobjects);
			for (UObject* Subobject : Subobjects)
			{
				Subobject->ClearFlags(RF_Transient);
				Subobject->SetFlags(RF_Transactional);
			}
		}

		DestinationGraph->AddNode(Node, /* bFromUI */ false, /* bSelectNewNode */ false);

		if (UMaterialGraphNode_Composite* Composite = Cast<UMaterialGraphNode_Composite>(Node))
		{
			OriginalGraph->SubGraphs.Remove(Composite->BoundGraph);
			DestinationGraph->SubGraphs.Add(Composite->BoundGraph);
		}

		UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Node);
		if (MaterialNode && MaterialNode->MaterialExpression->IsA(UMaterialExpressionPinBase::StaticClass()))
		{
			UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(MaterialNode->MaterialExpression);

			if (PinBase && PinBase->PinDirection == EGPD_Output)
			{
				*OutEntry = Node;
			}
			else if (PinBase && PinBase->PinDirection == EGPD_Input)
			{
				*OutResult = Node;
			}
		}
		else
		{
			OutExpandedNodes.Add(Node);
		}
	}
}

bool FMaterialEditor::CollapseGatewayNode(UEdGraphNode* InNode, UEdGraphNode* InEntryNode, UEdGraphNode* InResultNode, TSet<UEdGraphNode*>* OutExpandedNodes)
{
	bool bSuccessful = true;

	// We iterate the array in reverse so we can both remove the subpins safely after we've read them and
	// so we have split nested structs we combine them back together in the right order
	for (int32 BoundaryPinIndex = InNode->Pins.Num() - 1; BoundaryPinIndex >= 0; --BoundaryPinIndex)
	{
		UEdGraphPin* const BoundaryPin = InNode->Pins[BoundaryPinIndex];

		// For each pin in the gateway node, find the associated pin in the entry or result node.
		UEdGraphNode* const GatewayNode = (BoundaryPin->Direction == EGPD_Input) ? InEntryNode : InResultNode;
		UEdGraphPin* GatewayPin = nullptr;
		if (GatewayNode)
		{
			for (int32 PinIdx = GatewayNode->Pins.Num() - 1; PinIdx >= 0; --PinIdx)
			{
				UEdGraphPin* const Pin = GatewayNode->Pins[PinIdx];

				// Function graphs have a single exec path through them, so only one exec pin for input and another for output. In this fashion, they must not be handled by name.
				if ((Pin->PinName == BoundaryPin->PinName) && (Pin->Direction != BoundaryPin->Direction))
				{
					GatewayPin = Pin;
					break;
				}
			}
		}

		if (GatewayPin)
		{
			//@TODO: This is same as UEdGraphSchema_K2::CombineTwoPinNetsAndRemoveOldPins, except we don't care about default values
			// since material graphs pins can't do that. More consolidation
			auto CombineTwoPinNetsAndRemoveOldPins = [](UEdGraphPin* InPinA, UEdGraphPin* InPinB)
			{
				// Make direct connections between the things that connect to A or B, removing A and B from the picture
				for (int32 IndexA = 0; IndexA < InPinA->LinkedTo.Num(); ++IndexA)
				{
					UEdGraphPin* FarA = InPinA->LinkedTo[IndexA];
					// TODO: Michael N. says this if check should be unnecessary once the underlying issue is fixed.
					// (Probably should use a check() instead once it's removed though.  See additional cases above.
					if (FarA != nullptr)
					{
						for (int32 IndexB = 0; IndexB < InPinB->LinkedTo.Num(); ++IndexB)
						{
							UEdGraphPin* FarB = InPinB->LinkedTo[IndexB];

							if (FarB != nullptr)
							{
								FarA->Modify();
								FarB->Modify();
								FarA->MakeLinkTo(FarB);
							}

						}
					}
				}
			};
			CombineTwoPinNetsAndRemoveOldPins(BoundaryPin, GatewayPin);
		}
	}

	return bSuccessful;
}

void FMaterialEditor::SetQualityPreview(EMaterialQualityLevel::Type NewQuality)
{
	NodeQualityLevel = NewQuality;
	bPreviewFeaturesChanged = true;
}

bool FMaterialEditor::IsQualityPreviewChecked(EMaterialQualityLevel::Type TestQuality)
{
	return NodeQualityLevel == TestQuality;
}

EMaterialQualityLevel::Type FMaterialEditor::GetQualityPreviewLevel() const
{
	return NodeQualityLevel == EMaterialQualityLevel::Num ? GetCachedScalabilityCVars().MaterialQualityLevel : NodeQualityLevel;
}

void FMaterialEditor::SetFeaturePreview(ERHIFeatureLevel::Type NewFeatureLevel)
{
	NodeFeatureLevel = NewFeatureLevel;
	bPreviewFeaturesChanged = true;
}

bool FMaterialEditor::IsFeaturePreviewChecked(ERHIFeatureLevel::Type TestFeatureLevel) const
{
	return NodeFeatureLevel == TestFeatureLevel;
}

bool FMaterialEditor::IsFeaturePreviewAvailable(ERHIFeatureLevel::Type TestFeatureLevel) const
{
	return GMaxRHIFeatureLevel >= TestFeatureLevel;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FMaterialEditor::CreateInternalWidgets()
{
	PreviewViewport = SNew(SMaterialEditor3DPreviewViewport)
		.MaterialEditor(SharedThis(this));

	PreviewUIViewport = SNew(SMaterialEditorUIPreviewViewport, Material);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	MaterialDetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );

	FOnGetDetailCustomizationInstance LayoutExpressionParameterDetails = FOnGetDetailCustomizationInstance::CreateStatic(
		&FMaterialExpressionParameterDetails::MakeInstance, FOnCollectParameterGroups::CreateSP(this, &FMaterialEditor::UpdateAndGetMaterialExpressionGroups) );

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout( 
		UMaterialExpressionParameter::StaticClass(), 
		LayoutExpressionParameterDetails
		);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout( 
		UMaterialExpressionFontSampleParameter::StaticClass(), 
		LayoutExpressionParameterDetails
		);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout( 
		UMaterialExpressionTextureSampleParameter::StaticClass(), 
		LayoutExpressionParameterDetails
		);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout(
		UMaterialExpressionRuntimeVirtualTextureSampleParameter::StaticClass(),
		LayoutExpressionParameterDetails
		);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout(
		UMaterialExpressionSparseVolumeTextureSampleParameter::StaticClass(),
		LayoutExpressionParameterDetails
		);

	FOnGetDetailCustomizationInstance LayoutLayerExpressionParameterDetails = FOnGetDetailCustomizationInstance::CreateStatic(
		&FMaterialExpressionLayersParameterDetails::MakeInstance, FOnCollectParameterGroups::CreateSP(this, &FMaterialEditor::UpdateAndGetMaterialExpressionGroups));

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout(
		UMaterialExpressionMaterialAttributeLayers::StaticClass(),
		LayoutLayerExpressionParameterDetails
		);

	FOnGetDetailCustomizationInstance LayoutCollectionParameterDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialExpressionCollectionParameterDetails::MakeInstance, true);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout( 
		UMaterialExpressionCollectionParameter::StaticClass(), 
		LayoutCollectionParameterDetails
		);

	FOnGetDetailCustomizationInstance LayoutCollectionTransformDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialExpressionCollectionParameterDetails::MakeInstance, false);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout(
		UMaterialExpressionCollectionTransform::StaticClass(),
		LayoutCollectionTransformDetails
		);

	FOnGetDetailCustomizationInstance LayoutCompositeDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialExpressionCompositeDetails::MakeInstance);

	MaterialDetailsView->RegisterInstancedCustomPropertyLayout( 
		UMaterialExpressionComposite::StaticClass(), 
		LayoutCompositeDetails
		);

	MaterialDetailsView->OnFinishedChangingProperties().AddSP(this, &FMaterialEditor::OnFinishedChangingProperties);

	PropertyEditorModule.RegisterCustomClassLayout( UMaterial::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialDetailCustomization::MakeInstance ) );
	PropertyEditorModule.RegisterCustomClassLayout( UMaterialFunction::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialFunctionDetailCustomization::MakeInstance) );

	MaterialEditorInstance = NewObject<UMaterialEditorPreviewParameters>(GetTransientPackage(), NAME_None, RF_Transactional);
	MaterialEditorInstance->PreviewMaterial = Material;
	MaterialEditorInstance->OriginalMaterial = OriginalMaterial;
	if (MaterialFunction)
	{
		MaterialEditorInstance->OriginalFunction = MaterialFunction->ParentFunction;
	}
	FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	if (!Generator.IsValid())
	{
		FPropertyRowGeneratorArgs Args;
		Generator = Module.CreatePropertyRowGenerator(Args);
	}

	MaterialParametersOverviewWidget = SNew(SMaterialParametersOverviewPanel)
		.InMaterialEditorInstance(MaterialEditorInstance)
		.InGenerator(Generator)
		.SelectMaterialNode(this, &FMaterialEditor::JumpToNodeByGuid);

	Generator->OnFinishedChangingProperties().AddSP(this, &FMaterialEditor::OnFinishedChangingParametersFromOverview);
	Generator->OnRowsRefreshed().AddSP(this, &FMaterialEditor::GeneratorRowsRefreshed);
	MaterialCustomPrimitiveDataWidget = SNew(SMaterialCustomPrimitiveDataPanel, this, MaterialEditorInstance);

	MaterialExpressionsOverviewWidget = SNew(SMaterialExpressionsOverviewPanel)
		.InMaterialEditorInstance(MaterialEditorInstance)
		.InGenerator(Generator)
		.InMaterialDetailsView(MaterialDetailsView);

	MaterialLayersFunctionsInstance = SNew(SMaterialLayersFunctionsMaterialWrapper)
		.InMaterialEditorInstance(MaterialEditorInstance)
		.InGenerator(Generator);

	Palette = SNew(SMaterialPalette, SharedThis(this));
	NamedReroutesPanel = SNew(SMaterialNamedReroutesPanel, SharedThis(this));

	if (Material && Material->MaterialGraph)
	{
		OnGraphChangedDelegateHandle = Material->MaterialGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &FMaterialEditor::OnGraphChanged));
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogOptions;
	// Show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false; //TODO - Provide custom filters? E.g. "Critical Errors" vs "Errors" needed for materials?
	LogOptions.bAllowClear = false;
	LogOptions.MaxPageCount = 1;
	StatsListing = MessageLogModule.CreateLogListing( "MaterialEditorStats", LogOptions );

	Stats = MessageLogModule.CreateLogListingWidget( StatsListing.ToSharedRef() );

	FindResults =
		SNew(SFindInMaterial, SharedThis(this));

	SubstrateWidget = SNew(SMaterialEditorSubstrateWidget, SharedThis(this));

	RegenerateCodeView();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FMaterialEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// If this is a comment only change, skip updating the preview material and viewport
	{
		bool bIsCommentOnlyChange = true;
		for (int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); ++Index)
		{
			const UObject* EditedObject = PropertyChangedEvent.GetObjectBeingEdited(Index);
			if (!EditedObject->IsA<UMaterialExpression>())
			{
				bIsCommentOnlyChange = false;
				break;
			}

			if (EditedObject->IsA<UMaterialExpressionComment>())
			{
				continue;
			}

			if (PropertyChangedEvent.GetPropertyName() != TEXT("Desc"))
			{
				bIsCommentOnlyChange = false;
				break;
			}
		}
		if (bIsCommentOnlyChange)
		{
			return;
		}
	}

	bool bRefreshNodePreviews = false;
	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		FStructProperty* Property = CastField<FStructProperty>(PropertyChangedEvent.Property);
		if (Property != nullptr)
		{
			if (Property->Struct->GetFName() == TEXT("LinearColor") || Property->Struct->GetFName() == TEXT("Color")) // if we changed a color property refresh the previews
			{
				bRefreshNodePreviews = true;
			}
		}
		// If we changed any of the parameter values from the Parameter Defaults panel
		if (PropertyChangedEvent.Property->GetName() == TEXT("Parameters"))
		{
			bRefreshNodePreviews = true;
		}
		if (bRefreshNodePreviews)
		{
			RefreshExpressionPreviews(true);
		}

		RefreshPreviewViewport();
		UpdatePreviewMaterial();
		SetEditorInstanceForWidgets();
	}

	// Refresh the Named Reroutes panel when a declaration's name or color changes
	if (NamedReroutesPanel.IsValid())
	{
		for (int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); ++Index)
		{
			if (PropertyChangedEvent.GetObjectBeingEdited(Index)->IsA<UMaterialExpressionNamedRerouteDeclaration>())
			{
				NamedReroutesPanel->RequestRefresh();
				break;
			}
		}
	}
}

void FMaterialEditor::OnGraphChanged(const FEdGraphEditAction& Action)
{
	// This will handle adding nodes, removing nodes, and duplicating nodes.
	if (NamedReroutesPanel.IsValid())
	{
		NamedReroutesPanel->RequestRefresh();
	}
}

void FMaterialEditor::OnFinishedChangingParametersFromOverview(const FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bRefreshNodePreviews = false;
	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		RefreshExpressionPreviews(true);
		RefreshPreviewViewport();
		TArray<TWeakObjectPtr<UObject>> SelectedObjects = MaterialDetailsView->GetSelectedObjects();

		// Don't set directly on color structs since the color picker will do that for us
		FStructProperty* StructProperty = CastField<FStructProperty>(PropertyChangedEvent.Property);
		if (!StructProperty || StructProperty->Struct->GetFName() != "LinearColor")
		{
			MaterialDetailsView->SetObjects(SelectedObjects, true);
		}

		Material->MarkPackageDirty();
		SetMaterialDirty();
	}
}

void FMaterialEditor::OnChangeBreadCrumbGraph(UEdGraph* InGraph)
{
	if (InGraph && FocusedGraphEdPtr.IsValid())
	{
		OpenDocument(InGraph, FDocumentTracker::NavigatingCurrentDocument);
	}
}

void FMaterialEditor::GeneratorRowsRefreshed()
{
	MaterialParametersOverviewWidget->Refresh();
	if (MaterialLayersFunctionsInstance)
	{
		MaterialLayersFunctionsInstance->Refresh();
	}
}

FName FMaterialEditor::GetToolkitFName() const
{
	return FName("MaterialEditor");
}

FText FMaterialEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Material Editor");
}

FText FMaterialEditor::GetToolkitName() const
{
	const UObject* EditingObject = GetEditingObjects()[0];
	check(EditingObject);

	return FText::FromString(EditingObject->GetName());
}

FText FMaterialEditor::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetEditingObjects()[0];

	// Overridden to accommodate editing of multiple objects (original and preview materials)
	return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
}

FString FMaterialEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Material ").ToString();
}

FLinearColor FMaterialEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.3f, 0.2f, 0.5f, 0.5f );
}

void FMaterialEditor::Tick( float InDeltaTime )
{
	UpdateMaterialPlatformStats();
	UpdateMaterialInfoLog();
	UpdateGraphNodeStates();
}

TStatId FMaterialEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FMaterialEditor, STATGROUP_Tickables);
}

void FMaterialEditor::UpdateThumbnailInfoPreviewMesh(UMaterialInterface* MatInterface)
{
	if ( MatInterface )
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass( MatInterface->GetClass() );
		if ( AssetTypeActions.IsValid() )
		{
			USceneThumbnailInfoWithPrimitive* OriginalThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(AssetTypeActions.Pin()->GetThumbnailInfo(MatInterface));
			if ( OriginalThumbnailInfo )
			{
				OriginalThumbnailInfo->PreviewMesh = MatInterface->PreviewMesh;
				MatInterface->PostEditChange();
			}
		}
	}
}

void FMaterialEditor::ExtendToolbar()
{
	AddToolbarExtender(GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
	AddToolbarExtender(MaterialEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FMaterialEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UMaterialEditorMenuContext* Context = NewObject<UMaterialEditorMenuContext>();
	Context->MaterialEditor = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FMaterialEditor::RegisterToolBar()
{
	const FName MenuName = FAssetEditorToolkit::GetToolMenuToolbarName();
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(MenuName, "AssetEditor.DefaultToolBar", EMultiBoxType::ToolBar);

		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
		FToolMenuSection& MaterialSection = ToolBar->AddSection("MaterialTools", TAttribute<FText>(), InsertAfterAssetSection);
		MaterialSection.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().Apply));
		MaterialSection.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().FindInMaterial));
		MaterialSection.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().CameraHome));
		UMaterialEditorMenuContext* Context = ToolBar->FindContext<UMaterialEditorMenuContext>();
		if (!MaterialFunction)
		{
			MaterialSection.AddEntry(FToolMenuEntry::InitComboButton(
				"Hierarchy",
				FToolUIActionChoice(),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
					{
						UMaterialEditorMenuContext* SubMenuContext = InSubMenu->FindContext<UMaterialEditorMenuContext>();
						if (SubMenuContext && SubMenuContext->MaterialEditor.IsValid())
						{
							SubMenuContext->MaterialEditor.Pin()->GenerateInheritanceMenu(InSubMenu);
						}
					}),
				LOCTEXT("Hierarchy", "Hierarchy"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MaterialEditor.Hierarchy"),
				false
				));
		}
		FToolMenuEntry LiveUpdateMenu = FToolMenuEntry::InitComboButton(
			"LiveUpdate",
			FUIAction(),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
				{
					FToolMenuSection& Section = InSubMenu->FindOrAddSection("LiveUpdateOptions");
					Section.AddEntry(FToolMenuEntry::InitMenuEntry(FMaterialEditorCommands::Get().ToggleLivePreview));
					Section.AddEntry(FToolMenuEntry::InitMenuEntry(FMaterialEditorCommands::Get().ToggleRealtimeExpressions));
					Section.AddEntry(FToolMenuEntry::InitMenuEntry(FMaterialEditorCommands::Get().AlwaysRefreshAllPreviews));
				}),
			LOCTEXT("LiveUpdate_Label", "Live Update"),
					LOCTEXT("LiveUpdate_Tooltip", "Set the elements of the Material Editor UI to update in realtime"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MaterialEditor.LiveUpdate")
					);
		LiveUpdateMenu.StyleNameOverride = "CalloutToolbar";
		MaterialSection.AddEntry(LiveUpdateMenu);

		
		FToolMenuSection& GraphSection = ToolBar->AddSection("Graph", TAttribute<FText>(), InsertAfterAssetSection);

		FToolMenuEntry CleaningMenu = FToolMenuEntry::InitComboButton(
			"CleanGraph",
			FUIAction(),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
				{
					FToolMenuSection& Section = InSubMenu->FindOrAddSection("General");
					Section.AddEntry(FToolMenuEntry::InitMenuEntry(FMaterialEditorCommands::Get().CleanUnusedExpressions));
					Section.AddEntry(FToolMenuEntry::InitMenuEntry(FMaterialEditorCommands::Get().ShowHideConnectors));
					Section.AddEntry(FToolMenuEntry::InitMenuEntry(FMaterialEditorCommands::Get().FixupMismatchedSamplers));
				}),
			LOCTEXT("GraphCleanup_Label", "Clean Graph"),
			LOCTEXT("GraphCleanup_Tooltip", "Tools to help clean up graph nodes and connections"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GraphEditor.Clean")
		);
		CleaningMenu.StyleNameOverride = "CalloutToolbar";
		GraphSection.AddEntry(CleaningMenu);
		GraphSection.AddEntry(FToolMenuEntry::InitComboButton(
			"NodePreview",
			FUIAction(),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
				{
					UMaterialEditorMenuContext* SubMenuContext = InSubMenu->FindContext<UMaterialEditorMenuContext>();
					if (SubMenuContext && SubMenuContext->MaterialEditor.IsValid())
					{
						TSharedPtr<FMaterialEditor> MaterialEditor = StaticCastSharedPtr<FMaterialEditor>(SubMenuContext->MaterialEditor.Pin());
						MaterialEditor->GeneratePreviewMenuContent(InSubMenu);
					}
				}),
			LOCTEXT("NodePreview_Label", "Preview State"),
					LOCTEXT("NodePreviewToolTip", "Preview the graph state for a given feature level, material quality, or static switch value."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "FullBlueprintEditor.SwitchToScriptingMode"),
					false
					));
		GraphSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			FMaterialEditorCommands::Get().ToggleHideUnrelatedNodes,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.ToggleHideUnrelatedNodes")
		));
		GraphSection.AddEntry(FToolMenuEntry::InitComboButton(
			"HideUnrelatedNodesOptions",
			FUIAction(),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
				{
					UMaterialEditorMenuContext* SubMenuContext = InSubMenu->FindContext<UMaterialEditorMenuContext>();
					if (SubMenuContext && SubMenuContext->MaterialEditor.IsValid())
					{
						TSharedPtr<FMaterialEditor> MaterialEditor = StaticCastSharedPtr<FMaterialEditor>(SubMenuContext->MaterialEditor.Pin());
						MaterialEditor->MakeHideUnrelatedNodesOptionsMenu(InSubMenu);
					}
				}),
			LOCTEXT("HideUnrelatedNodesOptions", "Hide Unrelated Nodes Options"),
					LOCTEXT("HideUnrelatedNodesOptionsMenu", "Hide Unrelated Nodes options menu"),
					TAttribute<FSlateIcon>(),
					true
					));

		{
			FToolMenuSection& Section = ToolBar->AddSection("Stats", TAttribute<FText>(), InsertAfterAssetSection);
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().ToggleMaterialStats));
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().TogglePlatformStats));
		}

		SMaterialRevisionMenu::MakeDiffMenu(ToolBar);
	}
}

void FMaterialEditor::AddInheritanceMenuEntry(FToolMenuSection& Section, const FAssetData& AssetData, bool bIsFunctionPreviewMaterial)
{
	FExecuteAction OpenAction;
	if (bIsFunctionPreviewMaterial)
	{
		OpenAction.BindStatic(&FMaterialEditorUtilities::OnOpenFunction, AssetData);
	}
	else
	{
		OpenAction.BindStatic(&FMaterialEditorUtilities::OnOpenMaterial, AssetData);
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("ParentName"), FText::FromName(AssetData.AssetName));
	FText Label = FText::Format(LOCTEXT("InstanceParentName", "{ParentName}"), Args);

	Section.AddEntry(FToolMenuEntry::InitMenuEntry(
		NAME_None,
		Label,
		LOCTEXT("OpenInEditor", "Open In Editor"),
		FSlateIcon(),
		FUIAction(OpenAction)
	));
}

void FMaterialEditor::GenerateInheritanceMenu(UToolMenu* Menu)
{
	RebuildInheritanceList();
	Menu->bShouldCloseWindowAfterMenuSelection = true;
	Menu->SetMaxHeight(500);

	if (!MaterialFunction)
	{
		FToolMenuSection& Section = Menu->AddSection("MaterialInstances", LOCTEXT("MaterialInstances", "Material Instances"));
		if (MaterialChildList.Num() == 0)
		{
			const FText NoChildText = LOCTEXT("NoInstancesFound", "No Instances Found");
			TSharedRef<SWidget> NoChildWidget = SNew(STextBlock)
				.Text(NoChildText);
			Section.AddEntry(FToolMenuEntry::InitWidget("NoInstancesFound", NoChildWidget, FText::GetEmpty()));
		}
		for (const FAssetData& MaterialChild : MaterialChildList)
		{
			FMaterialEditor::AddInheritanceMenuEntry(Section, MaterialChild, false);
		}
	}
}

void FMaterialEditor::RefreshStatsMaterials()
{
	// unconditionally recreate as settings might have changed
	CreateDerivedMaterialInstancesPreviews();

	MaterialStatsManager->SetMaterial(bStatsFromPreviewMaterial ? Material : OriginalMaterial, bStatsFromPreviewMaterial ? DerivedMaterialInstances : OriginalDerivedMaterialInstances);
	MaterialStatsManager->SignalMaterialChanged();
}

void FMaterialEditor::GeneratePreviewMenuContent(UToolMenu* Menu)
{
	Menu->bShouldCloseWindowAfterMenuSelection = true;

	FToolMenuSection& QualityLevelSection = Menu->AddSection("MaterialEditorQualityPreview", LOCTEXT("MaterialQualityHeading", "Quality Level"));
	{
		QualityLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().QualityLevel_All);
		QualityLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().QualityLevel_Epic);
		QualityLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().QualityLevel_High);
		QualityLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().QualityLevel_Medium);
		QualityLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().QualityLevel_Low);
	}

	FToolMenuSection& FeatureLevelSection = Menu->AddSection("MaterialEditorFeaturePreview", LOCTEXT("MaterialFeatureHeading", "Feature Level"));
	{
		FeatureLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().FeatureLevel_All);
		FeatureLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().FeatureLevel_Mobile);
		FeatureLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().FeatureLevel_SM5);
		FeatureLevelSection.AddMenuEntry(FMaterialEditorCommands::Get().FeatureLevel_SM6);
	}

	FToolMenuSection& StaticSwitchSection = Menu->AddSection("StaticSwitchPreview", LOCTEXT("StaticSwitchHeading", "Switch Params"));
	TSharedRef<SWidget> StaticSwitchCheckbox = SNew(SBox)
	[
		SNew(SCheckBox)
			.IsChecked(bPreviewStaticSwitches ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged_Lambda(
				[this](ECheckBoxState NewState) 
				{
					bPreviewStaticSwitches = (NewState == ECheckBoxState::Checked);
					bPreviewFeaturesChanged = true;
				}
			)
			.Style(FAppStyle::Get(), "Menu.CheckBox")
			.ToolTipText(LOCTEXT("StaticSwitchCheckBoxToolTip", "Hide disabled nodes in the graph, according to switch params."))
			.Content()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("DisableInactiveSwitch", "Hide Disabled"))
				]
			]
	];
	StaticSwitchSection.AddEntry(FToolMenuEntry::InitMenuEntry("StaticSwitchCheckbox", FUIAction(), StaticSwitchCheckbox));
}

UMaterialInterface* FMaterialEditor::GetMaterialInterface() const
{
	return Material;
}

bool FMaterialEditor::ApproveSetPreviewAsset(UObject* InAsset)
{
	bool bApproved = true;

	// Only permit the use of a skeletal mesh if the material has SkeletalMesh usage flag.
	if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InAsset))
	{
		if (!Material->CheckMaterialUsage(MATUSAGE_SkeletalMesh))
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_MaterialEditor_CantPreviewOnSkelMesh", "Can't preview on the specified skeletal mesh because the material has not been compiled with SkeletalMesh usage flag."));
			bApproved = false;
		}
	}

	return bApproved;
}

void FMaterialEditor::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	if ((MaterialFunction != nullptr) && MaterialFunction->ParentFunction)
	{
		OutObjects.Add(MaterialFunction->ParentFunction);
	}
	else
	{
		OutObjects.Add(OriginalMaterial);
	}
}

void FMaterialEditor::SaveAsset_Execute()
{
	UE_LOGF(LogMaterialEditor, Log, "Saving and Compiling material %ls", *GetEditingObjects()[0]->GetName());

	// When Substrate is enabled, warns user about backward compatibility issue when a material is auto-converted 
	// and is about to be saved
	const bool bSaveSubstrateAutoConversion = OriginalMaterial && Substrate::IsSubstrateEnabled() && IsSubstrateAutoConvertedAndNotSaved(OriginalMaterial);
	if (bSaveSubstrateAutoConversion)
	{
		// Find out the user wants to do with this dirty material
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
			FText::Format(
			NSLOCTEXT("UnrealEd", "Prompt_MaterialEditorSaveAsset", "This material has been automatically converted to be compatible with Substrate.\nSaving this material will make is no longer works when Substrate is disabled.\nWould you like to apply the changes of the modified material to the original material?\n\n{0}\n\n(Selecting 'No' will cause all changes to be lost!)"),
			FText::FromString(OriginalMaterialObject->GetPathName())));

		// Skip saving the asset, based on user's choice
		if (YesNoCancelReply != EAppReturnType::Yes)
		{
			return;
		}
	}

	const bool bUpdateSucceeded = (bMaterialDirty ? UpdateOriginalMaterial() : true);

	if (bUpdateSucceeded)
	{
		if (bSaveSubstrateAutoConversion)
		{
			MarkSubstrateAutoConversionSaved(OriginalMaterial);
		}

		IMaterialEditor::SaveAsset_Execute();
	}
}

void FMaterialEditor::SaveAssetAs_Execute()
{
	UE_LOGF(LogMaterialEditor, Log, "Saving and Compiling material %ls", *GetEditingObjects()[0]->GetName());

	const bool bUpdateSucceeded = (bMaterialDirty ? UpdateOriginalMaterial() : true);

	if (bUpdateSucceeded)
	{
		IMaterialEditor::SaveAssetAs_Execute();
	}
}

bool FMaterialEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	DestroyColorPicker();

	// If the asset has been deleted, we don't want to show the save changes prompt
	if(InCloseReason == EAssetEditorCloseReason::AssetForceDeleted)
	{
		bMaterialDirty = false;
	}
	
	if (bMaterialDirty)
	{
		// find out the user wants to do with this dirty material
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
			FText::Format(
				NSLOCTEXT("UnrealEd", "Prompt_MaterialEditorClose", "Would you like to apply the changes of the modified material to the original material?\n{0}\n(Selecting 'No' will cause all changes to be lost!)"),
				FText::FromString(OriginalMaterialObject->GetPathName()) ));

		// act on it
		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			// update material and exit
			UpdateOriginalMaterial();
			break;
				
		case EAppReturnType::No:
			// exit
			bMaterialDirty = false;
			break;
				
		case EAppReturnType::Cancel:
			// don't exit
			return false;
		}
	}

	return true;
}

void FMaterialEditor::AddGraphEditorPinActionsToContextMenu(FToolMenuSection& InSection) const
{
	// Promote To Parameter
	{
		FToolUIAction PromoteToParameterAction;
		PromoteToParameterAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &FMaterialEditor::OnPromoteToParameter);
		PromoteToParameterAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &FMaterialEditor::OnCanPromoteToParameter);

		TSharedPtr<FUICommandInfo> PromoteToParameterCommand = FMaterialEditorCommands::Get().PromoteToParameter;
		InSection.AddMenuEntry(
			PromoteToParameterCommand->GetCommandName(),
			PromoteToParameterCommand->GetLabel(),
			PromoteToParameterCommand->GetDescription(),
			PromoteToParameterCommand->GetIcon(),
			PromoteToParameterAction
		);
	}

	// Reset to Default Value
	{
		FToolUIAction ResetToDefaultAction;
		ResetToDefaultAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &FMaterialEditor::OnResetToDefault);
		ResetToDefaultAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &FMaterialEditor::OnCanResetToDefault);

		TSharedPtr<FUICommandInfo> ResetToDefaultCommand = FMaterialEditorCommands::Get().ResetToDefault;
		InSection.AddMenuEntry(
			ResetToDefaultCommand->GetCommandName(),
			ResetToDefaultCommand->GetLabel(),
			ResetToDefaultCommand->GetDescription(),
			ResetToDefaultCommand->GetIcon(),
			ResetToDefaultAction
		);
	}

	// Delete Pin
	{
		FToolUIAction DeletePinAction;
		DeletePinAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &FMaterialEditor::OnDeletePin);
		DeletePinAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &FMaterialEditor::OnCanDeletePin);

		TSharedPtr<FUICommandInfo> DeletePinCommand = FMaterialEditorCommands::Get().DeletePin;
		InSection.AddMenuEntry(
			DeletePinCommand->GetCommandName(),
			DeletePinCommand->GetLabel(),
			DeletePinCommand->GetDescription(),
			DeletePinCommand->GetIcon(),
			DeletePinAction
		);
	}

	// Substrate Pin Actions
	{
		auto AddSubstrateContextualMenu = [&](TSharedPtr<FUICommandInfo> CreateNodeCommand, ESubstrateNodeForPin SubstrateNodeForPin)
		{
			FToolUIAction CreateNodeAction;
			CreateNodeAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &FMaterialEditor::OnCreateSubstrateNodeForPin, SubstrateNodeForPin);
			CreateNodeAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &FMaterialEditor::OnCanCreateSubstrateNodeForPin, SubstrateNodeForPin);

			InSection.AddMenuEntry(
				CreateNodeCommand->GetCommandName(),
				CreateNodeCommand->GetLabel(),
				CreateNodeCommand->GetDescription(),
				CreateNodeCommand->GetIcon(),
				CreateNodeAction
			);
		};
		AddSubstrateContextualMenu(FMaterialEditorCommands::Get().CreateSlabNode, ESubstrateNodeForPin::Slab);
		AddSubstrateContextualMenu(FMaterialEditorCommands::Get().CreateHorizontalMixNode, ESubstrateNodeForPin::HorizontalMix);
		AddSubstrateContextualMenu(FMaterialEditorCommands::Get().CreateVerticalLayerNode, ESubstrateNodeForPin::VerticalLayer);
		AddSubstrateContextualMenu(FMaterialEditorCommands::Get().CreateWeightNode, ESubstrateNodeForPin::Weight);
		AddSubstrateContextualMenu(FMaterialEditorCommands::Get().CreateSelectNode, ESubstrateNodeForPin::Select);
	}
}

bool FMaterialEditor::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectContext : TransactionObjectContexts)
	{
		UObject* Object = TransactionObjectContext.Get<0>();

		// Evaluate whether any object we are interested in matches an object part of the transaction.
		bool bIsMaterialRelatedObject =
			   Object->IsA<UMaterialInterface>()
			|| Object->IsA<UMaterialEditorOnlyData>()
			|| Object->IsA<UMaterialExpression>()
			|| Object->IsA<UMaterialGraph>()
			|| Object->IsA<UMaterialFunctionInterface>()
			|| Object->IsA<UMaterialEditingLibrary>()
			|| Object->IsA<UMaterialEditorSettings>()
			|| Object->IsA<UMaterialEditorInstanceConstant>()
			|| Object->IsA<UMaterialEditorMenuContext>()
			|| Object->IsA<UMaterialEditorOptions>()
			|| Object->IsA<UMaterialParameterCollection>()
			|| Object->IsA<UMaterialParameterCollectionInstance>();

		if (bIsMaterialRelatedObject)
		{
			return true;
		}
	}

	return false;
}

void FMaterialEditor::DrawMessages( FViewport* InViewport, FCanvas* Canvas )
{
	if( PreviewExpression != NULL )
	{
		Canvas->PushAbsoluteTransform( FTranslationMatrix(FVector::ZeroVector ) );

		// The message to display in the viewport.
		FString Name = FString::Printf( TEXT("Previewing: %s"), *PreviewExpression->GetName() );

		// Size of the tile we are about to draw.  Should extend the length of the view in X.
		const FIntPoint TileSize( InViewport->GetSizeXY().X / Canvas->GetDPIScale(), 25);

		const FColor PreviewColor( 70,100,200 );
		const FColor FontColor( 255,255,128 );

		UFont* FontToUse = GEditor->EditorFont;

		Canvas->DrawTile(  0.0f, 0.0f, TileSize.X, TileSize.Y, 0.0f, 0.0f, 0.0f, 0.0f, PreviewColor );

		int32 XL, YL;
		StringSize( FontToUse, XL, YL, *Name );
		if( XL > TileSize.X )
		{
			// There isn't enough room to show the preview expression name
			Name = TEXT("Previewing");
			StringSize( FontToUse, XL, YL, *Name );
		}

		// Center the string in the middle of the tile.
		const FIntPoint StringPos( (TileSize.X-XL)/2, ((TileSize.Y-YL)/2)+1 );
		// Draw the preview message
		Canvas->DrawShadowedString(  StringPos.X, StringPos.Y, *Name, FontToUse, FontColor );

		Canvas->PopTransform();
	}
}

void FMaterialEditor::RecenterEditor()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	UEdGraphNode* FocusNode = NULL;

	if (MaterialFunction)
	{
		bool bSetPreviewExpression = false;
		UMaterialExpressionFunctionOutput* FirstOutput = NULL;
		for (int32 ExpressionIndex = Material->GetExpressions().Num() - 1; ExpressionIndex >= 0; ExpressionIndex--)
		{
			UMaterialExpression* Expression = Material->GetExpressions()[ExpressionIndex];

			UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);
			if (FunctionOutput)
			{
				FirstOutput = FunctionOutput;
				if (FunctionOutput->bLastPreviewed)
				{
					bSetPreviewExpression = true;
					FocusNode = FunctionOutput->GraphNode;
				}
			}
		}

		if (!bSetPreviewExpression && FirstOutput)
		{
			FocusNode = FirstOutput->GraphNode;
		}
	}
	else
	{
		FocusNode = Material->MaterialGraph->RootNode;
	}

	if (FocusNode)
	{
		JumpToNode(FocusNode);
	}
	else
	{
		// Get current view location so that we don't change the zoom amount
		FVector2f CurrLocation;
		float CurrZoomLevel;
		FocusedGraphEd->GetViewLocation(CurrLocation, CurrZoomLevel);
		FocusedGraphEd->SetViewLocation(FVector2f::ZeroVector, CurrZoomLevel);
	}
}

bool FMaterialEditor::SetPreviewAsset(UObject* InAsset)
{
	if (PreviewViewport.IsValid())
	{
		return PreviewViewport->SetPreviewAsset(InAsset);
	}
	return false;
}

bool FMaterialEditor::SetPreviewAssetByName(const TCHAR* InAssetName)
{
	if (PreviewViewport.IsValid())
	{
		return PreviewViewport->SetPreviewAssetByName(InAssetName);
	}
	return false;
}

void FMaterialEditor::SetPreviewMaterial(UMaterialInterface* InMaterialInterface)
{
	if (Material->IsUIMaterial())
	{
		if (PreviewUIViewport.IsValid())
		{
			PreviewUIViewport->SetPreviewMaterial(InMaterialInterface);
		}

		if (PreviewViewport.IsValid())
		{
			PreviewViewport->SetPreviewMaterial(nullptr);
		}
	}
	else
	{
		if (PreviewUIViewport.IsValid())
		{
			PreviewUIViewport->SetPreviewMaterial(nullptr);
		}

		if (PreviewViewport.IsValid())
		{
			PreviewViewport->SetPreviewMaterial(InMaterialInterface);
		}
	}
}

void FMaterialEditor::RefreshPreviewViewport()
{
	if (PreviewViewport.IsValid())
	{
		UpdatePreviewCustomPrimitiveData();
		PreviewViewport->RefreshViewport();
	}
}

void FMaterialEditor::UpdatePreviewCustomPrimitiveData()
{
	if (!PreviewViewport.IsValid() || !PreviewViewport->PreviewMeshComponent || !Material)
	{
		return;
	}

	UPrimitiveComponent* PreviewComponent = PreviewViewport->PreviewMeshComponent;

	PreviewComponent->ResetCustomPrimitiveData();
	PreviewComponent->SetCustomPrimitiveDataFloatArray(0, CustomPrimitiveDataFloats);
	PreviewComponent->MarkRenderStateDirty();
}

void FMaterialEditor::LoadEditorSettings()
{
	EditorOptions = UMaterialEditorOptions::Get();
	
	if (EditorOptions->bHideUnusedConnectorsSetting) {OnHideConnectors();}
	if (bLivePreview != EditorOptions->bLivePreviewUpdate)
	{
		ToggleLivePreview();
	}
	if (EditorOptions->bAlwaysRefreshAllPreviews) {OnAlwaysRefreshAllPreviews();}
	if (EditorOptions->bRealtimeExpressionViewport) {ToggleRealTimeExpressions();}

	if ( PreviewViewport.IsValid() )
	{
		if (EditorOptions->bRealtimeMaterialViewport && PreviewViewport->GetViewportClient())
		{
			PreviewViewport->GetViewportClient()->SetRealtime(true);
		}
	}
	
	if (EditorOptions->bHideUnrelatedNodes)
	{
		ToggleHideUnrelatedNodes();
	}

	// Set the preview mesh for the material.  This call must occur after the toolbar is initialized.
	if (!SetPreviewAssetByName(*Material->PreviewMesh.ToString()))
	{
		// The material preview mesh couldn't be found or isn't loaded. Default to the saved editor preview mesh.
		SetPreviewAsset(EditorOptions->GetPreviewMesh());
	}
}

void FMaterialEditor::SaveEditorSettings()
{
	// Save the preview scene
	check( PreviewViewport.IsValid() );

	if ( EditorOptions )
	{
		EditorOptions->bRealtimeMaterialViewport	= PreviewViewport->IsRealtime();
		EditorOptions->bHideUnusedConnectorsSetting	= IsOnHideConnectorsChecked();
		EditorOptions->bAlwaysRefreshAllPreviews	= IsOnAlwaysRefreshAllPreviews();
		EditorOptions->bRealtimeExpressionViewport	= IsToggleRealTimeExpressionsChecked();
		EditorOptions->bLivePreviewUpdate           = IsToggleLivePreviewChecked();
		EditorOptions->bHideUnrelatedNodes          = bHideUnrelatedNodes;
		EditorOptions->SaveConfig();
	}
}

void FMaterialEditor::RegenerateCodeView(bool bForce)
{
	if (!bLivePreview && !bForce)
	{
		//When bLivePreview is false then the source can be out of date. 
		return;
	}

	MaterialStatsManager->SignalMaterialChanged();
}

void FMaterialEditor::UpdatePreviewMaterial( bool bForce )
{
	if (!bLivePreview && !bForce)
	{
		//Don't update the preview material
		return;
	}

	bStatsFromPreviewMaterial = true;

	if (ExpressionPreviewMaterial)
	{
		ExpressionPreviewMaterial->UpdateCachedExpressionData();
		if (PreviewExpression)
		{
			PreviewExpression->ConnectToPreviewMaterial(ExpressionPreviewMaterial, 0);
		}
	}

	if(PreviewExpression)
	{
		check(ExpressionPreviewMaterial);

		// The preview material's expressions array must stay up to date before recompiling 
		// So that RebuildMaterialFunctionInfo will see all the nested material functions that may need to be updated
		ExpressionPreviewMaterial->AssignExpressionCollection(Material->GetExpressionCollection());
		ExpressionPreviewMaterial->bEnableNewHLSLGenerator = Material->IsUsingNewHLSLGenerator();

		if (MaterialFunction)
		{
			ExpressionPreviewMaterial->BlendMode = MaterialFunction->PreviewBlendMode;
		}

		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::SyncWithRenderingThread);
		UpdateContext.AddMaterial(ExpressionPreviewMaterial);

		// If we are previewing an expression, update the expression preview material
		ExpressionPreviewMaterial->PreEditChange( NULL );
		ExpressionPreviewMaterial->PostEditChange();
	}
	
	{
		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::SyncWithRenderingThread);
		UpdateContext.AddMaterial(Material);

		// Update the regular preview material even when previewing an expression to allow code view regeneration.
		Material->PreEditChange(NULL);
		Material->PostEditChange();
	}

	Material->MaterialGraph->UpdatePinTypes();

	if (!PreviewExpression)
	{
		UpdateStatsMaterials();

		// Null out the expression preview material so they can be GC'ed
		ExpressionPreviewMaterial = NULL;
	}

	if (DerivedMaterialInstances.IsEmpty())
	{
		CreateDerivedMaterialInstancesPreviews();
	}

	if (MaterialFunction && ExpressionPreviewMaterial)
	{
		MaterialStatsManager->SetMaterial(ExpressionPreviewMaterial, bStatsFromPreviewMaterial ? DerivedMaterialInstances : OriginalDerivedMaterialInstances);
	}
	else
	{
		MaterialStatsManager->SetMaterial(bStatsFromPreviewMaterial ? Material : OriginalMaterial, bStatsFromPreviewMaterial ? DerivedMaterialInstances : OriginalDerivedMaterialInstances);
	}
	MaterialStatsManager->SignalMaterialChanged();

	// Reregister all components that use the preview material, since UMaterial::PEC does not reregister components using a bIsPreviewMaterial=true material
	RefreshPreviewViewport();
}

bool FMaterialEditor::UpdateOriginalMaterial()
{
	if (MaterialStatsManager->GetProvideDerivedMIFlag())
	{
		MaterialStatsManager->CacheAndCompilePendingShaders();
	}

	TArray<FText> Errors;
	bool bBaseMaterialFailsToCompile = false;

	EShaderPlatform EditorPreviewPlatform = GEditor->PreviewPlatform.GetShaderPlatform();
	EditorPreviewPlatform = EditorPreviewPlatform == GMaxRHIShaderPlatform ? EShaderPlatform::SP_NumPlatforms : EditorPreviewPlatform;
	EShaderPlatform ShaderPlatformsToTest[] = {GMaxRHIShaderPlatform, EditorPreviewPlatform};
	// If the Material has compilation errors, warn the user

	for (EShaderPlatform ShaderPlatform : ShaderPlatformsToTest)
	{
		if (!FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform))
		{
			continue;
		}

		FMaterialResource* CurrentResource = Material->GetMaterialResource(ShaderPlatform);
		if(CurrentResource && CurrentResource->GetCompileErrors().Num() > 0 )
		{
			Errors.Push(FText::Format(NSLOCTEXT("UnrealEd", "Warning_CompileErrorsInMaterial_ListEntryFeatureLevel", "- At Shader Platform {0}."), FText::FromString(*FDataDrivenShaderPlatformInfo::GetName(ShaderPlatform).ToString())));
			bBaseMaterialFailsToCompile = true;
		}
	}

	// If derived material instances have compilation errors, warn the user
	const auto& PlatformList = MaterialStatsManager->GetPlatformsDB();
	for (const auto& Pair : PlatformList)
	{
		const auto& PlatformPtr = Pair.Value;
		if (PlatformPtr->IsPresentInGrid())
		{
			for (int32 QualityLevel = 0; QualityLevel < EMaterialQualityLevel::Num; ++QualityLevel)
			{
				const auto& PlatformData = PlatformPtr->GetPlatformData((EMaterialQualityLevel::Type)QualityLevel);
				for (int32 InstanceIndex = 0; InstanceIndex < PlatformData.Instances.Num(); ++InstanceIndex)
				{
					const FMaterialResource* CurrentResource = PlatformData.Instances[InstanceIndex].MaterialResourcesStats;
					if (CurrentResource && CurrentResource->GetCompileErrors().Num() > 0)
					{
						const auto& PlatformName = MaterialStatsManager->GetPlatformName(Pair.Key);
						const auto& AssetName = MaterialStatsManager->GetMaterialName(InstanceIndex);
						const FString QualityName = FMaterialStatsUtils::MaterialQualityToShortString((EMaterialQualityLevel::Type)QualityLevel);

						if (InstanceIndex == 0)
						{
							Errors.Push(FText::Format(NSLOCTEXT("UnrealEd", "Warning_CompileErrorsInMaterial_ListEntryMaterial", "- For platform {0} at quality level {1}."), FText::FromName(PlatformName), FText::FromString(QualityName)));
						}
						else
						{
							Errors.Push(FText::Format(NSLOCTEXT("UnrealEd", "Warning_CompileErrorsInMaterial_ListEntryDerivedMaterial", "- Material instance {0} for platform {1} at quality level {2}."), FText::FromString(*AssetName), FText::FromName(PlatformName), FText::FromString(QualityName)));
						}
					}
				}
			}
		}
	}

	if (Errors.Num() > 0)
	{
		const FText JoinedErrors = FText::Join(FText::FromString(TEXT("\n")), Errors);
		if (Material->bUsedAsSpecialEngineMaterial && bBaseMaterialFailsToCompile)
		{
			FSuppressableWarningDialog::FSetupInfo Info(
				FText::Format(NSLOCTEXT("UnrealEd", "Error_CompileErrorsInDefaultMaterial", "The current material has the following compilation errors:\n{0}\nThis material is a Default Material which must be available as a code fallback at all times, compilation errors are not allowed."), JoinedErrors),
				NSLOCTEXT("UnrealEd", "Warning_CompileErrorsInDefaultMaterial_Title", "Error: Compilation errors in Default Material"), "Error_CompileErrorsInDefaultMaterial");
			Info.ConfirmText = NSLOCTEXT("ModalDialogs", "CompileErrorsInDefaultMaterialOk", "Ok");
			Info.DialogMode = FSuppressableWarningDialog::EMode::DontPersistSuppressionAcrossSessions;
			Info.WrapMessageAt = 0.0f;

			FSuppressableWarningDialog CompileErrors(Info);
			CompileErrors.ShowModal();
			return false;
		}
		else
		{
			FSuppressableWarningDialog::FSetupInfo Info(
				FText::Format(NSLOCTEXT("UnrealEd", "Warning_CompileErrorsInMaterial", "The current material has the following compilation errors:\n{0}\nAre you sure you wish to continue?"), JoinedErrors),
				NSLOCTEXT("UnrealEd", "Warning_CompileErrorsInMaterial_Title", "Warning: Compilation errors in this Material" ), "Warning_CompileErrorsInMaterial");
			Info.ConfirmText = NSLOCTEXT("ModalDialogs", "CompileErrorsInMaterialConfirm", "Continue");
			Info.CancelText = NSLOCTEXT("ModalDialogs", "CompileErrorsInMaterialCancel", "Abort");
			Info.DialogMode = FSuppressableWarningDialog::EMode::DontPersistSuppressionAcrossSessions;
			Info.WrapMessageAt = 0.0f;

			FSuppressableWarningDialog CompileErrorsWarning( Info );
			if(CompileErrorsWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
			{
				return false;
			}
		}
	}

	// Make sure any graph position changes that might not have been copied are taken into account
	Material->MaterialGraph->LinkMaterialExpressionsFromGraph();

	//remove any memory copies of shader files, so they will be reloaded from disk
	//this way the material editor can be used for quick shader iteration
	FlushShaderFileCache();

	//recompile and refresh the preview material so it will be updated if there was a shader change
	//Force it even if bLivePreview is false.
	UpdatePreviewMaterial(true);
	RegenerateCodeView(true);

	const FScopedBusyCursor BusyCursor;

	const FText LocalizedMaterialEditorApply = NSLOCTEXT("UnrealEd", "ToolTip_MaterialEditorApply", "Apply changes to original material and its use in the world.");
	GWarn->BeginSlowTask( LocalizedMaterialEditorApply, true );
	GWarn->StatusUpdate( 1, 1, LocalizedMaterialEditorApply );

	// Handle propagation of the material function being edited
	if (MaterialFunction)
	{
		// Copy the expressions back from the preview material
		MaterialFunction->AssignExpressionCollection(Material->GetExpressionCollection());

		// Preserve the thumbnail info
		UThumbnailInfo* OriginalThumbnailInfo = MaterialFunction->ParentFunction->ThumbnailInfo;
		UThumbnailInfo* ThumbnailInfo = MaterialFunction->ThumbnailInfo;
		MaterialFunction->ParentFunction->ThumbnailInfo = NULL;
		MaterialFunction->ThumbnailInfo = NULL;

		// Cache any metadata
		const TMap<FName, FString>* MetaData = FMetaData::GetMapForObject(MaterialFunction->ParentFunction);

		// overwrite the original material function in place by constructing a new one with the same name
		MaterialFunction->ParentFunction = (UMaterialFunction*)StaticDuplicateObject(
			MaterialFunction, 
			MaterialFunction->ParentFunction->GetOuter(), 
			MaterialFunction->ParentFunction->GetFName(), 
			RF_AllFlags, 
			MaterialFunction->ParentFunction->GetClass());

		// Restore the thumbnail info
		MaterialFunction->ParentFunction->ThumbnailInfo = OriginalThumbnailInfo;
		MaterialFunction->ThumbnailInfo = ThumbnailInfo;

		// Restore the metadata
		if (MetaData)
		{
			FMetaData& PackageMetaData = MaterialFunction->ParentFunction->GetPackage()->GetMetaData();
			PackageMetaData.SetObjectValues(MaterialFunction->ParentFunction, *MetaData);
		}

		// Restore RF_Standalone on the original material function, as it had been removed from the preview material so that it could be GC'd.
		MaterialFunction->ParentFunction->SetFlags( RF_Standalone );

		for (UMaterialExpression* CurrentExpression : MaterialFunction->ParentFunction->GetExpressions())
		{
			ensureMsgf(CurrentExpression, TEXT("Invalid expression whilst saving material function."));

			// Link the expressions back to their function
			if (CurrentExpression)
			{
				CurrentExpression->Material = NULL;
				CurrentExpression->Function = MaterialFunction->ParentFunction;
			}	
		}
		for (UMaterialExpressionComment* CurrentExpression : MaterialFunction->ParentFunction->GetEditorComments())
		{
			ensureMsgf(CurrentExpression, TEXT("Invalid comment whilst saving material function."));

			// Link the expressions back to their function
			if (CurrentExpression)
			{
				CurrentExpression->Material = NULL;
				CurrentExpression->Function = MaterialFunction->ParentFunction;
			}
		}

		// clear the dirty flag
		bMaterialDirty = false;
		bStatsFromPreviewMaterial = false;

		// Since MaterialFunction->ParentFunction gets initialized as a duplicate here and bExposeToLibrary 
		// is protected against duplicate for non-transient objects, we need to manually copy it
		MaterialFunction->ParentFunction->bExposeToLibrary = MaterialFunction->bExposeToLibrary;

		UMaterialEditingLibrary::UpdateMaterialFunction(MaterialFunction->ParentFunction, Material);
	}
	// Handle propagation of the material being edited
	else
	{
		FNavigationLockContext NavUpdateLock(ENavigationLockReason::MaterialUpdate);

		// ensure the original copy of the material is removed from the editor's selection set
		// or it will end up containing a stale, invalid entry
		if ( OriginalMaterial->IsSelected() )
		{
			GEditor->GetSelectedObjects()->Deselect( OriginalMaterial );
		}

		// Preserve the thumbnail info
		UThumbnailInfo* OriginalThumbnailInfo = OriginalMaterial->ThumbnailInfo;
		UThumbnailInfo* ThumbnailInfo = Material->ThumbnailInfo;
		OriginalMaterial->ThumbnailInfo = NULL;
		Material->ThumbnailInfo = NULL;

		// Cache any metadata
		const TMap<FName, FString>* MetaData = FMetaData::GetMapForObject(OriginalMaterial);

		// A bit hacky, but disable material compilation in post load when we duplicate the material.
		UMaterial::ForceNoCompilationInPostLoad(true);

		// Now, the material has been loaded and serialized. PostLoad has been called and the asset has been converted and now it is up to date.
		// Let's thus reset the linker (to the last version) to make sure PostLoad on the duplicated object is not doing any data conversion, squashing new properties with not properly initialized _DEPRECATED members (e.g. RefractionMode_DEPRECATED).
		ResetLoaders(OriginalMaterial->GetPackage());

		// overwrite the original material in place by constructing a new one with the same name
		OriginalMaterial = (UMaterial*)StaticDuplicateObject( Material, OriginalMaterial->GetOuter(), OriginalMaterial->GetFName(), 
			RF_AllFlags, 
			OriginalMaterial->GetClass());

		// Post load has been called, allow materials to be compiled in PostLoad.
		UMaterial::ForceNoCompilationInPostLoad(false);

		// Restore the thumbnail info
		OriginalMaterial->ThumbnailInfo = OriginalThumbnailInfo;
		Material->ThumbnailInfo = ThumbnailInfo;

		// Restore the metadata
		if (MetaData)
		{
			FMetaData& PackageMetaData = OriginalMaterial->GetPackage()->GetMetaData();
			PackageMetaData.SetObjectValues(OriginalMaterial, *MetaData);
		}

		// Change the original material object to the new original material
		OriginalMaterialObject = OriginalMaterial;

		// Restore RF_Standalone on the original material, as it had been removed from the preview material so that it could be GC'd.
		OriginalMaterial->SetFlags( RF_Standalone );

		// Manually copy bUsedAsSpecialEngineMaterial as it is duplicate transient to prevent accidental creation of new special engine materials
		OriginalMaterial->bUsedAsSpecialEngineMaterial = Material->bUsedAsSpecialEngineMaterial;

		UMaterialEditingLibrary::RecompileMaterial(OriginalMaterial);

		// clear the dirty flag
		bMaterialDirty = false;
		bStatsFromPreviewMaterial = false;
	}

	GWarn->EndSlowTask();

	return true;
}

void FMaterialEditor::OnMessageLogLinkActivated(const class TSharedRef<IMessageToken>& Token)
{
	const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
	if (UObjectToken->GetObject().IsValid())
	{
		UMaterialExpression* Expression = Cast<UMaterialExpression>(UObjectToken->GetObject().Get());
		if(UObject* MaterialOrFunction = Expression->GetAssetOwner())
		{
			UAssetEditorSubsystem* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			if(AssetEditor->OpenEditorForAsset(MaterialOrFunction))
			{
				FMaterialEditor* TargetEditor = static_cast<FMaterialEditor*>(AssetEditor->FindEditorForAsset(MaterialOrFunction, true));
				checkf(TargetEditor, TEXT("Could not find Editor for Asset: %s"), *(MaterialOrFunction->GetFName().ToString()));

				FMaterialExpressionCollection& Collection = Expression->Function ? TargetEditor->MaterialFunction->GetEditorOnlyData()->ExpressionCollection
																		   : TargetEditor->Material->GetEditorOnlyData()->ExpressionCollection;

				for (const TObjectPtr<UMaterialExpression>& EditorExpression : Collection.Expressions)
				{
					if (EditorExpression->MaterialExpressionGuid == Expression->MaterialExpressionGuid
						&& EditorExpression->MaterialExpressionEditorX == Expression->MaterialExpressionEditorX
						&& EditorExpression->MaterialExpressionEditorY == Expression->MaterialExpressionEditorY
						&& EditorExpression->GetName() == Expression->GetName())
					{
						if (EditorExpression->GraphNode)
						{
							TargetEditor->JumpToNode(EditorExpression->GraphNode);
							break;
						}
					}
				}
			}
		}
	}
}

void FMaterialEditor::UpdateMaterialInfoLog()
{
	bool bForceDisplay = false;
	TArray< TSharedRef<class FTokenizedMessage> > Messages;

	TArray<TSharedPtr<FMaterialInfo>> TempMaterialInfoList;

	ERHIFeatureLevel::Type FeatureLevelsToDisplay[2];
	int32 NumFeatureLevels = 0;
	// Always show basic features so that errors aren't hidden
	FeatureLevelsToDisplay[NumFeatureLevels++] = GMaxRHIFeatureLevel;

	UMaterial* MaterialForStats = bStatsFromPreviewMaterial ? Material : OriginalMaterial;

	for (int32 i = 0; i < NumFeatureLevels; ++i)
	{
		TArray<FString>				 CompileErrors;
		TArray<UMaterialExpression*> FailingExpression;

		ERHIFeatureLevel::Type FeatureLevel = FeatureLevelsToDisplay[i];
		const FMaterialResource* MaterialResource = MaterialForStats->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(FeatureLevel));

		if (MaterialResource == nullptr)
		{
			continue;
		}
		
		CompileErrors = MaterialResource->GetCompileErrors();
		FailingExpression = MaterialResource->GetErrorExpressions();

		// Only show general info if there are no errors and stats are enabled - Stats show for Materials, layers and blends
		if (CompileErrors.Num() == 0 && (!MaterialFunction || MaterialFunction->GetMaterialFunctionUsage() != EMaterialFunctionUsage::Default))
		{
			TArray<FMaterialStatsUtils::FShaderInstructionsInfo> Results;
			TArray<FMaterialStatsUtils::FShaderInstructionsInfo> EmptyMaterialResults;
			FMaterialStatsUtils::GetRepresentativeInstructionCounts(Results, MaterialResource);

			//Built in stats is no longer exposed to the UI but may still be useful so they're still in the code.
			bool bBuiltinStats = false;
			const FMaterialResource* EmptyMaterialResource = EmptyMaterial ? EmptyMaterial->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(FeatureLevel)) : nullptr;
			if (bShowBuiltinStats && bStatsFromPreviewMaterial && EmptyMaterialResource && Results.Num() > 0)
			{
				FMaterialStatsUtils::GetRepresentativeInstructionCounts(EmptyMaterialResults, EmptyMaterialResource);

				if (EmptyMaterialResults.Num() > 0)
				{
					//The instruction counts should match. If not, the preview material has been changed without the EmptyMaterial being updated to match.
					if (ensure(Results.Num() == EmptyMaterialResults.Num()))
					{
						bBuiltinStats = true;
					}
				}
			}
		}

		FString FeatureLevelName;
		GetFeatureLevelName(FeatureLevel,FeatureLevelName);
		for(int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
		{
			const FString ErrorString = FString::Printf(TEXT("[%s] %s"), *FeatureLevelName, *CompileErrors[ErrorIndex]);
			TempMaterialInfoList.Add(MakeShareable(new FMaterialInfo(ErrorString, FLinearColor::Red)));
			TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create( EMessageSeverity::Error );
			bool bHyperlinkAdded = false;

			if(FailingExpression.Num() && ensure(FailingExpression.Num() == CompileErrors.Num()) && FailingExpression[ErrorIndex])
			{
				Line->SetMessageLink(FUObjectToken::Create(FailingExpression[ErrorIndex]));

				const FString& CompilerError = CompileErrors[ErrorIndex];
				int32 NodeNameEndPos = INDEX_NONE;

				// If we have a failing expression, first matching parantheses will be for its name
				if (CompilerError.FindChar(L')', NodeNameEndPos))
				{
					bHyperlinkAdded = true;
					Line->AddToken(FTextToken::Create(FText::FromString(FeatureLevelName)));

					const FString NodeName = CompilerError.Left(NodeNameEndPos + 1);
					const FString ErrorBody = CompilerError.RightChop(NodeNameEndPos + 2);

					Line->AddToken(FUObjectToken::Create(FailingExpression[ErrorIndex], FText::FromString(NodeName)));
					Line->AddToken(FTextToken::Create(FText::FromString(ErrorBody)));
				}
			}

			if (!bHyperlinkAdded)
			{
				Line->AddToken(FTextToken::Create(FText::FromString(ErrorString)));
			}

			Messages.Add(Line);
			bForceDisplay = true;
		}
	}

	bool bNeedsRefresh = false;
	if (TempMaterialInfoList.Num() != MaterialInfoList.Num())
	{
		bNeedsRefresh = true;
	}

	for (int32 Index = 0; !bNeedsRefresh && Index < TempMaterialInfoList.Num(); ++Index)
	{
		if (TempMaterialInfoList[Index]->Color != MaterialInfoList[Index]->Color)
		{
			bNeedsRefresh = true;
			break;
		}

		if (TempMaterialInfoList[Index]->Text != MaterialInfoList[Index]->Text)
		{
			bNeedsRefresh = true;
			break;
		}
	}

	if (bNeedsRefresh)
	{
		MaterialInfoList = TempMaterialInfoList;

		auto Listing = MaterialStatsManager->GetOldStatsListing();
		Listing->ClearMessages();
		Listing->AddMessages(Messages);

		if (bForceDisplay)
		{
			TabManager->TryInvokeTab(MaterialStatsManager->GetGridOldStatsTabName());
		}
	}
}

void FMaterialEditor::UpdateMaterialPlatformStats()
{
	// setup stats widget visibility
	if (MaterialFunction)
	{
		return;
	}

	UMaterial* MaterialForStats = bStatsFromPreviewMaterial ? Material : OriginalMaterial;

	// check for errors
	TArray<FString> CompileErrors;
	if (MaterialFunction && ExpressionPreviewMaterial)
	{
		// Add a compile error message for functions missing an output
		CompileErrors = ExpressionPreviewMaterial->GetMaterialResource(GMaxRHIShaderPlatform)->GetCompileErrors();

		bool bFoundFunctionOutput = false;
		for (UMaterialExpression* MaterialExpression : Material->GetExpressions())
		{
			if (MaterialExpression->IsA(UMaterialExpressionFunctionOutput::StaticClass()))
			{
				bFoundFunctionOutput = true;
				break;
			}
		}

		if (!bFoundFunctionOutput)
		{
			CompileErrors.Add(TEXT("Missing a function output"));
		}
	}

	// compute error crc
	FString NewErrorHash;
	for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
	{
		NewErrorHash += FMD5::HashAnsiString(*CompileErrors[ErrorIndex]);
	}

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		TSharedPtr<SWidget> TitleBar = FocusedGraphEd->GetTitleBar();
		TSharedPtr<SMaterialEditorTitleBar> MaterialTitleBar = StaticCastSharedPtr<SMaterialEditorTitleBar>(TitleBar);
		if (NewErrorHash != MaterialErrorHash)
		{
			MaterialErrorHash = NewErrorHash;
			MaterialInfoList.Reset();

			TArray< TSharedRef<class FTokenizedMessage> > Messages;
			FString FeatureLevelName;
			GetFeatureLevelName(GMaxRHIFeatureLevel, FeatureLevelName);
			for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
			{
				FString ErrorString = FString::Printf(TEXT("[%s] %s"), *FeatureLevelName, *CompileErrors[ErrorIndex]);
				TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Error);
				Line->AddToken(FTextToken::Create(FText::FromString(ErrorString)));
				Messages.Add(Line);

				MaterialInfoList.Add(MakeShareable(new FMaterialInfo(ErrorString, FLinearColor::Red)));
			}

			StatsListing->ClearMessages();
			StatsListing->AddMessages(Messages);

				MaterialTitleBar->RequestRefresh();
		}

		if (MaterialTitleBar->MaterialInfoList)
		{
			MaterialTitleBar->MaterialInfoList->SetVisibility(CompileErrors.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);
		}
	}

	if (DerivedMaterialInstances.IsEmpty())
	{
		CreateDerivedMaterialInstancesPreviews();
	}

	// extract material stats
	MaterialStatsManager->SetMaterial(MaterialForStats, bStatsFromPreviewMaterial ? DerivedMaterialInstances : OriginalDerivedMaterialInstances);
	MaterialStatsManager->Update();

	// Update the num shaders UI
	if (NumShadersText.IsValid())
	{
		EShaderPlatform Platform = GEditor->GetActiveShaderPlatform();
		if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform))
		{
			Platform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(Platform);
		}
		NumShadersText->SetText(MaterialStatsManager->GetShaderCountText(Platform, GetQualityPreviewLevel(), 0));
	}

	// check if any derived instances fail, if so show the window
	if (MaterialStatsManager->AnyNewCompilationErrors(1))
	{
		// but then check if base material is compiling, if not, keep the old stats popup not to break UX flow until we merge old and new stats together
		if (!MaterialStatsManager->AnyNewCompilationErrors(0))
		{
			TabManager->TryInvokeTab(MaterialStatsManager->GetGridStatsTabName());
		}
	}
}

void FMaterialEditor::UpdateGraphNodeStates()
{
	const FMaterialResource* ErrorMaterialResource = PreviewExpression ? ExpressionPreviewMaterial->GetMaterialResource(GMaxRHIShaderPlatform) : Material->GetMaterialResource(GMaxRHIShaderPlatform);

	bool bUpdatedErrorState = false;
	bool bToggledVisibleState = bPreviewFeaturesChanged;
	bool bShowAllNodes = true;

	TArray<UMaterialExpression*> VisibleExpressions;

	FStaticParameterSet StaticSwitchSet;
	if (bPreviewFeaturesChanged && bPreviewStaticSwitches)
	{
		for (UMaterialExpression* Expression : Material->GetExpressions())
		{
			if (UMaterialExpressionStaticSwitchParameter* StaticSwitch = Cast<UMaterialExpressionStaticSwitchParameter>(Expression))
			{
				FStaticSwitchParameter SwitchParam;
				SwitchParam.Value = StaticSwitch->DefaultValue;
				SwitchParam.ExpressionGUID = StaticSwitch->ExpressionGUID;
				StaticSwitchSet.StaticSwitchParameters.Add(SwitchParam);
			}
		}
	}

	if (bPreviewFeaturesChanged)
	{
		Material->GetAllReferencedExpressions(VisibleExpressions, StaticSwitchSet.IsEmpty() ? nullptr : &StaticSwitchSet, NodeFeatureLevel, NodeQualityLevel);
		if (NodeFeatureLevel != ERHIFeatureLevel::Num || NodeQualityLevel != EMaterialQualityLevel::Num || !StaticSwitchSet.IsEmpty())
		{
			bShowAllNodes = false;
		}
	}
	
	// Update main material graph and all subgraphs
	bUpdatedErrorState |= UpdateGraphNodeState(Material->MaterialGraph, ErrorMaterialResource, VisibleExpressions, bShowAllNodes);

	bPreviewFeaturesChanged = false;

	if (bUpdatedErrorState || bToggledVisibleState)
	{
		// Rebuild the SGraphNodes to display/hide error block
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			FocusedGraphEd->NotifyGraphChanged();
		}
	}
}

bool FMaterialEditor::UpdateGraphNodeState(UEdGraph* Graph, const FMaterialResource* ErrorMaterialResource, TArray<UMaterialExpression*>& VisibleExpressions, bool bShowAllNodes)
{
	bool bUpdatedErrorState = false;

	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(Graph);

	// Have to loop through everything here as there's no way to be notified when the material resource updates
	for (int32 Index = 0; Index < MaterialGraph->Nodes.Num(); ++Index)
	{
		UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(MaterialGraph->Nodes[Index]);
		if (MaterialNode)
		{
			MaterialNode->bIsPreviewExpression = (PreviewExpression == MaterialNode->MaterialExpression);
			MaterialNode->bIsErrorExpression = (ErrorMaterialResource != nullptr) && (ErrorMaterialResource->GetErrorExpressions().Find(MaterialNode->MaterialExpression) != INDEX_NONE);

			if (MaterialNode->bIsErrorExpression && !MaterialNode->bHasCompilerMessage)
			{
				check(MaterialNode->MaterialExpression);

				bUpdatedErrorState = true;
				MaterialNode->bHasCompilerMessage = true;
				MaterialNode->ErrorMsg = MaterialNode->MaterialExpression->LastErrorText;
				MaterialNode->ErrorType = EMessageSeverity::Error;
			}
			else if (!MaterialNode->bIsErrorExpression && MaterialNode->bHasCompilerMessage)
			{
				bUpdatedErrorState = true;
				MaterialNode->bHasCompilerMessage = false;
			}
			if (MaterialNode->MaterialExpression && bPreviewFeaturesChanged)
			{
				if ((bShowAllNodes || VisibleExpressions.Contains(MaterialNode->MaterialExpression)))
				{
					MaterialNode->SetForceDisplayAsDisabled(false);
				}
				else if (!bShowAllNodes && !VisibleExpressions.Contains(MaterialNode->MaterialExpression))
				{
					MaterialNode->SetForceDisplayAsDisabled(true);
				}
			}
		}
	}

	for (UEdGraph* SubGraph : Graph->SubGraphs)
	{
		bUpdatedErrorState |= UpdateGraphNodeState(SubGraph, ErrorMaterialResource, VisibleExpressions, bShowAllNodes);
	}

	return bUpdatedErrorState;
}

void FMaterialEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(EditorOptions);
	Collector.AddReferencedObject(Material);
	Algo::ForEach(DerivedMaterialInstances, [&Collector](TObjectPtr<UMaterialInstance> MaterialInstance){ Collector.AddReferencedObject(MaterialInstance); });
	Algo::ForEach(OriginalDerivedMaterialInstances, [&Collector](TObjectPtr<UMaterialInstance> MaterialInstance){ Collector.AddReferencedObject(MaterialInstance); });
	Collector.AddReferencedObject(OriginalMaterial);
	Collector.AddReferencedObject(MaterialFunction);
	Collector.AddReferencedObject(ExpressionPreviewMaterial);
	Collector.AddReferencedObject(EmptyMaterial);
	Collector.AddReferencedObject(MaterialEditorInstance);
	Collector.AddReferencedObject(PreviewExpression);
	for (FMatExpressionPreview* ExpressionPreview : ExpressionPreviews)
	{
		ExpressionPreview->AddReferencedObjects(Collector);
	}
}

void FMaterialEditor::BindCommands()
{
	const FMaterialEditorCommands& Commands = FMaterialEditorCommands::Get();
	
	ToolkitCommands->MapAction(
		Commands.Apply,
		FExecuteAction::CreateSP( this, &FMaterialEditor::OnApply ),
		FCanExecuteAction::CreateSP( this, &FMaterialEditor::OnApplyEnabled ) );

	ToolkitCommands->MapAction(
		FEditorViewportCommands::Get().ToggleRealTime,
		FExecuteAction::CreateSP( PreviewViewport.ToSharedRef(), &SMaterialEditor3DPreviewViewport::OnToggleRealtime ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( PreviewViewport.ToSharedRef(), &SMaterialEditor3DPreviewViewport::IsRealtime ) );

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &FMaterialEditor::UndoGraphAction));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &FMaterialEditor::RedoGraphAction));

	ToolkitCommands->MapAction(
		Commands.CameraHome,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnCameraHome),
		FCanExecuteAction() );

	ToolkitCommands->MapAction(
		Commands.CleanUnusedExpressions,
		FExecuteAction::CreateSP(this, &FMaterialEditor::CleanUnusedExpressions),
		FCanExecuteAction() );

	ToolkitCommands->MapAction(
		Commands.FixupMismatchedSamplers,
		FExecuteAction::CreateSP(this, &FMaterialEditor::FixupMismatchedSamplerExpressions),
		FCanExecuteAction() );

	ToolkitCommands->MapAction(
		Commands.ShowHideConnectors,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnHideConnectors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsOnHideConnectorsChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleLivePreview,
		FExecuteAction::CreateSP(this, &FMaterialEditor::ToggleLivePreview),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsToggleLivePreviewChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleHideUnrelatedNodes,
		FExecuteAction::CreateSP(this, &FMaterialEditor::ToggleHideUnrelatedNodes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsToggleHideUnrelatedNodesChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleRealtimeExpressions,
		FExecuteAction::CreateSP(this, &FMaterialEditor::ToggleRealTimeExpressions),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsToggleRealTimeExpressionsChecked));

	ToolkitCommands->MapAction(
		Commands.AlwaysRefreshAllPreviews,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlwaysRefreshAllPreviews),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsOnAlwaysRefreshAllPreviews));

	ToolkitCommands->MapAction(
		Commands.UseCurrentTexture,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnUseCurrentTexture));

	ToolkitCommands->MapAction(
		Commands.ConvertObjects,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertObjects));

	ToolkitCommands->MapAction(
		Commands.PromoteToDouble,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnPromoteObjects));

	ToolkitCommands->MapAction(
		Commands.PromoteToFloat,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnPromoteObjects));

	ToolkitCommands->MapAction(
		Commands.ConvertToTextureObjects,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertTextures));

	ToolkitCommands->MapAction(
		Commands.ConvertToTextureSamples,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertTextures));

	ToolkitCommands->MapAction(
		Commands.ConvertToConstant,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertObjects));
	ToolkitCommands->MapAction(
		Commands.SelectNamedRerouteDeclaration,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectNamedRerouteDeclaration));
	
	ToolkitCommands->MapAction(
		Commands.SelectNamedRerouteUsages,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectNamedRerouteUsages));

	ToolkitCommands->MapAction(
		Commands.CreateRerouteUsageFromDeclaration,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnCreateRerouteUsageFromDeclaration));
	
	ToolkitCommands->MapAction(
		Commands.ConvertRerouteToNamedReroute,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertRerouteToNamedReroute));
	
	ToolkitCommands->MapAction(
		Commands.ConvertNamedRerouteToReroute,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertNamedRerouteToReroute));

	ToolkitCommands->MapAction(
		Commands.StopPreviewNode,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnPreviewNode));

	ToolkitCommands->MapAction(
		Commands.StartPreviewNode,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnPreviewNode));

	ToolkitCommands->MapAction(
		Commands.EnableRealtimePreviewNode,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnToggleRealtimePreview));

	ToolkitCommands->MapAction(
		Commands.DisableRealtimePreviewNode,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnToggleRealtimePreview));

	ToolkitCommands->MapAction(
		Commands.SelectDownstreamNodes,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectDownstreamNodes));

	ToolkitCommands->MapAction(
		Commands.SelectUpstreamNodes,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectUpstreamNodes));

	ToolkitCommands->MapAction(
		Commands.RemoveFromFavorites,
		FExecuteAction::CreateSP(this, &FMaterialEditor::RemoveSelectedExpressionFromFavorites));
		
	ToolkitCommands->MapAction(
		Commands.AddToFavorites,
		FExecuteAction::CreateSP(this, &FMaterialEditor::AddSelectedExpressionToFavorites));

	ToolkitCommands->MapAction(
		Commands.ForceRefreshPreviews,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnForceRefreshPreviews));

	ToolkitCommands->MapAction(
		Commands.FindInMaterial,
		FExecuteAction::CreateSP(this, &FMaterialEditor::OnFindInMaterial));

	ToolkitCommands->MapAction(
		Commands.QualityLevel_All,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetQualityPreview, EMaterialQualityLevel::Num),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsQualityPreviewChecked, EMaterialQualityLevel::Num));
	ToolkitCommands->MapAction(
		Commands.QualityLevel_Epic,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetQualityPreview, EMaterialQualityLevel::Epic),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsQualityPreviewChecked, EMaterialQualityLevel::Epic));
	ToolkitCommands->MapAction(
		Commands.QualityLevel_High,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetQualityPreview, EMaterialQualityLevel::High),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsQualityPreviewChecked, EMaterialQualityLevel::High));
	ToolkitCommands->MapAction(
		Commands.QualityLevel_Medium,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetQualityPreview, EMaterialQualityLevel::Medium),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsQualityPreviewChecked, EMaterialQualityLevel::Medium));
	ToolkitCommands->MapAction(
		Commands.QualityLevel_Low,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetQualityPreview, EMaterialQualityLevel::Low),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsQualityPreviewChecked, EMaterialQualityLevel::Low));

	ToolkitCommands->MapAction(
		Commands.FeatureLevel_All,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetFeaturePreview, ERHIFeatureLevel::Num),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsFeaturePreviewChecked, ERHIFeatureLevel::Num));
	ToolkitCommands->MapAction(
		Commands.FeatureLevel_Mobile,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetFeaturePreview, ERHIFeatureLevel::ES3_1),
		FCanExecuteAction::CreateSP(this, &FMaterialEditor::IsFeaturePreviewAvailable, ERHIFeatureLevel::ES3_1),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsFeaturePreviewChecked, ERHIFeatureLevel::ES3_1));
	ToolkitCommands->MapAction(
		Commands.FeatureLevel_SM5,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetFeaturePreview, ERHIFeatureLevel::SM5),
		FCanExecuteAction::CreateSP(this, &FMaterialEditor::IsFeaturePreviewAvailable, ERHIFeatureLevel::SM5),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsFeaturePreviewChecked, ERHIFeatureLevel::SM5));
	ToolkitCommands->MapAction(
		Commands.FeatureLevel_SM6,
		FExecuteAction::CreateSP(this, &FMaterialEditor::SetFeaturePreview, ERHIFeatureLevel::SM6),
		FCanExecuteAction::CreateSP(this, &FMaterialEditor::IsFeaturePreviewAvailable, ERHIFeatureLevel::SM6),
		FIsActionChecked::CreateSP(this, &FMaterialEditor::IsFeaturePreviewChecked, ERHIFeatureLevel::SM6));
}

void FMaterialEditor::OnApply()
{
	UE_LOGF(LogMaterialEditor, Log, "Applying material %ls", *GetEditingObjects()[0]->GetName());

	UpdateOriginalMaterial();

	GEditor->OnSceneMaterialsModified();
}

bool FMaterialEditor::OnApplyEnabled() const
{
	return bMaterialDirty == true;
}

void FMaterialEditor::OnCameraHome()
{
	RecenterEditor();
}

void FMaterialEditor::OnHideConnectors()
{
	bHideUnusedConnectors = !bHideUnusedConnectors;

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		FocusedGraphEd->SetPinVisibility(bHideUnusedConnectors ? SGraphEditor::Pin_HideNoConnection : SGraphEditor::Pin_Show);
	}
}

bool FMaterialEditor::IsOnHideConnectorsChecked() const
{
	return bHideUnusedConnectors == true;
}

void FMaterialEditor::ToggleLivePreview()
{
	bLivePreview = !bLivePreview;
	if (bLivePreview)
	{
		UpdatePreviewMaterial();
		RegenerateCodeView();
	}
}

bool FMaterialEditor::IsToggleLivePreviewChecked() const
{
	return bLivePreview;
}

void FMaterialEditor::ToggleRealTimeExpressions()
{
	bIsRealtime = !bIsRealtime;
}

bool FMaterialEditor::IsToggleRealTimeExpressionsChecked() const
{
	return bIsRealtime == true;
}

void FMaterialEditor::OnAlwaysRefreshAllPreviews()
{
	bAlwaysRefreshAllPreviews = !bAlwaysRefreshAllPreviews;
	if ( bAlwaysRefreshAllPreviews )
	{
		RefreshExpressionPreviews();
	}
}

bool FMaterialEditor::IsOnAlwaysRefreshAllPreviews() const
{
	return bAlwaysRefreshAllPreviews == true;
}

void FMaterialEditor::ToggleHideUnrelatedNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	bHideUnrelatedNodes = !bHideUnrelatedNodes;

	FocusedGraphEd->ResetAllNodesUnrelatedStates();

	if (bHideUnrelatedNodes && bSelectRegularNode)
	{
		HideUnrelatedNodes();
	}
	else
	{
		bLockNodeFadeState = false;
		bFocusWholeChain = false;
		bSelectFocusedNodes = false;
	}
}

bool FMaterialEditor::IsToggleHideUnrelatedNodesChecked() const
{
	return bHideUnrelatedNodes == true;
}

bool FMaterialEditor::IsSelectFocusedNodesEnabled() const
{
	return bFocusWholeChain;
}

void FMaterialEditor::CollectDownstreamNodes(UMaterialGraphNode* CurrentNode, TArray<UMaterialGraphNode*>& CollectedNodes)
{
	for (UEdGraphPin* OutputPin : CurrentNode->Pins)
	{
		if (OutputPin->Direction == EGPD_Output)
		{
			for (auto& Link : OutputPin->LinkedTo)
			{
				UMaterialGraphNode* LinkedNode = Cast<UMaterialGraphNode>(Link->GetOwningNode());
				if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
				{
					CollectedNodes.Add(LinkedNode);
					CollectDownstreamNodes(LinkedNode, CollectedNodes);

					if (bFocusWholeChain)
					{
						CollectUpstreamNodes(LinkedNode, CollectedNodes);
					}
				}
			}
		}
	}
}

void FMaterialEditor::CollectUpstreamNodes(UMaterialGraphNode* CurrentNode, TArray<UMaterialGraphNode*>& CollectedNodes)
{
	for (UEdGraphPin* InputPin : CurrentNode->Pins)
	{
		if (InputPin->Direction == EGPD_Input)
		{
			for (auto& Link : InputPin->LinkedTo)
			{
				UMaterialGraphNode* LinkedNode = Cast<UMaterialGraphNode>(Link->GetOwningNode());
				if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
				{
					CollectedNodes.Add(LinkedNode);
					CollectUpstreamNodes(LinkedNode, CollectedNodes);
				}
			}
		}
	}
}

void FMaterialEditor::HideUnrelatedNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	const TArray<UMaterialGraphNode*> NodesToShow = GetSelectionRelatedNodes();

	TArray<class UEdGraphNode*> AllNodes = FocusedGraphEd->GetCurrentGraph()->Nodes;

	TArray<UEdGraphNode*> CommentNodes;
	TArray<UEdGraphNode*> RelatedNodes;

	for (auto& Node : AllNodes)
	{
		// Always draw the root graph node which can't cast to UMaterialGraphNode
		if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Node))
		{
			if (NodesToShow.Contains(GraphNode))
			{
				Node->SetNodeUnrelated(false);
				RelatedNodes.Add(Node);
			}
			else
			{
				Node->SetNodeUnrelated(true);
			}
		}
		else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(Node))
		{
			CommentNodes.Add(Node);
		}
	}

	FocusedGraphEd->FocusCommentNodes(CommentNodes, RelatedNodes);

	if (bFocusWholeChain && bSelectFocusedNodes)
	{
		SelectMaterialGraphNodes(*FocusedGraphEd, NodesToShow);
	}
}

void FMaterialEditor::SelectMaterialGraphNodes(SGraphEditor& InGraphEd, const TArray<UMaterialGraphNode*>& InNodes)
{
	TArray<UObject*> SelectedObjects;
	SelectedObjects.Reserve(InNodes.Num());

	bFocusSelectionInProgress = true;
	for (UMaterialGraphNode* Node : InNodes)
	{
		SelectedObjects.Add(Node->MaterialExpression);
		InGraphEd.SetNodeSelection(Node, true);
	}
	bFocusSelectionInProgress = false;

	GetDetailView()->SetObjects(SelectedObjects, true);
	FocusDetailsPanel();
}

void FMaterialEditor::MakeHideUnrelatedNodesOptionsMenu(UToolMenu* Menu)
{
	Menu->bShouldCloseWindowAfterMenuSelection = true;

	TSharedRef<SWidget> LockNodeStateCheckBox = SNew(SBox)
		[
			SNew(SCheckBox)
				.IsChecked(bLockNodeFadeState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &FMaterialEditor::OnLockNodeStateCheckStateChanged)
				.Style(FAppStyle::Get(), "Menu.CheckBox")
				.ToolTipText(LOCTEXT("LockNodeStateCheckBoxToolTip", "Lock the current state of all nodes."))
				.Content()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("LockNodeState", "Lock Node State"))
					]
				]
		];

	TSharedRef<SWidget> FocusWholeChainCheckBox = SNew(SBox)
		[
			SNew(SCheckBox)
				.IsChecked(bFocusWholeChain ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &FMaterialEditor::OnFocusWholeChainCheckStateChanged)
				.Style(FAppStyle::Get(), "Menu.CheckBox")
				.ToolTipText(LOCTEXT("FocusWholeChainCheckBoxToolTip", "Focus all nodes in the chain."))
				.Content()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("FocusWholeChain", "Focus Whole Chain"))
					]
				]
		];

	TSharedRef<SWidget> SelectFocusedNodesCheckBox = SNew(SBox)
		.IsEnabled(this, &FMaterialEditor::IsSelectFocusedNodesEnabled)
		[
			SNew(SCheckBox)
				.IsChecked(bSelectFocusedNodes ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &FMaterialEditor::OnSelectFocusedNodesCheckStateChanged)
				.Style(FAppStyle::Get(), "Menu.CheckBox")
				.ToolTipText(
					LOCTEXT("SelectFocusedNodesTooltip", "Select focused nodes in the chain. \n" 
					"This only works if Focus Whole Chain is also enabled."))
				.Content()
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(2.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("SelectFocusedNodes", "Select Focused Nodes"))
								
						]
				]
		];

	FToolMenuSection& OptionsSection = Menu->AddSection("OptionsSection", LOCTEXT("HideUnrelatedOptions", "Hide Unrelated Options"));
	OptionsSection.AddEntry(FToolMenuEntry::InitMenuEntry("LockNodeStateCheckBox", FUIAction(), LockNodeStateCheckBox));
	OptionsSection.AddEntry(FToolMenuEntry::InitMenuEntry("FocusWholeChainCheckBox", FUIAction(), FocusWholeChainCheckBox));
	OptionsSection.AddEntry(FToolMenuEntry::InitMenuEntry("SelectFocusedNodesCheckBox", FUIAction(), SelectFocusedNodesCheckBox));
}

void FMaterialEditor::OnLockNodeStateCheckStateChanged(ECheckBoxState NewCheckedState)
{
	bLockNodeFadeState = (NewCheckedState == ECheckBoxState::Checked) ? true : false;
}

void FMaterialEditor::OnFocusWholeChainCheckStateChanged(ECheckBoxState NewCheckedState)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	bFocusWholeChain = (NewCheckedState == ECheckBoxState::Checked) ? true : false;

	if (bHideUnrelatedNodes && !bLockNodeFadeState && bSelectRegularNode)
	{
		FocusedGraphEd->ResetAllNodesUnrelatedStates();

		HideUnrelatedNodes();
	}
}

void FMaterialEditor::OnSelectFocusedNodesCheckStateChanged(ECheckBoxState NewCheckedState)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	bSelectFocusedNodes = NewCheckedState == ECheckBoxState::Checked;

	if (bHideUnrelatedNodes && bSelectRegularNode)
	{
		const TArray<UMaterialGraphNode*> NodesToSelect = GetSelectionRelatedNodes();
		SelectMaterialGraphNodes(*FocusedGraphEd, NodesToSelect);
	}
}

void FMaterialEditor::OnUseCurrentTexture()
{
	// Set the currently selected texture in the generic browser
	// as the texture to use in all selected texture sample expressions.
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
	UTexture* SelectedTexture = GEditor->GetSelectedObjects()->GetTop<UTexture>();
	USparseVolumeTexture* SelectedSparseVolumeTexture = GEditor->GetSelectedObjects()->GetTop<USparseVolumeTexture>();
	if ( SelectedTexture || SelectedSparseVolumeTexture )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "UseCurrentTexture", "Use Current Texture") );
		const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (SelectedTexture && GraphNode && GraphNode->MaterialExpression->IsA(UMaterialExpressionTextureBase::StaticClass()) )
			{
				UMaterialExpressionTextureBase* TextureBase = static_cast<UMaterialExpressionTextureBase*>(GraphNode->MaterialExpression);
				TextureBase->Modify();
				TextureBase->Texture = SelectedTexture;
				TextureBase->AutoSetSampleType();
			}
			else if (SelectedSparseVolumeTexture && GraphNode && GraphNode->MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureBase::StaticClass()))
			{
				UMaterialExpressionSparseVolumeTextureBase* TextureBase = static_cast<UMaterialExpressionSparseVolumeTextureBase*>(GraphNode->MaterialExpression);
				TextureBase->Modify();
				TextureBase->SparseVolumeTexture = SelectedSparseVolumeTexture;
			}
		}

		// Update the current preview material. 
		UpdatePreviewMaterial();
		Material->MarkPackageDirty();
		RegenerateCodeView();
		RefreshExpressionPreviews();
		SetMaterialDirty();
	}
}

void FMaterialEditor::OnPromoteObjects()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("MaterialEditorPromote", "Material Editor: Promote"));
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };
		ModifyMaterial();
		
		TArray<class UEdGraphNode*> NodesToDelete;
		TArray<class UEdGraphNode*> NodesToSelect;

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				// Look for the supported classes to convert from
				UMaterialExpression* CurrentSelectedExpression = GraphNode->MaterialExpression;
				UMaterialExpressionVectorParameter* VectorParameterExpression = Cast<UMaterialExpressionVectorParameter>(CurrentSelectedExpression);
				UMaterialExpressionDoubleVectorParameter* DoubleVectorParameterExpression = Cast<UMaterialExpressionDoubleVectorParameter>(CurrentSelectedExpression);

				// Setup the class to convert to
				UClass* ClassToCreate = nullptr;
				if (VectorParameterExpression)
				{
					ClassToCreate = UMaterialExpressionDoubleVectorParameter::StaticClass();
				}
				else if (DoubleVectorParameterExpression)
				{
					ClassToCreate = UMaterialExpressionVectorParameter::StaticClass();
				}

				if (ClassToCreate)
				{
					UMaterialExpression* NewExpression = CreateNewMaterialExpression(ClassToCreate, FVector2D(GraphNode->NodePosX, GraphNode->NodePosY), true, true);
					if (NewExpression)
					{
						UMaterialGraphNode* NewGraphNode = CastChecked<UMaterialGraphNode>(NewExpression->GraphNode);
						NewGraphNode->ReplaceNode(GraphNode);

						bool bNeedsRefresh = false;

						// Copy over any common values
						if (GraphNode->NodeComment.Len() > 0)
						{
							bNeedsRefresh = true;
							NewGraphNode->NodeComment = GraphNode->NodeComment;
						}

						// Copy over expression-specific values
						NewExpression->SetParameterName(CurrentSelectedExpression->GetParameterName());
						if (VectorParameterExpression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionDoubleVectorParameter>(NewExpression)->DefaultValue = FVector4d(VectorParameterExpression->DefaultValue);
						}
						else if (DoubleVectorParameterExpression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionVectorParameter>(NewExpression)->DefaultValue = FLinearColor(DoubleVectorParameterExpression->DefaultValue);
						}

						if (bNeedsRefresh)
						{
							// Refresh the expression preview if we changed its properties after it was created
							NewExpression->bNeedToUpdatePreview = true;
							RefreshExpressionPreview(NewExpression, true);

							UpdateGenerator();
						}

						NodesToDelete.AddUnique(GraphNode);
						NodesToSelect.Add(NewGraphNode);
					}
				}
			}
		}

		// Delete the replaced nodes
		DeleteNodes(NodesToDelete);

		// Select each of the newly converted expressions
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			for (TArray<UEdGraphNode*>::TConstIterator NodeIter(NodesToSelect); NodeIter; ++NodeIter)
			{
				FocusedGraphEd->SetNodeSelection(*NodeIter, true);
			}
		}
	}
}

void FMaterialEditor::OnConvertObjects()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("MaterialEditorConvert", "Material Editor: Convert") );
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };
		ModifyMaterial();

		TArray<class UEdGraphNode*> NodesToDelete;
		TArray<class UEdGraphNode*> NodesToSelect;

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				// Look for the supported classes to convert from
				UMaterialExpression* CurrentSelectedExpression = GraphNode->MaterialExpression;
				UMaterialExpressionConstant* Constant1Expression = Cast<UMaterialExpressionConstant>(CurrentSelectedExpression);
				UMaterialExpressionConstant2Vector* Constant2Expression = Cast<UMaterialExpressionConstant2Vector>(CurrentSelectedExpression);
				UMaterialExpressionConstant3Vector* Constant3Expression = Cast<UMaterialExpressionConstant3Vector>(CurrentSelectedExpression);
				UMaterialExpressionConstant4Vector* Constant4Expression = Cast<UMaterialExpressionConstant4Vector>(CurrentSelectedExpression);
				UMaterialExpressionTextureSample* TextureSampleExpression = Cast<UMaterialExpressionTextureSample>(CurrentSelectedExpression);
				UMaterialExpressionTextureObject* TextureObjectExpression = Cast<UMaterialExpressionTextureObject>(CurrentSelectedExpression);
				UMaterialExpressionComponentMask* ComponentMaskExpression = Cast<UMaterialExpressionComponentMask>(CurrentSelectedExpression);
				UMaterialExpressionParticleSubUV* ParticleSubUVExpression = Cast<UMaterialExpressionParticleSubUV>(CurrentSelectedExpression);
				UMaterialExpressionScalarParameter* ScalarParameterExpression = Cast<UMaterialExpressionScalarParameter>(CurrentSelectedExpression);
				UMaterialExpressionVectorParameter* VectorParameterExpression = Cast<UMaterialExpressionVectorParameter>(CurrentSelectedExpression);
				UMaterialExpressionTextureObjectParameter* TextureObjectParameterExpression = Cast<UMaterialExpressionTextureObjectParameter>(CurrentSelectedExpression);
				UMaterialExpressionRuntimeVirtualTextureSample* RuntimeVirtualTextureSampleExpression = Cast<UMaterialExpressionRuntimeVirtualTextureSample>(CurrentSelectedExpression);
				UMaterialExpressionSparseVolumeTextureSample* SparseVolumeTextureSampleExpression = Cast<UMaterialExpressionSparseVolumeTextureSample>(CurrentSelectedExpression);
				UMaterialExpressionSparseVolumeTextureObject* SparseVolumeTextureObjectExpression = Cast<UMaterialExpressionSparseVolumeTextureObject>(CurrentSelectedExpression);
				UMaterialExpressionSparseVolumeTextureObjectParameter* SparseVolumeTextureObjectParameterExpression = Cast<UMaterialExpressionSparseVolumeTextureObjectParameter>(CurrentSelectedExpression);

				// Setup the class to convert to
				UClass* ClassToCreate = NULL;
				if (Constant1Expression)
				{
					ClassToCreate = UMaterialExpressionScalarParameter::StaticClass();
				}
				else if (Constant2Expression || Constant3Expression || Constant4Expression)
				{
					ClassToCreate = UMaterialExpressionVectorParameter::StaticClass();
				}
				else if (ParticleSubUVExpression) // Has to come before the TextureSample comparison...
				{
					ClassToCreate = UMaterialExpressionTextureSampleParameterSubUV::StaticClass();
				}
				else if (TextureSampleExpression && TextureSampleExpression->Texture && TextureSampleExpression->Texture->IsA(UTextureCube::StaticClass()))
				{
					ClassToCreate = UMaterialExpressionTextureSampleParameterCube::StaticClass();
				}
				else if (TextureSampleExpression && TextureSampleExpression->Texture && TextureSampleExpression->Texture->IsA(UTexture2DArray::StaticClass()))
				{
					ClassToCreate = UMaterialExpressionTextureSampleParameter2DArray::StaticClass();
				}
				else if (TextureSampleExpression && TextureSampleExpression->Texture && TextureSampleExpression->Texture->IsA(UTextureCubeArray::StaticClass()))
				{
					ClassToCreate = UMaterialExpressionTextureSampleParameterCubeArray::StaticClass();
				}
				else if (TextureObjectExpression)
				{
					ClassToCreate = UMaterialExpressionTextureObjectParameter::StaticClass();
				}
				else if (TextureObjectParameterExpression) // Has to come before TextureSample comparison 
				{
					ClassToCreate = UMaterialExpressionTextureObject::StaticClass();
				}
				else if (TextureSampleExpression)
				{
					ClassToCreate = UMaterialExpressionTextureSampleParameter2D::StaticClass();
				}
				else if (RuntimeVirtualTextureSampleExpression)
				{
					ClassToCreate = UMaterialExpressionRuntimeVirtualTextureSampleParameter::StaticClass();
				}
				else if (SparseVolumeTextureObjectParameterExpression) // Has to come before SparseVolumeTextureSample comparison 
				{
					ClassToCreate = UMaterialExpressionSparseVolumeTextureObject::StaticClass();
				}
				else if (SparseVolumeTextureSampleExpression)
				{
					ClassToCreate = UMaterialExpressionSparseVolumeTextureSampleParameter::StaticClass();
				}
				else if (SparseVolumeTextureObjectExpression)
				{
					ClassToCreate = UMaterialExpressionSparseVolumeTextureObjectParameter::StaticClass();
				}
				else if (ComponentMaskExpression)
				{
					ClassToCreate = UMaterialExpressionStaticComponentMaskParameter::StaticClass();
				}
				else if (ScalarParameterExpression)
				{
					ClassToCreate = UMaterialExpressionConstant::StaticClass();
				}
				else if (VectorParameterExpression)
				{
					ClassToCreate = UMaterialExpressionConstant4Vector::StaticClass();
				}

				if (ClassToCreate)
				{
					UMaterialExpression* NewExpression = CreateNewMaterialExpression(ClassToCreate, FVector2D(GraphNode->NodePosX, GraphNode->NodePosY), true, true );
					if (NewExpression)
					{
						UMaterialGraphNode* NewGraphNode = CastChecked<UMaterialGraphNode>(NewExpression->GraphNode);
						NewGraphNode->ReplaceNode(GraphNode);

						bool bNeedsRefresh = false;

						// Copy over any common values
						if (GraphNode->NodeComment.Len() > 0)
						{
							bNeedsRefresh = true; 
							NewGraphNode->NodeComment = GraphNode->NodeComment;
						}

						// Copy over expression-specific values
						if (Constant1Expression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionScalarParameter>(NewExpression)->DefaultValue = Constant1Expression->R;
						}
						else if (Constant2Expression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionVectorParameter>(NewExpression)->DefaultValue = FLinearColor(Constant2Expression->R, Constant2Expression->G, 0);
						}
						else if (Constant3Expression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionVectorParameter>(NewExpression)->DefaultValue = Constant3Expression->Constant;
							CastChecked<UMaterialExpressionVectorParameter>(NewExpression)->DefaultValue.A = 1.0f;
						}
						else if (Constant4Expression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionVectorParameter>(NewExpression)->DefaultValue = Constant4Expression->Constant;
						}
						else if (TextureSampleExpression && !TextureObjectParameterExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionTextureSampleParameter* NewTextureExpr = CastChecked<UMaterialExpressionTextureSampleParameter>(NewExpression);
							NewTextureExpr->Texture = TextureSampleExpression->Texture;
							NewTextureExpr->Coordinates = TextureSampleExpression->Coordinates;
							NewTextureExpr->AutoSetSampleType();
							NewTextureExpr->IsDefaultMeshpaintTexture = TextureSampleExpression->IsDefaultMeshpaintTexture;
							NewTextureExpr->TextureObject = TextureSampleExpression->TextureObject;
							NewTextureExpr->MipValue = TextureSampleExpression->MipValue;
							NewTextureExpr->CoordinatesDX = TextureSampleExpression->CoordinatesDX;
							NewTextureExpr->CoordinatesDY = TextureSampleExpression->CoordinatesDY;
							NewTextureExpr->MipValueMode = TextureSampleExpression->MipValueMode;
							NewGraphNode->ReconstructNode();
						}
						else if (TextureObjectExpression && !TextureObjectParameterExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionTextureObjectParameter* NewTextureObjectParameterExpression = CastChecked<UMaterialExpressionTextureObjectParameter>(NewExpression);
							NewTextureObjectParameterExpression->Texture = TextureObjectExpression->Texture;
							NewTextureObjectParameterExpression->AutoSetSampleType();
							NewTextureObjectParameterExpression->IsDefaultMeshpaintTexture = TextureObjectExpression->IsDefaultMeshpaintTexture;
						}	
						else if (TextureObjectParameterExpression && (!TextureObjectExpression || !TextureSampleExpression))
						{
							bNeedsRefresh = true;
							UMaterialExpressionTextureObject* NewTextureObjectExpression = CastChecked<UMaterialExpressionTextureObject>(NewExpression);
							NewTextureObjectExpression->Texture = TextureObjectParameterExpression->Texture;
							NewTextureObjectExpression->AutoSetSampleType();
							NewTextureObjectExpression->IsDefaultMeshpaintTexture = TextureObjectParameterExpression->IsDefaultMeshpaintTexture;
						}
						else if (RuntimeVirtualTextureSampleExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionRuntimeVirtualTextureSampleParameter* NewRuntimeVirtualTextureExpression = CastChecked<UMaterialExpressionRuntimeVirtualTextureSampleParameter>(NewExpression);
							NewRuntimeVirtualTextureExpression->VirtualTexture = RuntimeVirtualTextureSampleExpression->VirtualTexture;
							NewRuntimeVirtualTextureExpression->MaterialType = RuntimeVirtualTextureSampleExpression->MaterialType;
							NewRuntimeVirtualTextureExpression->MipValueMode = RuntimeVirtualTextureSampleExpression->MipValueMode;
							NewGraphNode->ReconstructNode();
						}
						else if (SparseVolumeTextureSampleExpression && !SparseVolumeTextureObjectParameterExpression) // Sample -> SampleParameter
						{
							bNeedsRefresh = true;
							UMaterialExpressionSparseVolumeTextureSampleParameter* NewSparseVolumeTextureExpression = CastChecked<UMaterialExpressionSparseVolumeTextureSampleParameter>(NewExpression);
							NewSparseVolumeTextureExpression->SparseVolumeTexture = SparseVolumeTextureSampleExpression->SparseVolumeTexture;
							NewGraphNode->ReconstructNode();
						}
						else if (SparseVolumeTextureObjectExpression && !SparseVolumeTextureObjectParameterExpression) // Object -> ObjectParameter
						{
							bNeedsRefresh = true;
							UMaterialExpressionSparseVolumeTextureObjectParameter* NewSparseVolumeTextureObjectParameterExpression = CastChecked<UMaterialExpressionSparseVolumeTextureObjectParameter>(NewExpression);
							NewSparseVolumeTextureObjectParameterExpression->SparseVolumeTexture = SparseVolumeTextureObjectExpression->SparseVolumeTexture;
						}
						else if (SparseVolumeTextureObjectParameterExpression) // ObjectParameter -> Object
						{
							bNeedsRefresh = true;
							UMaterialExpressionSparseVolumeTextureObject* NewTextureObjectExpression = CastChecked<UMaterialExpressionSparseVolumeTextureObject>(NewExpression);
							NewTextureObjectExpression->SparseVolumeTexture = SparseVolumeTextureObjectParameterExpression->SparseVolumeTexture;
						}
						else if (ComponentMaskExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionStaticComponentMaskParameter* ComponentMask = CastChecked<UMaterialExpressionStaticComponentMaskParameter>(NewExpression);
							ComponentMask->DefaultR = ComponentMaskExpression->R;
							ComponentMask->DefaultG = ComponentMaskExpression->G;
							ComponentMask->DefaultB = ComponentMaskExpression->B;
							ComponentMask->DefaultA = ComponentMaskExpression->A;
						}
						else if (ParticleSubUVExpression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionTextureSampleParameterSubUV>(NewExpression)->Texture = ParticleSubUVExpression->Texture;
						}
						else if (ScalarParameterExpression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionConstant>(NewExpression)->R = ScalarParameterExpression->DefaultValue;
						}
						else if (VectorParameterExpression)
						{
							bNeedsRefresh = true;
							CastChecked<UMaterialExpressionConstant4Vector>(NewExpression)->Constant = VectorParameterExpression->DefaultValue;
						}

						if (bNeedsRefresh)
						{
							// Refresh the expression preview if we changed its properties after it was created
							NewExpression->bNeedToUpdatePreview = true;
							RefreshExpressionPreview( NewExpression, true );
							
							UpdateGenerator();
						}

						NodesToDelete.AddUnique(GraphNode);
						NodesToSelect.Add(NewGraphNode);
					}
				}
			}
		}

		// Delete the replaced nodes
		DeleteNodes(NodesToDelete);

		// Select each of the newly converted expressions
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
		for ( TArray<UEdGraphNode*>::TConstIterator NodeIter(NodesToSelect); NodeIter; ++NodeIter )
		{
				FocusedGraphEd->SetNodeSelection(*NodeIter, true);
			}
		}
	}
}

void FMaterialEditor::OnConvertTextures()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("MaterialEditorConvertTexture", "Material Editor: Convert to Texture") );
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };
		ModifyMaterial();
		TArray<class UEdGraphNode*> NodesToDelete;
		TArray<class UEdGraphNode*> NodesToSelect;

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				// Look for the supported classes to convert from
				UMaterialExpression* CurrentSelectedExpression = GraphNode->MaterialExpression;
				UMaterialExpressionTextureSample* TextureSampleExpression = Cast<UMaterialExpressionTextureSample>(CurrentSelectedExpression);
				UMaterialExpressionTextureObject* TextureObjectExpression = Cast<UMaterialExpressionTextureObject>(CurrentSelectedExpression);
				UMaterialExpressionSparseVolumeTextureSample* SparseVolumeTextureSampleExpression = Cast<UMaterialExpressionSparseVolumeTextureSample>(CurrentSelectedExpression);
				UMaterialExpressionSparseVolumeTextureObject* SparseVolumeTextureObjectExpression = Cast<UMaterialExpressionSparseVolumeTextureObject>(CurrentSelectedExpression);

				// Setup the class to convert to
				UClass* ClassToCreate = NULL;
				if (TextureSampleExpression)
				{
					ClassToCreate = UMaterialExpressionTextureObject::StaticClass();
				}
				else if (TextureObjectExpression)
				{
					ClassToCreate = UMaterialExpressionTextureSample::StaticClass();
				}
				else if (SparseVolumeTextureSampleExpression)
				{
					ClassToCreate = UMaterialExpressionSparseVolumeTextureObject::StaticClass();
				}
				else if (SparseVolumeTextureObjectExpression)
				{
					ClassToCreate = UMaterialExpressionSparseVolumeTextureSample::StaticClass();
				}

				if (ClassToCreate)
				{
					UMaterialExpression* NewExpression = CreateNewMaterialExpression(ClassToCreate, FVector2D(GraphNode->NodePosX, GraphNode->NodePosY), false, true);
					if (NewExpression)
					{
						UMaterialGraphNode* NewGraphNode = CastChecked<UMaterialGraphNode>(NewExpression->GraphNode);
						NewGraphNode->ReplaceNode(GraphNode);
						bool bNeedsRefresh = false;

						// Copy over expression-specific values
						if (TextureSampleExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionTextureObject* NewTextureExpr = CastChecked<UMaterialExpressionTextureObject>(NewExpression);
							NewTextureExpr->Texture = TextureSampleExpression->Texture;
							NewTextureExpr->AutoSetSampleType();
							NewTextureExpr->IsDefaultMeshpaintTexture = TextureSampleExpression->IsDefaultMeshpaintTexture;
						}
						else if (TextureObjectExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionTextureSample* NewTextureExpr = CastChecked<UMaterialExpressionTextureSample>(NewExpression);
							NewTextureExpr->Texture = TextureObjectExpression->Texture;
							NewTextureExpr->AutoSetSampleType();
							NewTextureExpr->IsDefaultMeshpaintTexture = TextureObjectExpression->IsDefaultMeshpaintTexture;
							NewTextureExpr->MipValueMode = TMVM_None;
						}
						else if (SparseVolumeTextureSampleExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionSparseVolumeTextureObject* NewTextureExpr = CastChecked<UMaterialExpressionSparseVolumeTextureObject>(NewExpression);
							NewTextureExpr->SparseVolumeTexture = SparseVolumeTextureSampleExpression->SparseVolumeTexture;
						}
						else if (SparseVolumeTextureObjectExpression)
						{
							bNeedsRefresh = true;
							UMaterialExpressionSparseVolumeTextureSample* NewTextureExpr = CastChecked<UMaterialExpressionSparseVolumeTextureSample>(NewExpression);
							NewTextureExpr->SparseVolumeTexture = SparseVolumeTextureObjectExpression->SparseVolumeTexture;
						}

						if (bNeedsRefresh)
						{
							// Refresh the expression preview if we changed its properties after it was created
							NewExpression->bNeedToUpdatePreview = true;
							RefreshExpressionPreview( NewExpression, true );
						}

						NodesToDelete.AddUnique(GraphNode);
						NodesToSelect.Add(NewGraphNode);
					}
				}
			}
		}

		// Delete the replaced nodes
		DeleteNodes(NodesToDelete);

		// Select each of the newly converted expressions
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
		for ( TArray<UEdGraphNode*>::TConstIterator NodeIter(NodesToSelect); NodeIter; ++NodeIter )
		{
				FocusedGraphEd->SetNodeSelection(*NodeIter, true);
			}
		}
	}
}

void FMaterialEditor::OnCollapseToFunction()
{
	FMaterialEditorHelpers::CollapseToFunction(*this);
}

bool FMaterialEditor::CanCollapseToFunction() const
{
	return CanCopyNodes();
}

void FMaterialEditor::OnExpandMaterialFunctionNode()
{
	FMaterialEditorHelpers::ExpandNode(*this);
}

bool FMaterialEditor::CanExpandMaterialFunctionNode() const
{
	// If any of the nodes can be expanded then we should allow expanding
	for (UObject* NodeObject : GetSelectedNodes())
	{
		UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(NodeObject);
		if (Node && Cast<UMaterialExpressionMaterialFunctionCall>(Node->MaterialExpression))
		{
			return true;
		}
	}
	return false;
}
void FMaterialEditor::OnSelectNamedRerouteDeclaration()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		const FGraphPanelSelectionSet SelectedNodes = FocusedGraphEd->GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
			FocusedGraphEd->ClearSelectionSet();
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				UMaterialExpression* CurrentSelectedExpression = GraphNode->MaterialExpression;
				UMaterialExpressionNamedRerouteUsage* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(CurrentSelectedExpression);
				if (Usage && Usage->Declaration)
				{
					UEdGraphNode* DeclarationGraphNode = Usage->Declaration->GraphNode;
					if (DeclarationGraphNode)
					{
							FocusedGraphEd->SetNodeSelection(DeclarationGraphNode, true);
						}
					}
				}
			}
			FocusedGraphEd->ZoomToFit(true);
		}
	}
}

void FMaterialEditor::SelectNamedRerouteUsages(UMaterialExpressionNamedRerouteDeclaration* Declaration)
{
	if (!Declaration)
	{
		return;
	}

	TArray<UEdGraphNode*> FoundGraphNodes;
	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		UMaterialExpressionNamedRerouteUsage* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(Expression);
		if (Usage && Usage->Declaration == Declaration)
		{
			if (UEdGraphNode* UsageGraphNode = Usage->GraphNode)
			{
				FoundGraphNodes.Add(UsageGraphNode);
			}
		}
	}

	if (FoundGraphNodes.Num())
	{
		// Spawn the tab in case the user doesn't have it open
		TabManager->TryInvokeTab(FMaterialEditorTabs::FindTabId);

		FindResults->PopulateSearchItems(FoundGraphNodes);
		FindResults->FocusForUse();
	}
}

void FMaterialEditor::OnSelectNamedRerouteUsages()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		const FGraphPanelSelectionSet SelectedNodes = FocusedGraphEd->GetSelectedNodes();
		if (SelectedNodes.Num() == 1)
		{
			FocusedGraphEd->ClearSelectionSet();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt))
				{
					if (UMaterialExpressionNamedRerouteDeclaration* Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(GraphNode->MaterialExpression))
					{
						SelectNamedRerouteUsages(Declaration);
					}
				}
			}
		}
	}
}

void FMaterialEditor::OnCreateRerouteUsageFromDeclaration()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		const FGraphPanelSelectionSet SelectedNodes = FocusedGraphEd->GetSelectedNodes();
		if (SelectedNodes.Num() == 1)
		{
			FocusedGraphEd->ClearSelectionSet();
			for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
			{
				UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
				if (GraphNode)
				{
					UMaterialExpression* CurrentSelectedExpression = GraphNode->MaterialExpression;
					UMaterialExpressionNamedRerouteDeclaration* Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(CurrentSelectedExpression);

					if (Declaration)
					{
						UMaterialExpressionNamedRerouteUsage* RerouteUsage = Cast<UMaterialExpressionNamedRerouteUsage>(FMaterialEditorUtilities::CreateNewMaterialExpression(
							GraphNode->GetGraph(),
							UMaterialExpressionNamedRerouteUsage::StaticClass(),
							FVector2D(GraphNode->NodePosX + 150, GraphNode->NodePosY),
							true,
							false));

						RerouteUsage->Declaration = Declaration;
						RerouteUsage->DeclarationGuid = Declaration->VariableGuid;
					}
				}
			}
		}
	}
}

void FMaterialEditor::OnConvertRerouteToNamedReroute()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		const FGraphPanelSelectionSet SelectedNodes = FocusedGraphEd->GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
			FocusedGraphEd->ClearSelectionSet();
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode_Knot* GraphNode = Cast<UMaterialGraphNode_Knot>(*NodeIt);
			if (GraphNode)
			{
				UEdGraph* Graph = GraphNode->GetGraph();
				const FScopedTransaction Transaction(LOCTEXT("ConvertRerouteToNamedReroute", "Convert reroute to named reroute"));
				Graph->Modify();

				auto* Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(FMaterialEditorUtilities::CreateNewMaterialExpression(
					Graph, 
					UMaterialExpressionNamedRerouteDeclaration::StaticClass(), 
					FVector2D(GraphNode->NodePosX - 50, GraphNode->NodePosY), 
					false, 
					true));
				if (!Declaration)
				{
					return;
				}

				UEdGraphPin* DeclarationInputPin = nullptr;
				for (auto* Pin : Declaration->GraphNode->GetAllPins())
				{
					if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
					{
						DeclarationInputPin = Pin;
						break;
					}
				}
				if (!ensure(DeclarationInputPin))
				{
					return;
				}

				for (auto* InputPin : GraphNode->GetInputPin()->LinkedTo)
				{
					InputPin->MakeLinkTo(DeclarationInputPin);
				}

				TArray<UEdGraphPin*> OutputPins = GraphNode->GetOutputPin()->LinkedTo;
				OutputPins.Sort([](UEdGraphPin& A, UEdGraphPin& B) { return A.GetOwningNode()->NodePosY < B.GetOwningNode()->NodePosY; });

				int Index = -OutputPins.Num() / 2;
				for (auto* OutputPin : OutputPins)
				{
					auto* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(FMaterialEditorUtilities::CreateNewMaterialExpression(
						Material->MaterialGraph,
						UMaterialExpressionNamedRerouteUsage::StaticClass(),
						FVector2D(GraphNode->NodePosX + 50, GraphNode->NodePosY + Index * 50),
						false,
						true));
					if (Usage)
					{
						Usage->Declaration = Declaration;
						Usage->DeclarationGuid = Declaration->VariableGuid;
						Usage->GraphNode->GetAllPins()[0]->MakeLinkTo(OutputPin); // usage node has a single pin
					}
					Index++;
				}
				GraphNode->DestroyNode();
					FocusedGraphEd->SetNodeSelection(Declaration->GraphNode, true);
				}
			}
		}
	}
}

void FMaterialEditor::OnConvertNamedRerouteToReroute()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		const FGraphPanelSelectionSet SelectedNodes = FocusedGraphEd->GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				UEdGraph* Graph = GraphNode->GetGraph();
				const FScopedTransaction Transaction(LOCTEXT("ConvertNamedRerouteToReroute", "Convert named reroute to reroute"));
				Graph->Modify();

				UMaterialExpression* CurrentSelectedExpression = GraphNode->MaterialExpression;
				UMaterialExpressionNamedRerouteDeclaration* Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(CurrentSelectedExpression);
				if (!Declaration)
				{
					UMaterialExpressionNamedRerouteUsage* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(CurrentSelectedExpression);
					if (Usage)
					{
						Declaration = Usage->Declaration;
					}
				}
				if (!Declaration)
				{
					return;
				}
				UEdGraphNode* DeclarationGraphNode = Declaration->GraphNode;
				FVector2D KnotPosition(DeclarationGraphNode->NodePosX + 50, DeclarationGraphNode->NodePosY);

				UMaterialExpression* Reroute = FMaterialEditorUtilities::CreateNewMaterialExpression(Graph, UMaterialExpressionReroute::StaticClass(), KnotPosition, false, true);
				auto KnotGraphNode = CastChecked<UMaterialGraphNode_Knot>(Reroute->GraphNode);

				for (UEdGraphPin* Pin : DeclarationGraphNode->GetAllPins())
				{
					if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
					{
						for (UEdGraphPin* InputPin : Pin->LinkedTo)
						{
							KnotGraphNode->GetInputPin()->MakeLinkTo(InputPin);
						}
					}
					if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
					{
						for (UEdGraphPin* OutputPin : Pin->LinkedTo)
						{
							KnotGraphNode->GetOutputPin()->MakeLinkTo(OutputPin);
						}
					}
				}
				DeclarationGraphNode->DestroyNode();

				for(UMaterialExpression* Expression : Material->GetExpressions())
				{
					auto* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(Expression);
					if (Usage && Usage->Declaration == Declaration)
					{
						UEdGraphNode* UsageGraphNode = Usage->GraphNode;
						if (UsageGraphNode)
						{
							UEdGraphPin* Pin = Usage->GraphNode->GetAllPins()[0]; // usage node has a single pin
							for (UEdGraphPin* OutputPin : Pin->LinkedTo)
							{
								KnotGraphNode->GetOutputPin()->MakeLinkTo(OutputPin);
							}
							UsageGraphNode->DestroyNode();
						}
					}
				}
			}
		}
	}
}
}

void FMaterialEditor::OnPreviewNode()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt))
			{
				PreviewNode(GraphNode);
			}
		}
	}
}

void FMaterialEditor::OnPreviewHoveredNode()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		if (SGraphPanel* GraphPanel = FocusedGraphEd->GetGraphPanel())
		{
			PreviewNode(Cast<UMaterialGraphNode>(GraphPanel->GetCurrentHoveredNode()));
		}
	}
}

void FMaterialEditor::PreviewNode(UMaterialGraphNode* InGraphNode)
{  
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
	SetPreviewExpression(InGraphNode ? InGraphNode->MaterialExpression : nullptr);
}

void FMaterialEditor::SetCustomPrimitiveData(TConstArrayView<float> CustomFloats)
{
	CustomPrimitiveDataFloats = CustomFloats;
	UpdatePreviewCustomPrimitiveData();
}

void FMaterialEditor::OnToggleRealtimePreview()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
			if (GraphNode)
			{
				UMaterialExpression* SelectedExpression = GraphNode->MaterialExpression;
				SelectedExpression->bRealtimePreview = !SelectedExpression->bRealtimePreview;

				if (SelectedExpression->bRealtimePreview)
				{
					SelectedExpression->bCollapsed = false;
				}

				RefreshExpressionPreviews();
				SetMaterialDirty();
			}
		}
	}
}

void FMaterialEditor::OnSelectDownstreamNodes()
{
	TArray<UMaterialGraphNode*> NodesToCheck;
	TArray<UMaterialGraphNode*> CheckedNodes;
	TArray<UMaterialGraphNode*> NodesToSelect;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
		if (GraphNode)
		{
			NodesToCheck.Add(GraphNode);
		}
	}

	while (NodesToCheck.Num() > 0)
	{
		UMaterialGraphNode* CurrentNode = NodesToCheck.Last();
		for (UEdGraphPin* Pin : CurrentNode->Pins)
		{
			if (Pin->Direction == EGPD_Output)
			{
				for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
				{
					UMaterialGraphNode* LinkedNode = Cast<UMaterialGraphNode>(Pin->LinkedTo[LinkIndex]->GetOwningNode());
					if (LinkedNode)
					{
						int32 FoundIndex = -1;
						CheckedNodes.Find(LinkedNode, FoundIndex);

						if (FoundIndex < 0)
						{
							NodesToSelect.Add(LinkedNode);
							NodesToCheck.Add(LinkedNode);
						}
					}
				}
			}
		}

		// This graph node has now been examined
		CheckedNodes.Add(CurrentNode);
		NodesToCheck.Remove(CurrentNode);
	}

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
	for (int32 Index = 0; Index < NodesToSelect.Num(); ++Index)
	{
			FocusedGraphEd->SetNodeSelection(NodesToSelect[Index], true);
		}
	}
}

void FMaterialEditor::OnSelectUpstreamNodes()
{
	TArray<UMaterialGraphNode*> NodesToCheck;
	TArray<UMaterialGraphNode*> CheckedNodes;
	TArray<UMaterialGraphNode*> NodesToSelect;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt);
		if (GraphNode)
		{
			NodesToCheck.Add(GraphNode);
		}
	}

	while (NodesToCheck.Num() > 0)
	{
		UMaterialGraphNode* CurrentNode = NodesToCheck.Last();
		for (UEdGraphPin* Pin : CurrentNode->Pins)
		{
			if (Pin->Direction == EGPD_Input)
			{
				for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
				{
					UMaterialGraphNode* LinkedNode = Cast<UMaterialGraphNode>(Pin->LinkedTo[LinkIndex]->GetOwningNode());
					if (LinkedNode)
					{
						int32 FoundIndex = -1;
						CheckedNodes.Find(LinkedNode, FoundIndex);

						if (FoundIndex < 0)
						{
							NodesToSelect.Add(LinkedNode);
							NodesToCheck.Add(LinkedNode);
						}
					}
				}
			}
		}

		// This graph node has now been examined
		CheckedNodes.Add(CurrentNode);
		NodesToCheck.Remove(CurrentNode);
	}

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
	for (int32 Index = 0; Index < NodesToSelect.Num(); ++Index)
	{
			FocusedGraphEd->SetNodeSelection(NodesToSelect[Index], true);
		}
	}
}

void FMaterialEditor::OnForceRefreshPreviews()
{
	ForceRefreshExpressionPreviews();
	RefreshPreviewViewport();
}

void FMaterialEditor::OnCreateComment()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		CreateNewMaterialExpressionComment(FocusedGraphEd->GetPasteLocation2f());
	}
}

void FMaterialEditor::OnCreateComponentMaskNode()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		CreateNewMaterialExpression(UMaterialExpressionComponentMask::StaticClass(), FocusedGraphEd->GetPasteLocation2f(), true, false);
	}
}

void FMaterialEditor::OnFindInMaterial()
{
	TabManager->TryInvokeTab(FMaterialEditorTabs::FindTabId);
	FindResults->FocusForUse();
}

void FMaterialEditor::OnGraphEditorFocused(const TSharedRef<class SGraphEditor>& InGraphEditor)
{
	if (FocusedGraphEdPtr == InGraphEditor)
	{
		return;
	}

	// Update the graph editor that is currently focused
	FocusedGraphEdPtr = InGraphEditor;

	// Refresh navigation history
	TSharedPtr<SWidget> TitleBar = FocusedGraphEdPtr.Pin()->GetTitleBar();
	StaticCastSharedPtr<SMaterialEditorTitleBar>(TitleBar)->RequestRefresh();

	// Update the inspector as well, to show selection from the focused graph editor
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (bHideUnrelatedNodes && SelectedNodes.Num() <= 0)
	{
		FocusedGraphEdPtr.Pin()->ResetAllNodesUnrelatedStates();
	}
}

void FMaterialEditor::OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	FocusedGraphEdPtr = nullptr;
}

UClass* FMaterialEditor::GetOnPromoteToParameterClass(const UEdGraphPin* TargetPin) const
{
	UMaterialGraphNode_Root* RootPinNode = Cast<UMaterialGraphNode_Root>(TargetPin->GetOwningNode());
	UMaterialGraphNode* OtherPinNode = Cast<UMaterialGraphNode>(TargetPin->GetOwningNode());

	if (RootPinNode != nullptr)
	{
		const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(RootPinNode->GetGraph());
		const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[TargetPin->SourceIndex];
		EMaterialProperty PropertyId = MaterialInput.GetProperty();
		
		switch (PropertyId)
		{
			case MP_Opacity:
			case MP_Metallic:
			case MP_Specular:
			case MP_Roughness:
			case MP_Anisotropy:
			case MP_CustomData0:
			case MP_CustomData1:
			case MP_AmbientOcclusion:
			case MP_Refraction:
			case MP_PixelDepthOffset:
			case MP_OpacityMask:
			case MP_SurfaceThickness:
			case MP_Displacement:
				return UMaterialExpressionScalarParameter::StaticClass();

			case MP_WorldPositionOffset:
			case MP_EmissiveColor:
			case MP_BaseColor:
			case MP_SubsurfaceColor:
			case MP_SpecularColor:
			case MP_Normal:
			case MP_Tangent:
				return UMaterialExpressionVectorParameter::StaticClass();

			case MP_ShadingModel:
				return UMaterialExpressionShadingModel::StaticClass();

			case MP_FrontMaterial:
				return nullptr;

		}
	}
	else if (OtherPinNode)
	{
		FName TargetPinName = OtherPinNode->GetShortenPinName(TargetPin->PinName);

		for (FExpressionInputIterator It{ OtherPinNode->MaterialExpression }; It; ++It)
		{
			FName InputName = OtherPinNode->MaterialExpression->GetInputName(It.Index);
			InputName = OtherPinNode->GetShortenPinName(InputName);

			if (InputName == TargetPinName)
			{
				switch (OtherPinNode->MaterialExpression->GetInputValueType(It.Index))
				{
					case MCT_Float1:
					case MCT_Float: return UMaterialExpressionScalarParameter::StaticClass();

					case MCT_Float2:
					case MCT_Float3:
					case MCT_Float4: return UMaterialExpressionVectorParameter::StaticClass();
					
					case MCT_StaticBool: return UMaterialExpressionStaticBoolParameter::StaticClass();

					case MCT_Texture2D:
					case MCT_TextureCube: 
					case MCT_VolumeTexture: 
					case MCT_Texture: return UMaterialExpressionTextureObjectParameter::StaticClass();

					case MCT_Substrate: return nullptr;
				}

				break;
			}
		}
	}

	return nullptr;
}

void FMaterialEditor::OnPromoteToParameter(const FToolMenuContext& InMenuContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
	const UEdGraphPin* TargetPin = NodeContext->Pin;
	UMaterialGraphNode_Base* PinNode = Cast<UMaterialGraphNode_Base>(TargetPin->GetOwningNode());

	FMaterialGraphSchemaAction_NewNode Action;	
	Action.MaterialExpressionClass = GetOnPromoteToParameterClass(TargetPin);

	if (Action.MaterialExpressionClass != nullptr)
	{
		check(PinNode);
		UEdGraph* GraphObj = PinNode->GetGraph();
		check(GraphObj);

		const FScopedTransaction Transaction(LOCTEXT("PromoteToParameter", "Promote To Parameter"));
		GraphObj->Modify();

		// Set position of new node to be close to node we clicked on
		FVector2f NewNodePos;
		NewNodePos.X = PinNode->NodePosX - 100.0f;
		NewNodePos.Y = PinNode->NodePosY;

		UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Action.PerformAction(GraphObj, const_cast<UEdGraphPin*>(TargetPin), NewNodePos));

		if (MaterialNode->MaterialExpression->HasAParameterName())
		{
			MaterialNode->MaterialExpression->SetParameterName(TargetPin->PinName);
			MaterialNode->MaterialExpression->ValidateParameterName(false);
		}
	}
	if (MaterialEditorInstance != nullptr)
	{
		SetEditorInstanceForWidgets();
	}
}

bool FMaterialEditor::OnCanResetToDefault(const FToolMenuContext& InMenuContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
	if (!NodeContext)
	{
		return false;
	}
	const UEdGraphPin* TargetPin = NodeContext->Pin;
	const UMaterialGraphNode_Root* RootPinNode = Cast<UMaterialGraphNode_Root>(TargetPin->GetOwningNode());
	if (RootPinNode != nullptr)
	{
		const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(RootPinNode->GetGraph());
		const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[TargetPin->SourceIndex];
		const EMaterialProperty PropertyId = MaterialInput.GetProperty();
		switch (PropertyId)
		{
			case MP_BaseColor:				
			case MP_Opacity:				
			case MP_Metallic:				
			case MP_Specular:				
			case MP_Roughness:				
			case MP_Anisotropy:				
			case MP_CustomData0:			
			case MP_CustomData1:			
			case MP_AmbientOcclusion:		
			case MP_Refraction:				
			case MP_OpacityMask:			
			case MP_SurfaceThickness:		
			case MP_Displacement:			
			case MP_WorldPositionOffset:	
			case MP_EmissiveColor:			
			case MP_SubsurfaceColor:		
			case MP_Normal:					
			case MP_Tangent:				
				return true;
				
			default:
				return false;
		}
	}
	return false;
}

void FMaterialEditor::OnResetToDefault(const FToolMenuContext& InMenuContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
	const int32 PinIndex = NodeContext->Pin->SourceIndex;
	const UMaterialGraphNode_Root* RootPinNode = Cast<UMaterialGraphNode_Root>(NodeContext->Pin->GetOwningNode());

	if (RootPinNode != nullptr)
	{
		UEdGraphPin* TargetPin = RootPinNode->GetPinAt(PinIndex);

		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ResetPinToDefault", "Reset Pin Value to its default" ) );
		TargetPin->Modify();
		
		const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(RootPinNode->GetGraph());
		const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[TargetPin->SourceIndex];
		const EMaterialProperty PropertyId = MaterialInput.GetProperty();
		UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();
		switch (PropertyId)
		{
			case MP_BaseColor:
				EditorOnlyData->BaseColor.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_BaseColor);
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->BaseColor.GetDefaultValue());
				break;

			case MP_Opacity:
				EditorOnlyData->Opacity.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Opacity).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Opacity.GetDefaultValue());
				break;
			
			case MP_Metallic:
				EditorOnlyData->Metallic.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Metallic).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Metallic.GetDefaultValue());
				break;
			
			case MP_Specular:
				EditorOnlyData->Specular.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Specular).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Specular.GetDefaultValue());
				break;
			
			case MP_Roughness:
				EditorOnlyData->Roughness.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Roughness).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Roughness.GetDefaultValue());
				break;
			
			case MP_Anisotropy:
				EditorOnlyData->Anisotropy.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Anisotropy).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Anisotropy.GetDefaultValue());
				break;
			
			case MP_CustomData0:
				EditorOnlyData->ClearCoat.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_CustomData0).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->ClearCoat.GetDefaultValue());
				break;
			
			case MP_CustomData1:
				EditorOnlyData->ClearCoatRoughness.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_CustomData1).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->ClearCoatRoughness.GetDefaultValue());
				break;
			
			case MP_AmbientOcclusion:
				EditorOnlyData->AmbientOcclusion.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_AmbientOcclusion).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->AmbientOcclusion.GetDefaultValue());
				break;
			
			case MP_Refraction:
				EditorOnlyData->Refraction.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Refraction).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Refraction.GetDefaultValue());
				break;
			
			case MP_OpacityMask:
				EditorOnlyData->OpacityMask.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_OpacityMask).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->OpacityMask.GetDefaultValue());
				break;
			
			case MP_SurfaceThickness:
				EditorOnlyData->SurfaceThickness.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_SurfaceThickness).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->SurfaceThickness.GetDefaultValue());
				break;
			
			case MP_Displacement:
				EditorOnlyData->Displacement.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Displacement).X;
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Displacement.GetDefaultValue());
				break;
			
			case MP_WorldPositionOffset:
				EditorOnlyData->WorldPositionOffset.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_WorldPositionOffset);
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->WorldPositionOffset.GetDefaultValue());
				break;
			
			case MP_EmissiveColor:
				EditorOnlyData->EmissiveColor.Constant = FLinearColor(FMaterialAttributeDefinitionMap::GetDefaultValue(MP_EmissiveColor));
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->EmissiveColor.GetDefaultValue());
				break;
			
			case MP_SubsurfaceColor:
				EditorOnlyData->SubsurfaceColor.Constant = FLinearColor(FMaterialAttributeDefinitionMap::GetDefaultValue(MP_SubsurfaceColor)); 
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->SubsurfaceColor.GetDefaultValue());
				break;
			
			case MP_Normal:
				EditorOnlyData->Normal.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Normal); 
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Normal.GetDefaultValue());
				break;
			
			case MP_Tangent:
				EditorOnlyData->Tangent.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Tangent);
				TargetPin->GetSchema()->TrySetDefaultValue(*TargetPin, EditorOnlyData->Tangent.GetDefaultValue());
				break;
			
			default:
				break;
		}
		FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(MaterialGraph);
	}
}

bool FMaterialEditor::OnCanDeletePin(const FToolMenuContext& InMenuContext) const
{
	if (UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>())
	{
		if (UEdGraphPin* TargetPin = const_cast<UEdGraphPin*>(NodeContext->Pin))
		{
			UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(TargetPin->GetOwningNode());
			if (MaterialGraphNode && MaterialGraphNode->MaterialExpression)
			{
				return MaterialGraphNode->MaterialExpression->CanDeletePin(TargetPin->Direction, TargetPin->SourceIndex);
			}
		}
	}

	return false;
}

void FMaterialEditor::OnDeletePin(const FToolMenuContext& InMenuContext) const
{
	if (UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>())
	{
		if (UEdGraphPin* TargetPin = const_cast<UEdGraphPin*>(NodeContext->Pin))
		{
			UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(TargetPin->GetOwningNode());
			if (MaterialGraphNode && MaterialGraphNode->MaterialExpression)
			{
				const FScopedTransaction Transaction(LOCTEXT("DeletePin", "Delete Pin"));
				MaterialGraphNode->MaterialExpression->Modify();
				MaterialGraphNode->MaterialExpression->DeletePin(TargetPin->Direction, TargetPin->SourceIndex);
			}
		}
	}
}

bool FMaterialEditor::OnCanPromoteToParameter(const FToolMenuContext& InMenuContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
	if (!NodeContext)
	{
		return false;
	}

	const UEdGraphPin* TargetPin = NodeContext->Pin;
	if (TargetPin && (TargetPin->Direction == EEdGraphPinDirection::EGPD_Input) && (TargetPin->LinkedTo.Num() == 0))
	{
		return GetOnPromoteToParameterClass(TargetPin) != nullptr;
	}

	return false;
}

void FMaterialEditor::OnCreateSubstrateNodeForPin(const FToolMenuContext& InMenuContext, ESubstrateNodeForPin NodeForPin) const
{
	UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
	const UEdGraphPin* TargetPin = NodeContext->Pin;
	UMaterialGraphNode_Base* PinNode = Cast<UMaterialGraphNode_Base>(TargetPin->GetOwningNode());
	const bool bTargetPinIsInput = TargetPin->Direction == EEdGraphPinDirection::EGPD_Input;

	FMaterialGraphSchemaAction_NewNode Action;
	Action.MaterialExpressionClass = UMaterialExpressionSubstrateSlabBSDF::StaticClass();
	if (NodeForPin == ESubstrateNodeForPin::HorizontalMix)
	{
		Action.MaterialExpressionClass = UMaterialExpressionSubstrateHorizontalMixing::StaticClass();
	}
	else if (NodeForPin == ESubstrateNodeForPin::VerticalLayer)
	{
		Action.MaterialExpressionClass = UMaterialExpressionSubstrateVerticalLayering::StaticClass();
	}
	else if (NodeForPin == ESubstrateNodeForPin::Weight)
	{
		Action.MaterialExpressionClass = UMaterialExpressionSubstrateWeight::StaticClass();
	}
	else if (NodeForPin == ESubstrateNodeForPin::Select)
	{
		Action.MaterialExpressionClass = UMaterialExpressionSubstrateSelect::StaticClass();
	}

	check(PinNode);
	UEdGraph* GraphObj = PinNode->GetGraph();
	check(GraphObj);

	const FScopedTransaction Transaction(LOCTEXT("CreateSubstrateNode", "Create Substrate Node"));
	GraphObj->Modify();

	// Set position of new node to be close to node we clicked on
	FVector2f NewNodePos;
	NewNodePos.X = PinNode->NodePosX + (bTargetPinIsInput ? -300 : 300);
	NewNodePos.Y = PinNode->NodePosY;


	if (bTargetPinIsInput)
	{
		UMaterialGraphNode* NewNode = Cast<UMaterialGraphNode>(Action.PerformAction(GraphObj, const_cast<UEdGraphPin*>(TargetPin), NewNodePos));
	}
	else
	{
		// Link manually
		UMaterialGraphNode* NewNode = Cast<UMaterialGraphNode>(Action.PerformAction(GraphObj, nullptr, NewNodePos));

		// From that direction, the node is never going to be a root node (a root node has no output we can connect from).
		UMaterialGraphNode* TargetPinNode = Cast<UMaterialGraphNode>(TargetPin->GetOwningNode());

		check(NewNode->MaterialExpression->GetInput(0) && TargetPin->SourceIndex < TargetPinNode->MaterialExpression->GetOutputs().Num());

		FName TargetPinName = TargetPinNode->MaterialExpression->GetOutputs()[TargetPin->SourceIndex].OutputName;
		UMaterialEditingLibrary::ConnectMaterialExpressions(TargetPinNode->MaterialExpression, TargetPinName.ToString(), NewNode->MaterialExpression, FString());

		Material->MaterialGraph->RebuildGraph();
	}

	if (MaterialEditorInstance != nullptr)
	{
		SetEditorInstanceForWidgets();
	}
}

bool FMaterialEditor::OnCanCreateSubstrateNodeForPin(const FToolMenuContext& InMenuContext, ESubstrateNodeForPin NodeForPin) const
{
	UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
	const UEdGraphPin* TargetPin = NodeContext->Pin;
	UMaterialGraphNode_Root* RootPinNode = Cast<UMaterialGraphNode_Root>(TargetPin->GetOwningNode());
	UMaterialGraphNode* OtherPinNode = Cast<UMaterialGraphNode>(TargetPin->GetOwningNode());

	if ((TargetPin->Direction == EEdGraphPinDirection::EGPD_Input) && (TargetPin->LinkedTo.Num() == 0))
	{
		return FSubstrateWidget::HasInputSubstrateType(TargetPin);
	}
	else if ((TargetPin->Direction == EEdGraphPinDirection::EGPD_Output) && (TargetPin->LinkedTo.Num() == 0) && NodeForPin != ESubstrateNodeForPin::Slab)
	{
		return FSubstrateWidget::HasOutputSubstrateType(TargetPin);
	}

	return false;
}

FString FMaterialEditor::GetDocLinkForSelectedNode()
{
	FString DocumentationLink;

	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		UObject* ObjectInSelection = *SelectedNodes.begin();
		if (ObjectInSelection != NULL)
		{
			UMaterialGraphNode* SelectedGraphNode = Cast<UMaterialGraphNode>(ObjectInSelection);
			FString DocLink = SelectedGraphNode->GetDocumentationLink();
			FString DocExcerpt = SelectedGraphNode->GetDocumentationExcerptName();

			DocumentationLink = FEditorClassUtils::GetDocumentationLinkFromExcerpt(DocLink, DocExcerpt);
		}
	}

	return DocumentationLink;
}

FString FMaterialEditor::GetDocLinkBaseUrlForSelectedNode()
{
	FString DocumentationLinkBaseUrl;

	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		UObject* ObjectInSelection = *SelectedNodes.begin();
		if (ObjectInSelection != NULL)
		{
			UMaterialGraphNode* SelectedGraphNode = Cast<UMaterialGraphNode>(ObjectInSelection);
			FString DocLink = SelectedGraphNode->GetDocumentationLink();
			FString DocExcerpt = SelectedGraphNode->GetDocumentationExcerptName();

			DocumentationLinkBaseUrl = FEditorClassUtils::GetDocumentationLinkBaseUrlFromExcerpt(DocLink, DocExcerpt);
		}
	}

	return DocumentationLinkBaseUrl;
}

void FMaterialEditor::OnGoToDocumentation()
{
	FString DocumentationLink = GetDocLinkForSelectedNode();
	FString DocumentationLinkBaseUrl = GetDocLinkBaseUrlForSelectedNode();
	if (!DocumentationLink.IsEmpty())
	{
		IDocumentation::Get()->Open(DocumentationLink, FDocumentationSourceInfo(TEXT("rightclick_matnode")), DocumentationLinkBaseUrl);
	}
}

bool FMaterialEditor::CanGoToDocumentation()
{
	FString DocumentationLink = GetDocLinkForSelectedNode();
	return !DocumentationLink.IsEmpty();
}

void FMaterialEditor::OnGoToDefinition()
{
	if (UMaterialGraphNode* Node = GetFirstSelectedGraphNode())
	{
		if (Node->CanJumpToDefinition())
		{
			Node->JumpToDefinition();
		}
	}
}

bool FMaterialEditor::CanGoToDefinition() const
{
	if (UMaterialGraphNode* Node = GetFirstSelectedGraphNode())
	{
		return Node->CanJumpToDefinition();
	}

	return false;
}

void FMaterialEditor::RenameAssetFromRegistry(const FAssetData& InAddedAssetData, const FString& InNewName)
{
	// Grab the asset class, it will be checked for being a material function.
	UClass* Asset = FindObject<UClass>(InAddedAssetData.AssetClassPath);

	if(Asset->IsChildOf(UMaterialFunction::StaticClass()))
	{
		ForceRefreshExpressionPreviews();
	}
}

void FMaterialEditor::OnMaterialUsageFlagsChanged(UMaterial* MaterialThatChanged, int32 FlagThatChanged)
{
	EMaterialUsage Flag = static_cast<EMaterialUsage>(FlagThatChanged);

	// Update only if a modification has been done to the original material by the user and guard when Editor
	// is opened for MF, as we don't have an original material there, thus OriginalMaterial == Material 
	// and Material can be modified and received here, depending on which Preview Mesh is selected
	if((MaterialThatChanged == OriginalMaterial) && !MaterialFunction)
	{
		Material->SetUsageByFlag(Flag, OriginalMaterial->GetUsageByFlag(Flag));
		UpdatePreviewMaterial();
	}
}

void GatherMaterialsToOverride(const TObjectPtr<UMaterialFunction>& MaterialFunction, const TObjectPtr<UMaterial>& Material, TObjectPtr<UMaterial> OriginalMaterial, TArray<UMaterial*>& OutMaterialsToOverride)
{
	if (MaterialFunction)
	{
		// Find all materials that reference this function
		for (TObjectIterator<UMaterial> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterial* CurrentMaterial = *It;
			if (CurrentMaterial != Material)
			{
				bool bUpdate = false;
				for (int32 FunctionIndex = 0; FunctionIndex < CurrentMaterial->GetCachedExpressionData().FunctionInfos.Num(); FunctionIndex++)
				{
					if (CurrentMaterial->GetCachedExpressionData().FunctionInfos[FunctionIndex].Function == MaterialFunction->ParentFunction)
					{
						bUpdate = true;
						break;
					}
				}

				if (bUpdate)
				{
					OutMaterialsToOverride.Add(CurrentMaterial);
				}
			}
		}
	}
	else
	{
		// When a parameter changes...
		// Update the preview material.
		OutMaterialsToOverride.Add(Material);
		// The original Material we are editing.
		OutMaterialsToOverride.Add(OriginalMaterial);
	}
}

void FMaterialEditor::SetNumericParameterDefaultOnDependentMaterials(EMaterialParameterType Type, FName ParameterName, const UE::Shader::FValue& Value)
{
	TArray<UMaterial*> MaterialsToOverride;
	GatherMaterialsToOverride(MaterialFunction, Material, OriginalMaterial, MaterialsToOverride);

	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform_Checked(GEditor->GetEditorWorldContext().World()->GetFeatureLevel());
	for (int32 MaterialIndex = 0; MaterialIndex < MaterialsToOverride.Num(); MaterialIndex++)
	{
		UMaterial* CurrentMaterial = MaterialsToOverride[MaterialIndex];
		CurrentMaterial->SetNumericParameterDefault(Type, ParameterName, Value, ShaderPlatform);
	}

	// Update MI's that reference any of the materials affected
	for (TObjectIterator<UMaterialInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UMaterialInstance* CurrentMaterialInstance = *It;

		// Only care about MI's with static parameters, because we are overriding parameter defaults, 
		// And only MI's with static parameters contain uniform expressions, which contain parameter defaults
		if (CurrentMaterialInstance->bHasStaticPermutationResource)
		{
			UMaterial* BaseMaterial = CurrentMaterialInstance->GetMaterial();
			if (MaterialsToOverride.Contains(BaseMaterial))
			{
				CurrentMaterialInstance->SetNumericParameterDefault(Type, ParameterName, Value, ShaderPlatform);
			}
		}
	}
}

void FMaterialEditor::ClearNumericParameterDefaultOnDependentMaterials(EMaterialParameterType Type, FName ParameterName)
{
	TArray<UMaterial*> MaterialsToOverride;
	GatherMaterialsToOverride(MaterialFunction, Material, OriginalMaterial, MaterialsToOverride);

	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform_Checked(GEditor->GetEditorWorldContext().World()->GetFeatureLevel());
	for (int32 MaterialIndex = 0; MaterialIndex < MaterialsToOverride.Num(); MaterialIndex++)
	{
		UMaterial* CurrentMaterial = MaterialsToOverride[MaterialIndex];
		CurrentMaterial->ClearNumericParameterDefault(Type, ParameterName, ShaderPlatform);
	}

	// Update MI's that reference any of the materials affected
	for (TObjectIterator<UMaterialInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UMaterialInstance* CurrentMaterialInstance = *It;

		// Only care about MI's with static parameters, because we are overriding parameter defaults, 
		// And only MI's with static parameters contain uniform expressions, which contain parameter defaults
		if (CurrentMaterialInstance->bHasStaticPermutationResource)
		{
			UMaterial* BaseMaterial = CurrentMaterialInstance->GetMaterial();
			if (MaterialsToOverride.Contains(BaseMaterial))
			{
				CurrentMaterialInstance->ClearNumericParameterDefault(Type, ParameterName, ShaderPlatform);
			}
		}
	}
}

void FMaterialEditor::OnNumericParameterDefaultChanged(class UMaterialExpression* Expression, EMaterialParameterType Type, FName ParameterName, const UE::Shader::FValue& Value)
{
	check(Expression);

	if (Expression->Material == Material && OriginalMaterial)
	{
		SetNumericParameterDefaultOnDependentMaterials(Type, ParameterName, Value);
		OverriddenNumericParametersToRevert.Add(MakeTuple(Type, ParameterName));
	}

	OnParameterDefaultChanged();
}

void FMaterialEditor::OnParameterDefaultChanged()
{
	// Brute force all flush virtual textures if this material writes to any runtime virtual texture.
	if (Material->WritesToRuntimeVirtualTexture())
	{
		ENQUEUE_RENDER_COMMAND(FlushVTCommand)([](FRHICommandListImmediate& RHICmdList)
		{
			GetRendererModule().FlushVirtualTextureCache(); 
		});
	}
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_Preview(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		.Label(LOCTEXT("ViewportTabTitle", "Viewport"))
		[
			SNew( SOverlay )
			+ SOverlay::Slot()
			[
				PreviewViewport.ToSharedRef()
			]
			+ SOverlay::Slot()
			[
				PreviewUIViewport.ToSharedRef()
			]
			+ SOverlay::Slot()
			.Padding(8)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(SNotificationBackground)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(6.f)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SAssignNew(NumShadersText, STextBlock)
					]
				]
			]
		];

	NumShadersText->SetVisibility(MaterialFunction ? EVisibility::Collapsed : EVisibility::Visible);

	PreviewViewport->OnAddedToTab( SpawnedTab );

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_MaterialProperties(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.Label( LOCTEXT("MaterialDetailsTitle", "Details") )
		[
			MaterialDetailsView.ToSharedRef()
		];

	if (FocusedGraphEdPtr.IsValid())
	{
		// Since we're initialising, make sure nothing is selected
		FocusedGraphEdPtr.Pin()->ClearSelectionSet();
	}
	SpawnedDetailsTab = DetailsTab;
	return DetailsTab.ToSharedRef();
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_Palette(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == FMaterialEditorTabs::PaletteTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialPaletteTitle", "Palette"))
		[
			SNew( SBox )
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MaterialPalette")))
			[
				Palette.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_NamedReroutes(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FMaterialEditorTabs::NamedReroutesTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialNamedReroutesTitle", "Named Reroutes"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MaterialNamedReroutes")))
			[
				NamedReroutesPanel.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_Find(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FMaterialEditorTabs::FindTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialFindTitle", "Find Results"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MaterialFind")))
			[
				FindResults.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FMaterialEditorTabs::PreviewSettingsTabId);

	TSharedRef<SWidget> InWidget = SNullWidget::NullWidget;
	if (PreviewViewport.IsValid())
	{
		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		InWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewViewport->GetPreviewScene());
	}

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		[
			SNew(SBox)
			[
				InWidget
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_ParameterDefaults(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("Parameters", "Parameters"))
		[
			SNew(SBox)
			[
				MaterialParametersOverviewWidget.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_CustomPrimitiveData(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("CustomPrimitiveData", "Custom Primitive Data"))
		[
			SNew(SBox)
			[
				MaterialCustomPrimitiveDataWidget.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_LayerProperties(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialLayerPropertiesTitle", "Layer Parameter Preview"))
		[
			SNew(SBorder)
			.Padding(4)
			[
				MaterialLayersFunctionsInstance.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_Substrate(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FMaterialEditorTabs::SubstrateTabId);

	TSharedRef<SDockTab> SubstrateTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialSubstrateTabTitle", "Substrate"))
		[
			SNew(SBox)
			[
				SubstrateWidget.ToSharedRef()
			]
		];

	return SubstrateTab;
}

TSharedRef<SDockTab> FMaterialEditor::SpawnTab_MaterialExpressionsOverview(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FMaterialEditorTabs::MaterialOverviewTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialOverviewTabTitle", "Nodes Overview"))
		[
			SNew(SBox)
			[
				MaterialExpressionsOverviewWidget.ToSharedRef()
			]
		];

	return SpawnedTab;
}

void FMaterialEditor::SetPreviewExpression(UMaterialExpression* NewPreviewExpression)
{
	UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(NewPreviewExpression);

	// If the NewPreviewExpression is null or the same as existing, stop previewing
	if (!NewPreviewExpression || PreviewExpression == NewPreviewExpression)
	{
		if (FunctionOutput)
		{
			FunctionOutput->bLastPreviewed = false;
		}
		// If we are already previewing the selected expression toggle previewing off
		PreviewExpression = nullptr;

		if (ExpressionPreviewMaterial)
		{
			ExpressionPreviewMaterial->GetExpressionCollection().Empty();
		}
		
		// Set Preview Material to the material we're working on
		SetPreviewMaterial(Material);
		
		// Recompile the preview material to get changes that might have been made during previewing
		UpdatePreviewMaterial();
	}
	else
	{
		if (ExpressionPreviewMaterial == nullptr)
		{
			// Create the expression preview material if it hasnt already been created
			ExpressionPreviewMaterial = NewObject<UPreviewMaterial>(GetTransientPackage(), NAME_None, RF_Public | RF_Transactional);
			ExpressionPreviewMaterial->bIsPreviewMaterial = true;
			ExpressionPreviewMaterial->bEnableNewHLSLGenerator = Material->IsUsingNewHLSLGenerator();
			if (Material->IsUIMaterial())
			{
				ExpressionPreviewMaterial->MaterialDomain = MD_UI;
			}
			else if (Material->IsPostProcessMaterial())
			{
				ExpressionPreviewMaterial->MaterialDomain = MD_PostProcess;
			}
		}

		if (FunctionOutput)
		{
			FunctionOutput->bLastPreviewed = true;
		}
		else
		{
			//Hooking up the output of the break expression doesn't make much sense, preview the expression feeding it instead.
			UMaterialExpressionBreakMaterialAttributes* BreakExpr = Cast<UMaterialExpressionBreakMaterialAttributes>(NewPreviewExpression);
			if( BreakExpr && BreakExpr->GetInput(0) && BreakExpr->GetInput(0)->Expression )
			{
				NewPreviewExpression = BreakExpr->GetInput(0)->Expression;
			}
		}

		// The expression preview material's expressions array must stay up to date before recompiling 
		// So that RebuildMaterialFunctionInfo will see all the nested material functions that may need to be updated
		ExpressionPreviewMaterial->AssignExpressionCollection(Material->GetExpressionCollection());

		// The preview window should now show the expression preview material
		SetPreviewMaterial( ExpressionPreviewMaterial );

		// Set the preview expression
		PreviewExpression = NewPreviewExpression;

		// Recompile the preview material
		UpdatePreviewMaterial();
	}
}

void FMaterialEditor::JumpToNode(const UEdGraphNode* Node)
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		if (Node->GetGraph() != FocusedGraphEd->GetCurrentGraph())
		{
			JumpToHyperlink(Node->GetGraph());

			// Graph changed so editor changed, use new one to jump
			FocusedGraphEdPtr.Pin()->JumpToNode(Node, false);
		}
		else
		{
			FocusedGraphEd->JumpToNode(Node, false);
		}
	}
}

bool FMaterialEditor::FindOpenTabsContainingDocument(const UObject* DocumentID, TArray<TSharedPtr<SDockTab>>& Results)
{
	int32 StartingCount = Results.Num();

	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);

	DocumentManager->FindMatchingTabs(Payload, /*inout*/ Results);

	// Did we add anything new?
	return (StartingCount != Results.Num());
}

TSharedPtr<SDockTab> FMaterialEditor::OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	TSharedPtr<SDockTab> DocumentTab = DocumentManager->OpenDocument(Payload, Cause);

	return DocumentTab;
}

void FMaterialEditor::CloseDocumentTab(const UObject* DocumentID)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	DocumentManager->CloseTab(Payload);
}

UMaterialExpression* FMaterialEditor::CreateNewMaterialExpression(UClass* NewExpressionClass, const FVector2D& NodePos, bool bAutoSelect, bool bAutoAssignResource, const UEdGraph* Graph)
{
	UMaterialExpression* DefaultExpression = CastChecked<UMaterialExpression>(NewExpressionClass->GetDefaultObject());
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	UMaterialGraph* ExpressionGraph = Graph ? ToRawPtr(CastChecked<UMaterialGraph>(const_cast<UEdGraph*>(Graph))) : ToRawPtr(Material->MaterialGraph); 
	ExpressionGraph->Modify();

	if (!DefaultExpression->IsAllowedIn(OriginalMaterialObject))
	{
		UE_LOGF(LogMaterialEditor, Warning, "Trying to create a disallowed material expression of type %ls.", *NewExpressionClass->GetName());
		return NULL;
	}

	// Clear the selection
	if ( bAutoSelect && FocusedGraphEd )
	{
		FocusedGraphEd->ClearSelectionSet();
	}

	// Create the new expression.
	UMaterialExpression* NewExpression = NULL;
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MaterialEditorNewExpression", "Material Editor: New Expression") );
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };
		ModifyMaterial();

		UObject* SelectedAsset = nullptr;
		if (bAutoAssignResource)
		{
			// Load selected assets
			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
			SelectedAsset = GEditor->GetSelectedObjects()->GetTop<UObject>();
		}

		NewExpression = UMaterialEditingLibrary::CreateMaterialExpressionEx(Material, MaterialFunction, NewExpressionClass, SelectedAsset, NodePos.X, NodePos.Y);

		if (NewExpression)
		{
			ExpressionGraph->AddExpression(NewExpression, bAutoSelect);

			NewExpression->GraphNode->SnapToGrid(SNodePanel::GetSnapGridSize());
			NewExpression->MaterialExpressionEditorX = NewExpression->GraphNode->NodePosX;
			NewExpression->MaterialExpressionEditorY = NewExpression->GraphNode->NodePosY;

			// Select the new node.
			if ( bAutoSelect && FocusedGraphEd )
			{
				FocusedGraphEd->SetNodeSelection(NewExpression->GraphNode, true);
			}
		}
	}

	RegenerateCodeView();

	// Update the current preview material.
	UpdatePreviewMaterial();
	Material->MarkPackageDirty();

	RefreshExpressionPreviews();
	if (FocusedGraphEd)
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
	SetMaterialDirty();
	return NewExpression;
}

UMaterialExpressionComposite* FMaterialEditor::CreateNewMaterialExpressionComposite(const FVector2D& NodePos, const UEdGraph* Graph)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	UMaterialGraph* ExpressionGraph = Graph ? ToRawPtr(CastChecked<UMaterialGraph>(const_cast<UEdGraph*>(Graph))) : ToRawPtr(Material->MaterialGraph);
	ExpressionGraph->Modify();

	UMaterialExpressionComposite* NewComposite = nullptr;
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorNewComposite", "Material Editor: New Composite"));
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };

		ModifyMaterial();

		UObject* ExpressionOuter = Material;
		if (MaterialFunction)
		{
			ExpressionOuter = MaterialFunction;
		}

		UMaterialExpression* NewExpression = UMaterialEditingLibrary::CreateMaterialExpressionEx(Material, MaterialFunction, UMaterialExpressionComposite::StaticClass(), nullptr, NodePos.X, NodePos.Y);
		NewComposite = Cast<UMaterialExpressionComposite>(NewExpression);

		if (NewComposite)
		{
			UMaterialExpression* InputPinBase = UMaterialEditingLibrary::CreateMaterialExpressionEx(Material, MaterialFunction, UMaterialExpressionPinBase::StaticClass());
			NewComposite->InputExpressions = Cast<UMaterialExpressionPinBase>(InputPinBase);
			NewComposite->InputExpressions->PinDirection = EGPD_Output;
			NewComposite->InputExpressions->SubgraphExpression = NewComposite;

			UMaterialExpression* OutputPinBase = UMaterialEditingLibrary::CreateMaterialExpressionEx(Material, MaterialFunction, UMaterialExpressionPinBase::StaticClass());
			NewComposite->OutputExpressions = Cast<UMaterialExpressionPinBase>(OutputPinBase);
			NewComposite->OutputExpressions->PinDirection = EGPD_Input;
			NewComposite->OutputExpressions->SubgraphExpression = NewComposite;

			// Create graph node, subgraph, and pinbase graph nodes
			{
				ExpressionGraph->AddExpression(NewComposite, true);

				UMaterialGraphNode_Composite* CompositeNode = CastChecked<UMaterialGraphNode_Composite>(NewComposite->GraphNode);
				CompositeNode->BoundGraph = ExpressionGraph->AddSubGraph(NewComposite);
				CompositeNode->BoundGraph->Rename(*CastChecked<UMaterialExpressionComposite>(CompositeNode->MaterialExpression)->SubgraphName);

				CompositeNode->BoundGraph->AddExpression(InputPinBase, false);
				CompositeNode->BoundGraph->AddExpression(OutputPinBase, false);
			}

			if (FocusedGraphEd)
			{
				FocusedGraphEd->ClearSelectionSet();
				FocusedGraphEd->SetNodeSelection(NewComposite->GraphNode, true);
			}
		}
	}

	RefreshExpressionPreviews();
	if (FocusedGraphEd)
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
	SetMaterialDirty();
	return NewComposite;
}

UMaterialExpressionComment* FMaterialEditor::CreateNewMaterialExpressionComment(const FVector2D& NodePos, const UEdGraph* Graph)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	UMaterialGraph* ExpressionGraph = Graph ? ToRawPtr(CastChecked<UMaterialGraph>(const_cast<UEdGraph*>(Graph))) : ToRawPtr(Material->MaterialGraph);
	ExpressionGraph->Modify();

	UMaterialExpressionComment* NewComment = NULL;
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorCreateComment", "Material Editor: Create comment"));
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };
		ModifyMaterial();

		UObject* ExpressionOuter = Material;
		if (MaterialFunction)
		{
			ExpressionOuter = MaterialFunction;
		}

		NewComment = NewObject<UMaterialExpressionComment>(ExpressionOuter, NAME_None, RF_Transactional);

		// Add to the list of comments associated with this material.
		Material->GetExpressionCollection().AddComment( NewComment );

		FSlateRect Bounds;
		if (FocusedGraphEd && FocusedGraphEd->GetBoundsForSelectedNodes(Bounds, 50.0f))
		{
			NewComment->MaterialExpressionEditorX = Bounds.Left;
			NewComment->MaterialExpressionEditorY = Bounds.Top;

			FVector2D Size = Bounds.GetSize();
			NewComment->SizeX = Size.X;
			NewComment->SizeY = Size.Y;
		}
		else
		{

			NewComment->MaterialExpressionEditorX = NodePos.X;
			NewComment->MaterialExpressionEditorY = NodePos.Y;
			NewComment->SizeX = 400;
			NewComment->SizeY = 100;
		}

		NewComment->Text = NSLOCTEXT("K2Node", "CommentBlock_NewEmptyComment", "Comment").ToString();
		NewComment->CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

		ExpressionGraph->AddComment(NewComment, true);

		NewComment->GraphNode->SnapToGrid(SNodePanel::GetSnapGridSize());
		NewComment->MaterialExpressionEditorX = NewComment->GraphNode->NodePosX;
		NewComment->MaterialExpressionEditorY = NewComment->GraphNode->NodePosY;

		// Select the new comment.
		if (FocusedGraphEd)
		{
			FocusedGraphEd->ClearSelectionSet();
			FocusedGraphEd->SetNodeSelection(NewComment->GraphNode, true);
		}
	}

	Material->MarkPackageDirty();
	if (FocusedGraphEd)
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
	SetMaterialDirty();
	return NewComment;
}

void FMaterialEditor::ForceRefreshExpressionPreviews()
{
	// Initialize expression previews.
	const bool bOldAlwaysRefreshAllPreviews = bAlwaysRefreshAllPreviews;
	bAlwaysRefreshAllPreviews = true;
	RefreshExpressionPreviews();
	bAlwaysRefreshAllPreviews = bOldAlwaysRefreshAllPreviews;
}

void FMaterialEditor::AddToSelection(UMaterialExpression* Expression)
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		FocusedGraphEd->SetNodeSelection(Expression->GraphNode, true);
	}
}

void FMaterialEditor::JumpToExpression(UMaterialExpression* Expression)
{
	check(Expression);
	UEdGraphNode* ExpressionNode = nullptr;

	// Note: 'Expression' may be from a serialized material with no graph, we compare to our material with a graph if this occurs
	if (Expression->GraphNode)
	{
		ExpressionNode = Expression->GraphNode;
	}
	else if (Expression->bIsParameterExpression)
	{
		TArray<UMaterialExpression*>* GraphExpressions = Material->EditorParameters.Find(Expression->GetParameterName());
		if (GraphExpressions && GraphExpressions->Num() == 1)
		{
			ExpressionNode = GraphExpressions->Last()->GraphNode;
		}
		else
		{
			UMaterialExpressionParameter* GraphExpression = Material->FindExpressionByGUID<UMaterialExpressionParameter>(Expression->GetParameterExpressionId());
			ExpressionNode = GraphExpression ? ToRawPtr(GraphExpression->GraphNode) : nullptr;
		}
	}
	else if (UMaterialExpressionFunctionOutput* ExpressionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression))
	{
		TArray<UMaterialExpressionFunctionOutput*> FunctionOutputExpressions;
		Material->GetAllFunctionOutputExpressions(FunctionOutputExpressions);
		UMaterialExpressionFunctionOutput** GraphExpression = FunctionOutputExpressions.FindByPredicate(
			[&](const UMaterialExpressionFunctionOutput* GraphExpressionOutput) 
			{
				return GraphExpressionOutput->Id == ExpressionOutput->Id;
			});
		ExpressionNode = GraphExpression ? ToRawPtr((*GraphExpression)->GraphNode) : nullptr;
	}

	if (ExpressionNode)
	{
		JumpToNode(ExpressionNode);
	}
}

void FMaterialEditor::ModifyMaterial()
{
	Material->MaterialGraph->Modify();
	
	Material->Modify();
	Material->GetEditorOnlyData()->Modify();

	if (MaterialFunction)
	{
		MaterialFunction->Modify();
		MaterialFunction->GetEditorOnlyData()->Modify();
	}
}

void FMaterialEditor::SelectAllNodes()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		FocusedGraphEd->SelectAllNodes();
	}
}

bool FMaterialEditor::CanSelectAllNodes() const
{
	return FocusedGraphEdPtr.IsValid();
}

void FMaterialEditor::DeleteSelectedNodes()
{
	DeleteSelectedNodes(true);
}

void FMaterialEditor::DeleteSelectedNodes(bool bShowConfirmation)
{
	TArray<UEdGraphNode*> NodesToDelete;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		NodesToDelete.Add(CastChecked<UEdGraphNode>(*NodeIt));
	}
	
	DeleteNodes(NodesToDelete, bShowConfirmation);
}

void FMaterialEditor::DeleteNodes(const TArray<UEdGraphNode*>& NodesToDelete)
{
	DeleteNodes(NodesToDelete, true);
}

void FMaterialEditor::DeleteNodes(const TArray<UEdGraphNode*>& NodesToDelete, bool bShowConfirmation)
{
	if (NodesToDelete.Num() > 0)
	{
		if (bShowConfirmation && !CheckExpressionRemovalWarnings(NodesToDelete))
		{
			return;
		}

		// If we are previewing an expression and the expression being previewed was deleted
		bool bHaveExpressionsToDelete			= false;
		bool bPreviewExpressionDeleted			= false;

		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MaterialEditorDelete", "Material Editor: Delete") );
			DeleteNodesInternal(NodesToDelete, bHaveExpressionsToDelete, bPreviewExpressionDeleted);
			Material->MaterialGraph->LinkMaterialExpressionsFromGraph();
		} // ScopedTransaction

		// Deselect all expressions and comments.
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			FocusedGraphEd->ClearSelectionSet();
			FocusedGraphEd->NotifyGraphChanged();
		}

		if ( bHaveExpressionsToDelete )
		{
			if( bPreviewExpressionDeleted )
			{
				// The preview expression was deleted.  Null out our reference to it and reset to the normal preview material
				SetPreviewExpression(nullptr);
			}
			RegenerateCodeView();
		}

		InvalidateSubstrateConversionVersion(Material);
		UpdatePreviewMaterial();
		Material->MarkPackageDirty();
		SetMaterialDirty();

		if ( bHaveExpressionsToDelete )
		{
			RefreshExpressionPreviews();
		}
	}
}

bool FMaterialEditor::CanDeleteNodes() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	bool bDeletableNodeExists = false;

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* GraphNode = Cast<UEdGraphNode>(*NodeIt);
		if (GraphNode && GraphNode->CanUserDeleteNode())
		{
			bDeletableNodeExists = true;
		}
	}

	return SelectedNodes.Num() > 0 && bDeletableNodeExists;
}

void FMaterialEditor::DeleteSelectedDuplicatableNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	// Cache off the old selection
	const FGraphPanelSelectionSet OldSelectedNodes = GetSelectedNodes();

	// Clear the selection and only select the nodes that can be duplicated
	FGraphPanelSelectionSet RemainingNodes;
	FocusedGraphEd->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			FocusedGraphEd->SetNodeSelection(Node, true);
		}
		else
		{
			RemainingNodes.Add(Node);
		}
	}

	// Delete the duplicatable nodes
	DeleteSelectedNodes();

	// Reselect whatever's left from the original selection after the deletion
	FocusedGraphEd->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(RemainingNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			FocusedGraphEd->SetNodeSelection(Node, true);
		}
	}
}

void FMaterialEditor::DeleteNodesInternal(const TArray<class UEdGraphNode*>& NodesToDelete, bool& bHaveExpressionsToDelete, bool& bPreviewExpressionDeleted, bool bForceDeletion)
{
	FScopedMaterialChecker Checker{ this, __FUNCTION__ };
	ModifyMaterial();

	for (int32 Index = 0; Index < NodesToDelete.Num(); ++Index)
	{
		if (bForceDeletion || NodesToDelete[Index]->CanUserDeleteNode())
		{
			if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(NodesToDelete[Index]))
			{
				// Break all node links first so that we don't update the material before deleting
				GraphNode->Modify();
				GraphNode->BreakAllNodeLinks();

				UMaterialExpression* MaterialExpression = GraphNode->MaterialExpression;

				bHaveExpressionsToDelete = true;

				DestroyColorPicker();

				if( PreviewExpression == MaterialExpression )
				{
					// The expression being previewed is also being deleted
					bPreviewExpressionDeleted = true;
				}

				if (UMaterialGraphNode_PinBase* PinBaseNode = Cast<UMaterialGraphNode_PinBase>(GraphNode))
				{
					UMaterialExpressionPinBase* PinBase = CastChecked<UMaterialExpressionPinBase>(MaterialExpression);
					PinBase->DeleteReroutePins();
				}
						
				if (UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(GraphNode))
				{
					CloseDocumentTab(CompositeNode->BoundGraph);

					UMaterialExpressionComposite* SubgraphComposite = CastChecked<UMaterialExpressionComposite>(MaterialExpression);
					SubgraphComposite->Modify();

					// Remove all subgraph nodes, note that composites remove their subgraph in DestroyNode
					const TArray<UEdGraphNode*> SubgraphNodesToDelete = CompositeNode->BoundGraph->Nodes;
					DeleteNodesInternal(SubgraphNodesToDelete, bHaveExpressionsToDelete, bPreviewExpressionDeleted, true);
				}
				if(MaterialExpression)
				{
					MaterialExpression->Modify();
					Material->GetExpressionCollection().RemoveExpression( MaterialExpression );
					Material->RemoveExpressionParameter(MaterialExpression);
					// Make sure the deleted expression is caught by gc
					MaterialExpression->MarkAsGarbage();
				}
			}
			else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(NodesToDelete[Index]))
			{
				CommentNode->Modify();
				CommentNode->BreakAllNodeLinks();
				CommentNode->MaterialExpressionComment->Modify();
				Material->GetExpressionCollection().RemoveComment( CommentNode->MaterialExpressionComment );
			}

			// Now that we are done with the node, remove it
			FBlueprintEditorUtils::RemoveNode(NULL, NodesToDelete[Index], true);
		}
	}

	SetEditorInstanceForWidgets();
}

void FMaterialEditor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FString Buffer = CopyNodesToBuffer(GetSelectedNodes());
	FPlatformApplicationMisc::ClipboardCopy(*Buffer);
}

FString FMaterialEditor::CopyNodesToBuffer(const FGraphPanelSelectionSet& Nodes)
{

	FString ExportedText;

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(Nodes); SelectedIter; ++SelectedIter)
	{
		if(UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			Node->PrepareForCopying();
		}
	}

	FEdGraphUtilities::ExportNodesToText(Nodes, /*out*/ ExportedText);

	// Make sure Material remains the owner of the copied nodes
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(Nodes); SelectedIter; ++SelectedIter)
	{
		if (UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(*SelectedIter))
		{
			Node->PostCopyNode();
		}
		else if (UMaterialGraphNode_Comment* Comment = Cast<UMaterialGraphNode_Comment>(*SelectedIter))
		{
			Comment->PostCopyNode();
		}
	}

	return ExportedText;
}

FString FMaterialEditor::CopyNodesToBuffer(const TSet<UEdGraphNode*>& Nodes)
{
	static_assert(std::is_convertible<UEdGraphNode*, UObject*>::value, "UEdGraphNode must be derived from UObject");
	static_assert(std::is_same_v<FGraphPanelSelectionSet, TSet<UObject*>>, "FGraphPanelSelectionSet is expected to be defined as TSet<UObject*>");

	return CopyNodesToBuffer(reinterpret_cast<const FGraphPanelSelectionSet&>(Nodes));
}

bool FMaterialEditor::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void FMaterialEditor::PasteNodes()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		PasteNodesHere(FocusedGraphEd->GetPasteLocation2f(), FocusedGraphEd->GetCurrentGraph());
	}
}

void FMaterialEditor::PostPasteMaterialExpression(UMaterialExpression* NewExpression)
{
	// Deep copy subgraph expressions
	if (UMaterialExpressionComposite* Composite = Cast<UMaterialExpressionComposite>(NewExpression))
	{
		UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(Composite->GraphNode);
		DeepCopyExpressions(CompositeNode->BoundGraph, Composite);

		// We just updated all our child expressions, reconstruct node.
		Composite->GraphNode->ReconstructNode();
	}
	else
	{
		// Give new expression a different Guid from the old one after pasting
		NewExpression->UpdateMaterialExpressionGuid(true, true);
	}

	// There can be only one default mesh paint texture.
	UMaterialExpressionTextureBase* TextureSample = Cast<UMaterialExpressionTextureBase>(NewExpression);
	if (TextureSample)
	{
		TextureSample->IsDefaultMeshpaintTexture = false;
	}

	NewExpression->UpdateParameterGuid(true, true);
	Material->AddExpressionParameter(NewExpression, Material->EditorParameters);

	UMaterialExpressionFunctionInput* FunctionInput = Cast<UMaterialExpressionFunctionInput>(NewExpression);
	if (FunctionInput)
	{
		FunctionInput->ConditionallyGenerateId(true);
		FunctionInput->ValidateName();
	}

	UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(NewExpression);
	if (FunctionOutput)
	{
		FunctionOutput->ConditionallyGenerateId(true);
		FunctionOutput->ValidateName();
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(NewExpression);
	if (FunctionCall)
	{
		// When pasting new nodes, we don't want to break all node links as this information is used by UpdateMaterialAfterGraphChange() below,
		// to recreate all the connections in the pasted group.
		// Just update the function input/outputs here.
		const bool bRecreateAndLinkNode = false;
		FunctionCall->UpdateFromFunctionResource(bRecreateAndLinkNode);

		// Some data is lost due to being transient on paste, we need to reconstruct that data now
		// PinCategory - Ensure we set PC_Optional as needed, otherwise they default to PC_Required
		// PinName - Add back in the type information (V4) / (S) / etc
		if (UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(FunctionCall->GraphNode))
		{
			for (UEdGraphPin* Pin : MaterialGraphNode->Pins)
			{
				if (Pin->Direction == EGPD_Input)
				{
					const int32 InputIndex = Pin->SourceIndex;
					if (InputIndex >= 0 && InputIndex < FunctionCall->FunctionInputs.Num())
					{
						Pin->PinType.PinCategory = FunctionCall->IsInputConnectionRequired(InputIndex) ? UMaterialGraphSchema::PC_Required : UMaterialGraphSchema::PC_Optional;

						const FName NewPinName = UMaterialGraphNode::GetShortenPinName(FunctionCall->GetInputName(InputIndex));
						if (!NewPinName.IsNone())
						{
							Pin->PinName = NewPinName;
						}
					}
				}
			}
		}

		// If an unknown material function has been pasted, remove the graph node pins (as the expression will also have had its inputs/outputs removed).
		// This will be displayed as an orphaned "Unspecified Function" node.
		if (FunctionCall->MaterialFunction == nullptr &&
			FunctionCall->FunctionInputs.Num() == 0 &&
			FunctionCall->FunctionOutputs.Num() == 0)
		{
			NewExpression->GraphNode->Pins.Empty();
		}
	}
}

void FMaterialEditor::PasteNodesHere(const FVector2D& Location, const class UEdGraph* Graph)
{
	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	
	PasteNodesHereFromBuffer(Location, Graph, TextToImport, nullptr);
}

void FMaterialEditor::PasteNodesHereFromBuffer(const FVector2D& Location, const class UEdGraph* Graph, const FString& TextToImport, TMap<FGuid, FGuid>* OutOldToNewGuids)
{
	// Undo/Redo support
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MaterialEditorPaste", "Material Editor: Paste") );
	FScopedMaterialChecker Checker{ this, __FUNCTION__ };
	ModifyMaterial();
	
	UMaterialGraph* ExpressionGraph = Graph ? ToRawPtr(CastChecked<UMaterialGraph>(const_cast<UEdGraph*>(Graph))) : ToRawPtr(Material->MaterialGraph);
	ExpressionGraph->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		FocusedGraphEd->ClearSelectionSet();
	}

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(ExpressionGraph, TextToImport, /*out*/ PastedNodes);

	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AvgNodePosition(0.0f,0.0f);

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;
		AvgNodePosition.X += Node->NodePosX;
		AvgNodePosition.Y += Node->NodePosY;
	}

	if ( PastedNodes.Num() > 0 )
	{
		float InvNumNodes = 1.0f/float(PastedNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	TArray<UMaterialExpression*> NewMaterialExpressions;
	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;
		if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Node))
		{
			// These are not copied and we must account for expressions pasted between different materials anyway
			GraphNode->RealtimeDelegate = Material->MaterialGraph->RealtimeDelegate;
			GraphNode->MaterialDirtyDelegate = Material->MaterialGraph->MaterialDirtyDelegate;
			GraphNode->bPreviewNeedsUpdate = false;

			UMaterialExpression* NewExpression = GraphNode->MaterialExpression;
			NewExpression->Material = Material;
			NewExpression->Function = MaterialFunction;
			NewExpression->SubgraphExpression = ExpressionGraph->SubgraphExpression;

			// Make sure the param name is valid after the paste
			if (NewExpression->HasAParameterName())
			{
				NewExpression->ValidateParameterName();
			}

			NewMaterialExpressions.Add(NewExpression);
			Material->GetExpressionCollection().AddExpression(NewExpression);

			ensure(NewExpression->GraphNode == GraphNode);
			PostPasteMaterialExpression(NewExpression);
		}
		else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(Node))
		{
			if (CommentNode->MaterialExpressionComment)
			{
				CommentNode->MaterialDirtyDelegate = Material->MaterialGraph->MaterialDirtyDelegate;
				CommentNode->MaterialExpressionComment->Material = Material;
				CommentNode->MaterialExpressionComment->Function = MaterialFunction;
				CommentNode->MaterialExpressionComment->SubgraphExpression = ExpressionGraph->SubgraphExpression;
				Material->GetExpressionCollection().AddComment(CommentNode->MaterialExpressionComment);
			}
		}

		// Select the newly pasted stuff
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			FocusedGraphEd->SetNodeSelection(Node, true);
		}

		Node->NodePosX = (Node->NodePosX - AvgNodePosition.X) + Location.X ;
		Node->NodePosY = (Node->NodePosY - AvgNodePosition.Y) + Location.Y ;

		Node->SnapToGrid(SNodePanel::GetSnapGridSize());

		const FGuid OldNodeGuid = Node->NodeGuid;
		
		// Give new node a different Guid from the old one
		Node->CreateNewGuid();
		if (OutOldToNewGuids)
		{
			OutOldToNewGuids->Add(OldNodeGuid, Node->NodeGuid);
		}
	}

	for (auto* NewExpression : NewMaterialExpressions)
	{
		// For named reroute fixup: once all nodes are added to Material->Expressions, and that all their Material property are valid
		NewExpression->PostCopyNode(NewMaterialExpressions);
	}

	UpdateMaterialAfterGraphChange();

	// Update UI
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FMaterialEditor::CanPasteNodes() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(Material->MaterialGraph, ClipboardContent);
}

void FMaterialEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedDuplicatableNodes();
}

bool FMaterialEditor::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FMaterialEditor::DuplicateNodes()
{
	// Copy and paste current selection
	CopySelectedNodes();
	PasteNodes();
}

bool FMaterialEditor::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

FText FMaterialEditor::GetOriginalObjectName() const
{
	return FText::FromString(GetEditingObjects()[0]->GetName());
}

TSharedPtr<IAssetReferenceFilter> FMaterialEditor::MakeAssetReferenceFilter() const
{
	if (GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(GetEditingObjects());
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

void FMaterialEditor::UpdateSubstrateTopologyPreview()
{
	if (Substrate::IsSubstrateEnabled())
	{
		// Update all Substrate node which have a preview.
		for (int32 Index = 0; Index < Material->MaterialGraph->Nodes.Num(); ++Index)
		{
			UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Material->MaterialGraph->Nodes[Index]);
			if (MaterialNode && MaterialNode->MaterialExpression && MaterialNode->MaterialExpression->IsA(UMaterialExpressionSubstrateBSDF::StaticClass()))
			{
				UEdGraph* Graph = MaterialNode->GetGraph();
				if (Graph)
				{
					Graph->NotifyGraphChanged();
				}
			}
		}
	}
}

void FMaterialEditor::CreateDerivedMaterialInstancesPreviews()
{
	DerivedMaterialInstances.Empty();
	OriginalDerivedMaterialInstances.Empty();

	TArray<FString> DisplayNames;
	if (OriginalMaterial)
	{
		DisplayNames.Push(OriginalMaterial->GetName());
	}
	else
	{
		//UE-296291, currently unable to track down why the editor occasionally crashes from OriginalMaterial object being a nullptr at this point.
		//Our earliest report we are aware of is around UE5.5.3 but there's no obvious criteria as to why, and currently unable to repro deterministically, so this debug path is temporary to try and pull more info via the CrashReporter.
		//If you see this message and know a method to reproduce please let Epic know!
		const FString MissingEntry = "NULL";
		UE_LOGF(LogMaterialEditor, Warning, "MaterialEditor missing valid OriginalMaterial.");
		UE_LOGF(LogMaterialEditor, Warning, "OriginalMaterialObject = %ls", *(OriginalMaterialObject ? OriginalMaterialObject->GetPathName() : MissingEntry));
		UE_LOGF(LogMaterialEditor, Warning, "Material = %ls", *(Material ? Material->GetPathName() : MissingEntry));
		UE_LOGF(LogMaterialEditor, Warning, "ExpressionPreviewMaterial = %ls", *(ExpressionPreviewMaterial ? ExpressionPreviewMaterial->GetPathName() : MissingEntry));
		UE_LOGF(LogMaterialEditor, Warning, "PreviewExpression = %ls", *(PreviewExpression ? PreviewExpression->GetPathName() : MissingEntry));
		UE_LOGF(LogMaterialEditor, Warning, "MaterialFunction = %ls", *(MaterialFunction ? MaterialFunction->GetPathName() : MissingEntry));
		UE_LOGF(LogMaterialEditor, Warning, "bMaterialDirty = %ls", *LexToString(bMaterialDirty));
		UE_LOGF(LogMaterialEditor, Warning, "bStatsFromPreviewMaterial = %ls", *LexToString(bStatsFromPreviewMaterial));
		UE_LOGF(LogMaterialEditor, Warning, "bDestructing = %ls", *LexToString(bDestructing));

		UE_LOGF(LogMaterialEditor, Fatal, "OriginalMaterial is invalid in the material editor, with no deterministic repro of the problem. This message is to highlight the crash/callstack via the crash reporter with further Log info for Epic to investigate.");
	}

	const int32 MaxCount = CVarMaterialEdMaxDerivedMaterialInstances.GetValueOnGameThread();
	if (MaterialStatsManager->GetProvideDerivedMIFlag() && MaxCount != 0)
	{
		// TODO consider a mode where we load all MaterialChildList also

		for (TObjectIterator<UMaterialInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterialInstance* Instance = *It;
			if (!Instance->HasStaticParameters())
			{
				continue;
			}

			FMaterialInheritanceChain Chain;
			Instance->GetMaterialInheritanceChain(Chain);
			if (Chain.GetBaseMaterial() != OriginalMaterial)
			{
				continue;
			}

			if (Instance->IsEditorOnly())
			{
				continue;
			}

			bool bSkip = false;
			for (int32 i = 1; i < Chain.MaterialInstances.Num(); ++i)
			{
				auto NestedMaterialInstance = Chain.MaterialInstances[i];
				if (NestedMaterialInstance->HasStaticParameters())
				{
					bSkip = true;
				}
			}

			if (bSkip)
			{
				UE_LOGF(LogMaterialEditor, Display, "Skipping material instance '%ls' with static parameters due to depending on other material instances with static parameters", *Instance->GetName());
				continue;
			}

			const UMaterial* InstanceBaseMaterial = Instance->GetBaseMaterial();
			const FStaticParameterSet& InstanceStaticParameters = Instance->GetStaticParameters();
			for(int32_t i = 0; i < OriginalDerivedMaterialInstances.Num(); ++i)
			{
				auto ExistingInstance = OriginalDerivedMaterialInstances[i];
				if (ExistingInstance->GetStaticParameters().Equivalent(InstanceStaticParameters))
				{
					bSkip = true;
				}
			}

			if (bSkip)
			{
				UE_LOGF(LogMaterialEditor, Display, "Skipping material instance '%ls' because instance with same static parameters and base material is already present", *Instance->GetName());
				continue;
			}

			const bool bIsLandscapeMaterial = Instance->IsA<ULandscapeMaterialInstanceConstant>();

			FString DisplayName;
			if (bIsLandscapeMaterial)
			{
				const auto LandscapeMaterial = Cast<ULandscapeMaterialInstanceConstant>(Instance);
				if (LandscapeMaterial->bEditorToolUsage)
				{
					UE_LOGF(LogMaterialEditor, Display, "Skipping material instance '%ls' because it's used by landscape editor", *Instance->GetName());
					continue;
				}

				// Provide a special name for landscape MIC's to improve UX, so user will be able to tell which combination is not compiling.
				FString BaseMaterialName = Instance->Parent ? Instance->Parent->GetName() : Instance->GetName();
				FString LayerNames = FString::JoinBy(LandscapeMaterial->GetEditorOnlyStaticParameters().TerrainLayerWeightParameters,
					TEXT(","),
					[](const FStaticTerrainLayerWeightParameter& x) -> FString { return x.LayerName.ToString(); });

				DisplayName = FString::Format(TEXT("{0}({1})"), {*BaseMaterialName, *LayerNames});
			}
			else
			{
				DisplayName = Instance->GetName();
			}

			UMaterialInstance* DerivedInstance = Cast<UMaterialInstance>(StaticDuplicateObject(Instance, GetTransientPackage(), NAME_None, ~RF_Standalone, Instance->GetClass()));

			// Beware that this potentially ruins inheritance chain:
			// - Let's say we have Base material <- Material instance 1 with no static params <- Material instance 2 with static params
			// - Material instance 1 doesn't influence shader compilation
			// - So it's safe to change inheritance to Base material <- Material instance 2 with static params
			DerivedInstance->Parent = Material;

			DerivedMaterialInstances.Add(DerivedInstance);
			OriginalDerivedMaterialInstances.Add(Instance);
			DisplayNames.Push(DisplayName);

			if (MaxCount >= 0 && OriginalDerivedMaterialInstances.Num() >= MaxCount)
			{
				break;
			}
		}
	}

	MaterialStatsManager->SetMaterialsDisplayNames(DisplayNames);
}

UMaterialGraphNode* FMaterialEditor::GetFirstSelectedGraphNode() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt))
		{
			return GraphNode;
		}
	}

	return nullptr;
}

void FMaterialEditor::UpdateMaterialAfterGraphChange()
{
	FlushRenderingCommands();
	InvalidateSubstrateConversionVersion(Material);
	Material->MaterialGraph->LinkMaterialExpressionsFromGraph();

	// Update the current preview material.
	UpdatePreviewMaterial();

	Material->MarkPackageDirty();
	RegenerateCodeView();
	RefreshExpressionPreviews();
	SetMaterialDirty();
	UpdateGenerator();

	if (NodeFeatureLevel != ERHIFeatureLevel::Num || NodeQualityLevel != EMaterialQualityLevel::Num || bPreviewStaticSwitches)
	{
		bPreviewFeaturesChanged = true;
	}

	if (bHideUnrelatedNodes && !bLockNodeFadeState && bSelectRegularNode)
	{
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			FocusedGraphEd->ResetAllNodesUnrelatedStates();
		}

		HideUnrelatedNodes();
	}

	Material->MaterialGraph->UpdatePinTypes();

	UpdateSubstrateTopologyPreview();

	if (Substrate::IsSubstrateEnabled())
	{
		// Update preview tab to adapt to the change of domain.
		// This is because with Substrate, we do not receive any event that the material domain has changed from the UI combobox (no longer editable, set form the graph via RebuildShadingModelField).
		if (ExpressionPreviewMaterial)
		{
			SetPreviewMaterial(ExpressionPreviewMaterial);
		}
		else
		{
			SetPreviewMaterial(Material);
		}
		UpdatePreviewViewportsVisibility();
	}
}

void FMaterialEditor::MarkMaterialDirty()
{
	SetMaterialDirty();
}

void FMaterialEditor::JumpToHyperlink(const UObject* ObjectReference)
{
	if (const UEdGraphNode* Node = Cast<const UEdGraphNode>(ObjectReference))
	{
		JumpToNode(Node);
	}
	else if (const UEdGraph* Graph = Cast<const UEdGraph>(ObjectReference))
	{
		// Navigating into things should re-use the current tab when it makes sense
		FDocumentTracker::EOpenDocumentCause OpenMode = FDocumentTracker::OpenNewDocument;
		if ((Graph == Material->MaterialGraph) || Cast<UMaterialGraphNode_Composite>(Graph->GetOuter()))
		{
			OpenMode = FDocumentTracker::NavigatingCurrentDocument;
		}
		else
		{
			// Walk up the outer chain to see if any tabs have a parent of this document open for edit, and if so
			// we should reuse that one and drill in deeper instead
			for (UObject* WalkPtr = const_cast<UEdGraph*>(Graph); WalkPtr != nullptr; WalkPtr = WalkPtr->GetOuter())
			{
				TArray< TSharedPtr<SDockTab> > TabResults;
				if (FindOpenTabsContainingDocument(WalkPtr, /*out*/ TabResults))
				{
					// See if the parent was active
					bool bIsActive = false;
					for (TSharedPtr<SDockTab> Tab : TabResults)
					{
						if (Tab->IsActive())
						{
							bIsActive = true;
							break;
						}
					}

					if (bIsActive)
					{
						OpenMode = FDocumentTracker::NavigatingCurrentDocument;
						break;
					}
				}
			}
		}

		// Force it to open in a new document if shift is pressed
		const bool bIsShiftPressed = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		if (bIsShiftPressed)
		{
			auto PayloadAlreadyOpened = [&]()
			{
				TArray< TSharedPtr<SDockTab> > GraphEditorTabs;
				DocumentManager->FindAllTabsForFactory(GraphEditorTabFactoryPtr, /*out*/ GraphEditorTabs);

				for (TSharedPtr<SDockTab>& GraphEditorTab : GraphEditorTabs)
				{
					TSharedRef<SGraphEditor> Editor = StaticCastSharedRef<SGraphEditor>((GraphEditorTab)->GetContent());

					if (Editor->GetCurrentGraph() == Graph)
					{
						return true;
					}
				}

				return false;
			};

			OpenMode = PayloadAlreadyOpened() ? FDocumentTracker::RestorePreviousDocument : FDocumentTracker::ForceOpenNewDocument;
		}

		// Open the document
		OpenDocument(Graph, OpenMode);
	}
}

int32 FMaterialEditor::GetNumberOfSelectedNodes() const
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		return FocusedGraphEd->GetSelectedNodes().Num();
	}

	return 0;
}

FGraphPanelSelectionSet FMaterialEditor::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	if (FocusedGraphEdPtr.IsValid())
	{
		CurrentSelection = FocusedGraphEdPtr.Pin()->GetSelectedNodes();
	}
	return CurrentSelection;
}

void FMaterialEditor::GetBoundsForNode(const UEdGraphNode* InNode, class FSlateRect& OutRect, float InPadding) const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->GetBoundsForNode(InNode, OutRect, InPadding);
	}
}


FMatExpressionPreview* FMaterialEditor::GetExpressionPreview(UMaterialExpression* InExpression)
{
	bool bNewlyCreated;
	return GetExpressionPreview(InExpression, bNewlyCreated);
}

void FMaterialEditor::UndoGraphAction()
{
	FlushRenderingCommands();
	
	int32 NumExpressions = Material->GetExpressions().Num();
	GEditor->UndoTransaction();

	if(NumExpressions != Material->GetExpressions().Num())
	{
		Material->BuildEditorParameterList();
	}
}

void FMaterialEditor::RedoGraphAction()
{
	FlushRenderingCommands();

	int32 NumExpressions = Material->GetExpressions().Num();
	GEditor->RedoTransaction();

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		// @TODO Find a more coherent check for this, rather than rely on sage knowledge that the schema won't exist.
		// If our previous transaction created the current graph, it won't have a schema after undo.
		if (!IsValidChecked(FocusedGraphEd->GetCurrentGraph()))
		{
			CloseDocumentTab(FocusedGraphEd->GetCurrentGraph());
			DocumentManager->CleanInvalidTabs();
			DocumentManager->RefreshAllTabs();
			GetTabManager()->TryInvokeTab(FMaterialEditorTabs::GraphEditor);
		}

		// Active graph can change above, re-acquire ptr
		FocusedGraphEdPtr.Pin()->NotifyGraphChanged();
	}

	if(NumExpressions != Material->GetExpressions().Num())
	{
		Material->BuildEditorParameterList();
	}

	UpdateGenerator();
}

void FMaterialEditor::OnCollapseNodes()
{
	const UMaterialGraphSchema* Schema = GetDefault<UMaterialGraphSchema>();

	// Does the selection set contain anything that is legal to collapse?
	TSet<UEdGraphNode*> CollapsableNodes;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Schema->CanEncapuslateNode(*SelectedNode))
			{
				CollapsableNodes.Add(SelectedNode);
			}
		}
	}
	
	// Sort for deterministic pin ordering, unaffected by selection order.
	CollapsableNodes.StableSort([](const UEdGraphNode& A, const UEdGraphNode& B)
	{
		if (A.NodePosY == B.NodePosY)
		{
			return A.NodePosX > B.NodePosX;
		}
		return A.NodePosY < B.NodePosY;
	});

	// Collapse them
	if (CollapsableNodes.Num())
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().CollapseNodes->GetDescription());
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };
		ModifyMaterial();

		CollapseNodes(CollapsableNodes);
	}
}

bool FMaterialEditor::CanCollapseNodes() const
{
	// Does the selection set contain anything that is legal to collapse?
	const UMaterialGraphSchema* Schema = GetDefault<UMaterialGraphSchema>();
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Schema->CanEncapuslateNode(*Node))
			{
				return true;
			}
		}
	}

	return false;
}

void FMaterialEditor::OnExpandNodes()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().ExpandNodes->GetLabel());
	FScopedMaterialChecker Checker{ this, __FUNCTION__ };
	ModifyMaterial();

	TSet<UEdGraphNode*> ExpandedNodes;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

	// Expand selected nodes into the focused graph context.
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (FocusedGraphEd)
	{
		FocusedGraphEd->ClearSelectionSet();
	}
	TMap<UMaterialGraphNode*, FMaterialEditor*> FunctionCalls;
       
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		ExpandedNodes.Empty();
		bool bExpandedNodesNeedUniqueGuid = true;

		DocumentManager->CleanInvalidTabs();
		UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(*NodeIt);
		 	
		if (UMaterialGraphNode_Composite* SelectedCompositeNode = Cast<UMaterialGraphNode_Composite>(*NodeIt))
		{
			// No need to assign unique GUIDs since the source graph will be removed.
			bExpandedNodesNeedUniqueGuid = false;

			// Expand the composite node back into the world
			UEdGraph* SourceGraph = SelectedCompositeNode->BoundGraph;
			ExpandNode(SelectedCompositeNode, SourceGraph, /*inout*/ ExpandedNodes);

			FBlueprintEditorUtils::RemoveGraph(nullptr, SourceGraph, EGraphRemoveFlags::None);
			SourceGraph->MarkAsGarbage();
		}
		else if (Node)
		{
			UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(Node->MaterialExpression);
			if (FunctionCallExpression && FunctionCallExpression->MaterialFunction)
			{
				FMaterialEditor* FunctionMaterialEditor = FMaterialEditorHelpers::OpenMaterialEditorForAsset(FunctionCallExpression->MaterialFunction);
				if (!ensure(FunctionMaterialEditor))
				{
					continue;
				}
				// FunctionCalls.Add(Node, FunctionMaterialEditor);
				FMaterialEditorHelpers::ExpandNode(*this, *FunctionMaterialEditor, Node);

				this->FocusWindow();
			}
		}
		

		UEdGraphNode* SourceNode = CastChecked<UEdGraphNode>(*NodeIt);
		check(SourceNode);
		MoveNodesToAveragePos(ExpandedNodes, FVector2D(SourceNode->NodePosX, SourceNode->NodePosY), bExpandedNodesNeedUniqueGuid);
	}

	UMaterialExpression* SubgraphExpression = FocusedGraphEd ? ToRawPtr(Cast<UMaterialGraph>(FocusedGraphEd->GetCurrentGraph())->SubgraphExpression) : ToRawPtr(Material->MaterialGraph->SubgraphExpression);

	for (UEdGraphNode* ExpandedNode : ExpandedNodes)
	{
		if (UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(ExpandedNode))
		{
			MaterialNode->MaterialExpression->Modify();
			MaterialNode->MaterialExpression->SubgraphExpression = SubgraphExpression;
		}
		else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(ExpandedNode))
		{
			CommentNode->MaterialExpressionComment->Modify();
			CommentNode->MaterialExpressionComment->SubgraphExpression = SubgraphExpression;
		}
	}

	UpdateMaterialAfterGraphChange();
}

bool FMaterialEditor::CanExpandNodes() const
{
	// Does the selection set contain any composite nodes that are legal to expand?
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(*NodeIt);
		if (Cast<UMaterialGraphNode_Composite>(Node))
		{
			return true;
		}
		else if (Node && Cast<UMaterialExpressionMaterialFunctionCall>(Node->MaterialExpression))
		{
			return true;
		}
	}

	return false;
}

void FMaterialEditor::OnOpenSelectedNodesInNewTab()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (const UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(*NodeIt))
		{
			if (CompositeNode->BoundGraph)
			{
				OpenDocument(CompositeNode->BoundGraph, FDocumentTracker::ForceOpenNewDocument);
			}
		}
	}
}

bool FMaterialEditor::CanOpenSelectedNodesInNewTab() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (const UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(*NodeIt))
		{
			if (CompositeNode->BoundGraph)
			{
				return true;
			}
		}
	}

	return false;
}

void FMaterialEditor::OnAlignTop()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnAlignTop();
	}
}

void FMaterialEditor::OnAlignMiddle()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnAlignMiddle();
	}
}

void FMaterialEditor::OnAlignBottom()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnAlignBottom();
	}
}

void FMaterialEditor::OnAlignLeft()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnAlignLeft();
	}
}

void FMaterialEditor::OnAlignCenter()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnAlignCenter();
	}
}

void FMaterialEditor::OnAlignRight()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnAlignRight();
	}
}

void FMaterialEditor::OnStraightenConnections()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnStraightenConnections();
	}
}

void FMaterialEditor::OnDistributeNodesH()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnDistributeNodesH();
	}
}

void FMaterialEditor::OnDistributeNodesV()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->OnDistributeNodesV();
	}
}


void FMaterialEditor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{	
		checkf(Material, TEXT("Unknown material during material Undo operation"));
		FScopedMaterialChecker Checker{ this, __FUNCTION__ };

		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			FocusedGraphEd->ClearSelectionSet();
		}

		RestoreExpressionBackReferences(Material);

		Material->BuildEditorParameterList();

		// Update the current preview material.
		UpdatePreviewMaterial();
		UpdatePreviewViewportsVisibility();
		RefreshExpressionPreviews();
		UpdateOriginalMaterial();

		// Remove any tabs are that are pending kill or otherwise invalid UObject pointers.
		bool bNeedOpenGraphEditor = false;

		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			checkf(FocusedGraphEd->GetCurrentGraph(), TEXT("Null FocusedGraphEd->GetCurrentGraph() during Undo operation in material '%s'"), *Material->GetName());

			// @TODO Find a more coherent check for this, rather than rely on sage knowledge that the schema won't exist.
			// If our previous transaction created the current graph, it won't have a schema after undo.
			if (!FocusedGraphEd->GetCurrentGraph()->GetSchema())
			{
				checkf(DocumentManager, TEXT("Null DocumentManager during Undo operation in material '%s'"), *Material->GetName());

				CloseDocumentTab(FocusedGraphEd->GetCurrentGraph());
				DocumentManager->CleanInvalidTabs();
				DocumentManager->RefreshAllTabs();
				GetTabManager()->TryInvokeTab(FMaterialEditorTabs::GraphEditor);
			}

			// Active graph can change above, re-acquire ptr
			FocusedGraphEd->NotifyGraphChanged();
		}

		SetEditorInstanceForWidgets();

		SetMaterialDirty();

		UpdateGenerator();

		FSlateApplication::Get().DismissAllMenus();
	}
}

void FMaterialEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	check( !ScopedTransaction );
	ScopedTransaction = new FScopedTransaction( NSLOCTEXT("UnrealEd", "MaterialEditorEditProperties", "Material Editor: Edit Properties") );
	FlushRenderingCommands();
}

void FMaterialEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	check( ScopedTransaction );

	if ( PropertyThatChanged )
	{
		SetEditorInstanceForWidgets();

		const FName NameOfPropertyThatChanged( *PropertyThatChanged->GetName() );
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if ((NameOfPropertyThatChanged == GET_MEMBER_NAME_CHECKED(UMaterialInterface, PreviewMesh)) ||
			(NameOfPropertyThatChanged == GET_MEMBER_NAME_CHECKED(UMaterial, bUsedWithSkeletalMesh)))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// Revert to editor preview mesh when usage isn't supported.
			if (!SetPreviewAssetByName(*Material->PreviewMesh.ToString()))
			{
				if (EditorOptions)
				{
					SetPreviewAsset(EditorOptions->GetPreviewMesh());
				}
			}
		}
		else if( NameOfPropertyThatChanged == GET_MEMBER_NAME_CHECKED(UMaterial, MaterialDomain) ||
				 NameOfPropertyThatChanged == GET_MEMBER_NAME_CHECKED(UMaterial, ShadingModel) ||
				 NameOfPropertyThatChanged == GET_MEMBER_NAME_CHECKED(UMaterialFunction, PreviewMaterialDomain))
		{
			if (MaterialFunction)
			{
				Material->MaterialDomain = MaterialFunction->PreviewMaterialDomain;
			}
			
			Material->MaterialGraph->RebuildGraph();
			TArray<TWeakObjectPtr<UObject>> SelectedObjects = MaterialDetailsView->GetSelectedObjects();
			MaterialDetailsView->SetObjects( SelectedObjects, true );
			SetPreviewMaterial(Material);

			if (ExpressionPreviewMaterial)
			{
				if (Material->IsUIMaterial())
				{
					ExpressionPreviewMaterial->MaterialDomain = MD_UI;
				}
				else
				{
					ExpressionPreviewMaterial->MaterialDomain = MD_Surface;
				}

				SetPreviewMaterial(ExpressionPreviewMaterial);
			}

			UpdatePreviewViewportsVisibility();
		}
		else if (bPreviewStaticSwitches &&
				 PropertyThatChanged->GetOwnerClass() == UMaterialExpressionStaticBoolParameter::StaticClass() &&
				 NameOfPropertyThatChanged == GET_MEMBER_NAME_CHECKED(UMaterialExpressionStaticBoolParameter, DefaultValue))
		{
			bPreviewFeaturesChanged = true;
		}

		FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UMaterialGraphNode* SelectedNode = Cast<UMaterialGraphNode>(*NodeIt);

			if (SelectedNode && SelectedNode->MaterialExpression)
			{
				if(NameOfPropertyThatChanged == FName(TEXT("ParameterName")))
				{
					Material->UpdateExpressionParameterName(SelectedNode->MaterialExpression);
				}
				else if (SelectedNode->MaterialExpression->IsA(UMaterialExpressionDynamicParameter::StaticClass()))
				{
					Material->UpdateExpressionDynamicParameters(SelectedNode->MaterialExpression);
				}
				else if (PropertyThatChanged->IsA<FTextProperty>())
				{
					// Do nothing to the expression if we are just changing the label
				}
				else
				{
					Material->PropagateExpressionParameterChanges(SelectedNode->MaterialExpression);
				}
			}
		}
	}

	// Prevent constant recompilation of materials while properties are being interacted with
	if( PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive )
	{
		// Also prevent recompilation when properties have no effect on material output
		const FName PropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
		if (PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionComment, Text)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionComment, CommentColor)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionComment, bColorCommentBubble)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionComment, bGroupMode)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionComment, bCommentBubbleVisible_InDetailsPanel)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpression, Desc)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorParameter, ChannelNames)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSampleParameter, ChannelNames))
		{
			// Update the current preview material.
			InvalidateSubstrateConversionVersion(Material);
			UpdatePreviewMaterial();
			RefreshExpressionPreviews();
			RegenerateCodeView();
		}

		GetDefault<UMaterialGraphSchema>()->ForceVisualizationCacheClear();
	}

	delete ScopedTransaction;
	ScopedTransaction = NULL;

	Material->MarkPackageDirty();
	SetMaterialDirty();

	UpdateGenerator();
}

void FMaterialEditor::ToggleCollapsed(UMaterialExpression* MaterialExpression)
{
	check( MaterialExpression );
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MaterialEditorToggleCollapsed", "Material Editor: Toggle Collapsed") );
		MaterialExpression->Modify();
		MaterialExpression->bCollapsed = !MaterialExpression->bCollapsed;
	}
	MaterialExpression->PreEditChange( NULL );
	MaterialExpression->PostEditChange();
	MaterialExpression->MarkPackageDirty();
	SetMaterialDirty();

	// Update the preview.
	RefreshExpressionPreview( MaterialExpression, true );
	RefreshPreviewViewport();
}

void FMaterialEditor::RefreshExpressionPreviews(bool bForceRefreshAll /*= false*/)
{
	const FScopedBusyCursor BusyCursor;

	Material->UpdateCachedExpressionData();
	
	// @todo-massimo.tristano
	// We have a bug causing nulls to end up in the expressions array, but we are not yet aware of
	// a repro to track down what actually causes it. This temporary code not only sanitizes any
	// null expression in the array (that would crash the editor) but also allows us to know when
	// null expressions are found.
	if (Material->GetEditorOnlyData()->ExpressionCollection.Expressions.Remove(nullptr) > 0)
	{
		UE_LOGF(LogMaterialEditor, Warning, "Null expressions found in expressions list during expression previews refresh in material '%ls'.", *GetEditingObjects()[0]->GetName());
	}

	if ( bAlwaysRefreshAllPreviews || bForceRefreshAll)
	{
		// Refresh all expression previews.
		FMaterial::DeferredDeleteArray(ExpressionPreviews);

		for (UMaterialExpression* MaterialExpression : Material->GetExpressions())
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(MaterialExpression->GraphNode);
			if (GraphNode)
			{
				GraphNode->InvalidatePreviewMaterialDelegate.ExecuteIfBound();
			}
		}
	}
	else
	{
		// Only refresh expressions that are marked for realtime update.
		for (UMaterialExpression* MaterialExpression : Material->GetExpressions())
		{
			RefreshExpressionPreview(MaterialExpression, false);
		}
	}

	TArray<FMatExpressionPreview*> ExpressionPreviewsBeingCompiled;
	ExpressionPreviewsBeingCompiled.Empty(50);

	// Go through all expression previews and create new ones as needed, and maintain a list of previews that are being compiled
	for (UMaterialExpression* MaterialExpression : Material->GetExpressions())
	{
		if (!MaterialExpression->IsA(UMaterialExpressionComment::StaticClass()))
		{
			bool bNewlyCreated;
			FMatExpressionPreview* Preview = GetExpressionPreview( MaterialExpression, bNewlyCreated );
			if (bNewlyCreated && Preview)
			{
				ExpressionPreviewsBeingCompiled.Add(Preview);
			}
		}
	}
}

void FMaterialEditor::RefreshExpressionPreview(UMaterialExpression* MaterialExpression, bool bRecompile)
{
	check(MaterialExpression);

	if ( (MaterialExpression->bRealtimePreview || MaterialExpression->bNeedToUpdatePreview) && !MaterialExpression->bCollapsed )
	{
		for( int32 PreviewIndex = 0 ; PreviewIndex < ExpressionPreviews.Num() ; ++PreviewIndex )
		{
			FMatExpressionPreview* ExpressionPreview = ExpressionPreviews[PreviewIndex];
			if (ensure(ExpressionPreview) && ExpressionPreview->GetExpression() == MaterialExpression)
			{
				// We need to make sure we don't hold any other references before queuing the deferred delete as it may execute immediately (e.g. -onethread, or just very fast RT).
				// There's no danger in releasing the last reference to FMaterial as this doesn't delete it anyway, it needs to be deleted manually via DeferredDelete.
				ExpressionPreviews.RemoveAt(PreviewIndex);
				FMaterial::DeferredDelete(ExpressionPreview);
				ExpressionPreview = nullptr;
				MaterialExpression->bNeedToUpdatePreview = false;

				if (bRecompile)
				{
					bool bNewlyCreated;
					GetExpressionPreview(MaterialExpression, bNewlyCreated);
				}

				UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(MaterialExpression->GraphNode);
				if (GraphNode)
				{
					GraphNode->InvalidatePreviewMaterialDelegate.ExecuteIfBound();
				}

				break;
			}
		}
	}
}

FMatExpressionPreview* FMaterialEditor::GetExpressionPreview(UMaterialExpression* MaterialExpression, bool& bNewlyCreated)
{
	bNewlyCreated = false;
	if (MaterialExpression->ShouldShowPreview() && !MaterialExpression->IsA<UMaterialExpressionCustomOutput>())
	{
		FMatExpressionPreview* Preview = NULL;
		for( int32 PreviewIndex = 0 ; PreviewIndex < ExpressionPreviews.Num() ; ++PreviewIndex )
		{
			FMatExpressionPreview* ExpressionPreview = ExpressionPreviews[PreviewIndex];
			if( ExpressionPreview->GetExpression() == MaterialExpression )
			{
				Preview = ExpressionPreviews[PreviewIndex];
				break;
			}
		}

		if (!Preview && MaterialExpression->Material->GetExpressions().Contains(MaterialExpression))
		{
			bNewlyCreated = true;
			Preview = new FMatExpressionPreview(MaterialExpression);
			ExpressionPreviews.Add(Preview);
			Preview->CacheShaders(EMaterialShaderPrecompileMode::None);
		}

		return Preview;
	}

	return NULL;
}

void FMaterialEditor::OnColorPickerCommitted(FLinearColor LinearColor, TWeakObjectPtr<UObject> ColorPickerObject)
{	
	if (UObject* Object = ColorPickerObject.Get())
	{
		// Begin a property edit transaction.
		if (GEditor)
		{
			GEditor->BeginTransaction(LOCTEXT("ModifyColorPicker", "Modify Color Picker Value"));
		}

		NotifyPreChange(NULL);

		Object->PreEditChange(NULL);

		FName PropertyName;
		if (UMaterialExpressionConstant3Vector* Constant3Expression = Cast<UMaterialExpressionConstant3Vector>(Object))
		{
			Constant3Expression->Constant = LinearColor;
			PropertyName = TEXT("Constant");
		}
		else if (UMaterialExpressionConstant4Vector* Constant4Expression = Cast<UMaterialExpressionConstant4Vector>(Object))
		{
			Constant4Expression->Constant = LinearColor;
			PropertyName = TEXT("Constant");
		}
		else if (UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(Object))
		{
			InputExpression->PreviewValue = LinearColor;
		}
		else if (UMaterialExpressionVectorParameter* VectorExpression = Cast<UMaterialExpressionVectorParameter>(Object))
		{
			VectorExpression->DefaultValue = LinearColor;
			PropertyName = TEXT("DefaultValue");
		}
		else
		{
			checkf(false, TEXT("The expression type is supported in OnNodeDoubleClicked but not in OnColorPickerCommitted"));
		}

		Object->MarkPackageDirty();

		FProperty* ColorPickerProperty = PropertyName.IsNone() ? nullptr : Object->GetClass()->FindPropertyByName(PropertyName);
		FPropertyChangedEvent Event(ColorPickerProperty);
		Object->PostEditChangeProperty(Event);

		NotifyPostChange(NULL, NULL);

		if (GEditor)
		{
			GEditor->EndTransaction();
		}

		RefreshExpressionPreviews();
		SetEditorInstanceForWidgets();
	}
}

/** Create new tab for the supplied graph - don't call this directly, instead call OpenDocument to track history.*/
TSharedRef<SGraphEditor> FMaterialEditor::CreateGraphEditorWidget(TSharedRef<class FTabInfo> InTabInfo, class UEdGraph* InGraph)
{
	check((InGraph != nullptr) && Cast<UMaterialGraph>(InGraph));

	if (!GraphEditorCommands)
{
	GraphEditorCommands = MakeShareable( new FUICommandList );

		// Editing commands
		GraphEditorCommands->MapAction( FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP( this, &FMaterialEditor::SelectAllNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanSelectAllNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP( this, &FMaterialEditor::DeleteSelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanDeleteNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP( this, &FMaterialEditor::CopySelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanCopyNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP( this, &FMaterialEditor::PasteNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanPasteNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP( this, &FMaterialEditor::CutSelectedNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanCutNodes )
			);

		GraphEditorCommands->MapAction( FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP( this, &FMaterialEditor::DuplicateNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanDuplicateNodes )
			);
		GraphEditorCommands->MapAction(FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnRenameNode),
			FCanExecuteAction::CreateSP(this, &FMaterialEditor::CanRenameNodes)
		);

		// Graph Editor Commands
		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CreateComment,
			FExecuteAction::CreateSP( this, &FMaterialEditor::OnCreateComment )
			);

		// Material specific commands
		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().UseCurrentTexture,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnUseCurrentTexture)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ConvertObjects,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertObjects)
			);

		GraphEditorCommands->MapAction(FMaterialEditorCommands::Get().PromoteToDouble,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnPromoteObjects)
		);

		GraphEditorCommands->MapAction(FMaterialEditorCommands::Get().PromoteToFloat,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnPromoteObjects)
		);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ConvertToTextureObjects,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertTextures)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ConvertToTextureSamples,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertTextures)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ConvertToConstant,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertObjects)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().SelectNamedRerouteDeclaration,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectNamedRerouteDeclaration)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().SelectNamedRerouteUsages,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectNamedRerouteUsages)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().CreateRerouteUsageFromDeclaration,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnCreateRerouteUsageFromDeclaration)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ConvertRerouteToNamedReroute,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertRerouteToNamedReroute)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ConvertNamedRerouteToReroute,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnConvertNamedRerouteToReroute)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().StopPreviewNode,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnPreviewNode)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().StartPreviewNode,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnPreviewNode)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().PreviewHoveredNode,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnPreviewHoveredNode)
		);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().EnableRealtimePreviewNode,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnToggleRealtimePreview)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().DisableRealtimePreviewNode,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnToggleRealtimePreview)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().SelectDownstreamNodes,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectDownstreamNodes)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().SelectUpstreamNodes,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnSelectUpstreamNodes)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().RemoveFromFavorites,
			FExecuteAction::CreateSP(this, &FMaterialEditor::RemoveSelectedExpressionFromFavorites)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().AddToFavorites,
			FExecuteAction::CreateSP(this, &FMaterialEditor::AddSelectedExpressionToFavorites)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().ForceRefreshPreviews,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnForceRefreshPreviews)
			);

		GraphEditorCommands->MapAction( FMaterialEditorCommands::Get().CreateComponentMaskNode,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnCreateComponentMaskNode)
			);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().GoToDefinition,
									   FExecuteAction::CreateSP(this, &FMaterialEditor::OnGoToDefinition),
									   FCanExecuteAction::CreateSP(this, &FMaterialEditor::CanGoToDefinition)
									   );

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().GoToDocumentation,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnGoToDocumentation),
			FCanExecuteAction::CreateSP(this, &FMaterialEditor::CanGoToDocumentation)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().OpenInNewTab,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnOpenSelectedNodesInNewTab),
			FCanExecuteAction::CreateSP(this, &FMaterialEditor::CanOpenSelectedNodesInNewTab),
			FIsActionButtonVisible::CreateSP(this, &FMaterialEditor::CanOpenSelectedNodesInNewTab)
		);

		// Collapse Node Commands
		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CollapseNodes,
			FExecuteAction::CreateSP( this, &FMaterialEditor::OnCollapseNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanCollapseNodes )
		);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CollapseSelectionToFunction,
					FExecuteAction::CreateSP( this, &FMaterialEditor::OnCollapseToFunction ),
					FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanCollapseToFunction )
				);
		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().ExpandNodes,
			FExecuteAction::CreateSP( this, &FMaterialEditor::OnExpandNodes ),
			FCanExecuteAction::CreateSP( this, &FMaterialEditor::CanExpandNodes ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FMaterialEditor::CanExpandNodes )
		);

		// Alignment Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignTop)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignMiddle)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignBottom)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignLeft)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignCenter)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignRight)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnStraightenConnections)
		);

		// Distribution Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnDistributeNodesH)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnDistributeNodesV)
		);

	}

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FMaterialEditor::OnSelectedNodesChanged);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FMaterialEditor::OnNodeDoubleClicked);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FMaterialEditor::OnNodeTitleCommitted);
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FMaterialEditor::OnVerifyNodeTextCommit);
	InEvents.OnSpawnNodeByShortcutAtLocation = SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateSP(this, &FMaterialEditor::OnSpawnGraphNodeByShortcut, static_cast<UEdGraph*>(InGraph));

	// Create the title bar widget
	TSharedPtr<SWidget> TitleBarWidget = SNew(SMaterialEditorTitleBar)
		.EdGraphObj(InGraph)
		.TitleText(this, &FMaterialEditor::GetOriginalObjectName)
		.OnDifferentGraphCrumbClicked(this, &FMaterialEditor::OnChangeBreadCrumbGraph)
		.HistoryNavigationWidget(InTabInfo->CreateHistoryNavigationWidget())
		.MaterialInfoList(&MaterialInfoList);

	return SNew(SGraphEditor)
		.GraphSupportsInvalidation(true)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.TitleBar(TitleBarWidget)
		.Appearance(this, &FMaterialEditor::GetGraphAppearance)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents)
		.ShowGraphStateOverlay(false)
		.OnNavigateHistoryBack(FSimpleDelegate::CreateSP(this, &FMaterialEditor::NavigateTab, FDocumentTracker::NavigateBackwards))
		.OnNavigateHistoryForward(FSimpleDelegate::CreateSP(this, &FMaterialEditor::NavigateTab, FDocumentTracker::NavigateForwards))
		.AssetEditorToolkit(this->AsShared());
}

FGraphAppearanceInfo FMaterialEditor::GetGraphAppearance() const
{
	FGraphAppearanceInfo AppearanceInfo;

	if (MaterialFunction)
	{
		switch (MaterialFunction->GetMaterialFunctionUsage())
		{
		case EMaterialFunctionUsage::MaterialLayer:
			AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_MaterialLayer", "MATERIAL LAYER");
			break;
		case EMaterialFunctionUsage::MaterialLayerBlend:
			AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_MaterialLayerBlend", "MATERIAL LAYER BLEND");
			break;
		default:
			AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_MaterialFunction", "MATERIAL FUNCTION");
		}
	}
	else
	{
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Material", "MATERIAL"); 
	}

	if (Substrate::IsSubstrateEnabled())
	{
		UMaterial* MaterialForStats = this->bStatsFromPreviewMaterial ? this->Material : this->OriginalMaterial;
		const FMaterialResource* MaterialResource = MaterialForStats->GetMaterialResource(GMaxRHIShaderPlatform);
		if (MaterialResource)
		{
			FString MaterialDescription;

			FMaterialShaderMap* ShaderMap = MaterialResource->GetGameThreadShaderMap();
			if (ShaderMap)
			{
				const FSubstrateMaterialCompilationOutput& CompilationOutput = ShaderMap->GetSubstrateMaterialCompilationOutput();
				if (CompilationOutput.bMaterialOutOfBudgetHasBeenSimplified)
				{
					AppearanceInfo.WarningText = LOCTEXT("AppearanceWarningText_Material", "Substrate material was out of budget and has been simplified.");

				}
			}
		}
	}

	return AppearanceInfo;
}

TArray<UMaterialGraphNode*> FMaterialEditor::GetSelectionRelatedNodes()
{
	TArray<UMaterialGraphNode*> RelatedNodes;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UMaterialGraphNode* SelectedNode = Cast<UMaterialGraphNode>(*NodeIt);

		if (SelectedNode)
		{
			RelatedNodes.Add(SelectedNode);
			CollectDownstreamNodes(SelectedNode, RelatedNodes);
			CollectUpstreamNodes(SelectedNode, RelatedNodes);
		}
	}

	return RelatedNodes;
}

void FMaterialEditor::DeepCopyExpressions(UMaterialGraph* CopyGraph, UMaterialExpression* NewSubgraphExpression)
{
	if (!CopyGraph || !NewSubgraphExpression)
	{
		return;
	}
	
	CopyGraph->Modify();
	CopyGraph->SubgraphExpression = NewSubgraphExpression;

	// Duplicate subnodes
	auto DuplicateExpression = [&](auto* Expression)
	{
		using ExpressionType = typename std::remove_pointer<decltype(Expression)>::type;
		return Cast<ExpressionType>(UMaterialEditingLibrary::DuplicateMaterialExpression(Material, MaterialFunction, Expression));
	};

	for (UEdGraphNode* Node : CopyGraph->Nodes)
	{
		if (UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Node))
		{
			MaterialNode->Modify();
			MaterialNode->Rename(/*NewName=*/ NULL, /*NewOuter=*/ CopyGraph);
			if (UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(MaterialNode->MaterialExpression))
			{
				UMaterialExpressionPinBase* OldPinBase = PinBase;
				UMaterialExpressionPinBase* NewPinBase = DuplicateExpression(OldPinBase);
				MaterialNode->MaterialExpression = NewPinBase;

				NewPinBase->SubgraphExpression = NewSubgraphExpression;
				NewPinBase->ReroutePins.Empty();
				for (FCompositeReroute& Reroute : OldPinBase->ReroutePins)
				{
					UMaterialExpressionReroute* DupReroute = DuplicateExpression(ToRawPtr(Reroute.Expression));
					DupReroute->SubgraphExpression = NewSubgraphExpression;
					NewPinBase->ReroutePins.Add({ Reroute.Name, decltype(FCompositeReroute::Expression)(DupReroute) });
				}

				UMaterialExpressionComposite* SubGraphComposite = CastChecked<UMaterialExpressionComposite>(NewSubgraphExpression);
				if (NewPinBase->PinDirection == EGPD_Output)
				{
					SubGraphComposite->InputExpressions = NewPinBase;
				}
				else
				{
					SubGraphComposite->OutputExpressions = NewPinBase;
				}
			}
			else
			{
				MaterialNode->MaterialExpression = DuplicateExpression(ToRawPtr(MaterialNode->MaterialExpression));
				MaterialNode->MaterialExpression->SubgraphExpression = NewSubgraphExpression;
			}
			
			// GraphNode is transient so it won't be duplicated. 
			MaterialNode->MaterialExpression->GraphNode = MaterialNode;
			PostPasteMaterialExpression(MaterialNode->MaterialExpression);

		}
		else if (UMaterialGraphNode_Comment* Comment = Cast<UMaterialGraphNode_Comment>(Node))
		{
			Comment->Modify();
			Comment->Rename(/*NewName=*/ NULL, /*NewOuter=*/ CopyGraph);
			Comment->MaterialExpressionComment = DuplicateExpression(ToRawPtr(Comment->MaterialExpressionComment));
			Comment->MaterialExpressionComment->SubgraphExpression = NewSubgraphExpression;

			// GraphNode is transient so it won't be duplicated. 
			Comment->MaterialExpressionComment->GraphNode = Comment;
		}
	}
}

void FMaterialEditor::CleanUnusedExpressions()
{
	TArray<UEdGraphNode*> UnusedNodes;

	Material->MaterialGraph->GetUnusedExpressions(UnusedNodes);

	if (UnusedNodes.Num() > 0 && CheckExpressionRemovalWarnings(UnusedNodes))
	{
		{
			// Kill off expressions referenced by the material that aren't reachable.
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MaterialEditorCleanUnusedExpressions", "Material Editor: Clean Unused Expressions") );
				
			FScopedMaterialChecker Checker{ this, __FUNCTION__ };
			ModifyMaterial();

			for (int32 Index = 0; Index < UnusedNodes.Num(); ++Index)
			{
				UMaterialGraphNode* GraphNode = CastChecked<UMaterialGraphNode>(UnusedNodes[Index]);
				UMaterialExpression* MaterialExpression = GraphNode->MaterialExpression;

				FBlueprintEditorUtils::RemoveNode(NULL, GraphNode, true);

				if (PreviewExpression == MaterialExpression)
				{
					SetPreviewExpression(NULL);
				}

				MaterialExpression->Modify();
				Material->GetExpressionCollection().RemoveExpression(MaterialExpression);
				Material->RemoveExpressionParameter(MaterialExpression);
				// Make sure the deleted expression is caught by gc
				MaterialExpression->MarkAsGarbage();
			}

			Material->MaterialGraph->LinkMaterialExpressionsFromGraph();
		} // ScopedTransaction

		if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			FocusedGraphEd->ClearSelectionSet();
			FocusedGraphEd->NotifyGraphChanged();
		}

		SetMaterialDirty();
	}
}

void FMaterialEditor::FixupMismatchedSamplerExpressions()
{
	// Find all expressions that need to be fixed.
	TArray<UMaterialExpressionTextureBase*> ExpressionsToFix;
	for (UMaterialExpression* MaterialExpression : Material->GetExpressions())
	{
		if (UMaterialExpressionTextureBase* TextureBase = Cast<UMaterialExpressionTextureBase>(MaterialExpression))
		{
			FString SamplerTypeError;
			if (!MaterialExpressionUtils::VerifySamplerType(GEditor->PreviewPlatform.GetShaderPlatform(), nullptr, TextureBase->Texture, TextureBase->SamplerType, SamplerTypeError))
			{
				ExpressionsToFix.Add(TextureBase);
			}
		}
	}

	if (ExpressionsToFix.Num() > 0)
	{
		if (EAppReturnType::Yes != FMessageDialog::Open(EAppMsgType::YesNo,
														FText::Format(
															NSLOCTEXT("UnrealEd", "Prompt_MaterialEditorFixupTextureSamplers", "There are \"{0}\" texture sampler(s) with the incorrect sampler picked for their texture.\nWould you like to automatically fix them up?"),
															ExpressionsToFix.Num())))
		{
			return;
		}

		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorFixupSamplerExpressions", "Material Editor: Fixup Sampler Expressions"));
			
		{
			FScopedMaterialChecker Checker{ this, __FUNCTION__ };
			ModifyMaterial();

			for (UMaterialExpressionTextureBase* TextureBase : ExpressionsToFix)
			{
				TextureBase->Modify();
				TextureBase->AutoSetSampleType();
			}
		}

		UpdateMaterialAfterGraphChange();
	}
}

void FMaterialEditor::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	// no-op if there are no derived materials used
	if (DerivedMaterialInstances.Num() == 0 && OriginalDerivedMaterialInstances.Num() == 0)
	{
		return;
	}

	DerivedMaterialInstances.Empty();
	OriginalDerivedMaterialInstances.Empty();
	// will free all referenced resources via FMaterial::DeleteMaterialsOnRenderThread
	MaterialStatsManager->SetMaterial(bStatsFromPreviewMaterial ? Material : OriginalMaterial, {});
}

void FMaterialEditor::OnPostWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	MaterialStatsManager->SignalMaterialChanged();
}

bool FMaterialEditor::CheckExpressionRemovalWarnings(const TArray<UEdGraphNode*>& NodesToRemove)
{
	FString FunctionWarningString;
	bool bFirstExpression = true;
	for (int32 Index = 0; Index < NodesToRemove.Num(); ++Index)
	{
		if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(NodesToRemove[Index]))
		{
			UMaterialExpressionFunctionInput* FunctionInput = Cast<UMaterialExpressionFunctionInput>(GraphNode->MaterialExpression);
			UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(GraphNode->MaterialExpression);

			if (FunctionInput)
			{
				if (!bFirstExpression)
				{
					FunctionWarningString += TEXT(", ");
				}
				bFirstExpression = false;
				FunctionWarningString += FunctionInput->InputName.ToString();
			}

			if (FunctionOutput && MaterialFunction && MaterialFunction->GetMaterialFunctionUsage() == EMaterialFunctionUsage::Default)
			{
				if (!bFirstExpression)
				{
					FunctionWarningString += TEXT(", ");
				}
				bFirstExpression = false;
				FunctionWarningString += FunctionOutput->OutputName.ToString();
			}
		}
	}

	if (FunctionWarningString.Len() > 0)
	{
		if (EAppReturnType::Yes != FMessageDialog::Open( EAppMsgType::YesNo,
				FText::Format(
					NSLOCTEXT("UnrealEd", "Prompt_MaterialEditorDeleteFunctionInputs", "Delete function inputs or outputs \"{0}\"?\nAny materials which use this function will lose their connections to these function inputs or outputs once deleted."),
					FText::FromString(FunctionWarningString) )))
		{
			// User said don't delete
			return false;
		}
	}

	return true;
}

void FMaterialEditor::RemoveSelectedExpressionFromFavorites()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt))
			{
				MaterialExpressionClasses::Get()->RemoveMaterialExpressionFromFavorites(GraphNode->MaterialExpression->GetClass());
				EditorOptions->FavoriteExpressions.Remove(GraphNode->MaterialExpression->GetClass()->GetName());
				EditorOptions->SaveConfig();
			}
		}
	}
}

void FMaterialEditor::AddSelectedExpressionToFavorites()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*NodeIt))
			{
				MaterialExpressionClasses::Get()->AddMaterialExpressionToFavorites(GraphNode->MaterialExpression->GetClass());
				EditorOptions->FavoriteExpressions.AddUnique(GraphNode->MaterialExpression->GetClass()->GetName());
				EditorOptions->SaveConfig();
			}
		}
	}
}

void FMaterialEditor::SetEditorInstanceForWidgets() const
{
	MaterialCustomPrimitiveDataWidget->SetEditorInstance(MaterialEditorInstance);
	MaterialExpressionsOverviewWidget->SetEditorInstance(MaterialEditorInstance);
}

void FMaterialEditor::JumpToNodeByGuid(const FGuid& InGuid)
{
	if (!InGuid.IsValid() || !IsValid(Material))
	{
		return;
	}

	UMaterialExpression* Expression = Material->FindExpressionByGUID<UMaterialExpression>(InGuid);

	if (!Expression)
	{
		return;
	}

	// A parameter with a valid opened graph node can be jumped to straight away.
	if (IsValid(Expression->GraphNode))
	{
		JumpToExpression(Expression);
		return;
	}

	if (!GEngine)
	{
		return;
	}

	UMaterial* ParentMaterial = Cast<UMaterial>(Expression->GetOuter());
	UMaterialFunction* ParentFunction = Cast<UMaterialFunction>(Expression->GetOuter());
	UObject* AssetToOpen = nullptr;

	if (ParentMaterial && ParentMaterial != OriginalMaterial)
	{
		AssetToOpen = ParentMaterial;
	}
	else if (ParentFunction && ParentFunction != MaterialFunction)
	{
		AssetToOpen = ParentFunction;
	}

	if (!AssetToOpen)
	{
		return;
	}
		
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

	if (!AssetEditorSubsystem)
	{
		return;
	}

	IAssetEditorInstance* CurrentInstance = AssetEditorSubsystem->FindEditorForAsset(AssetToOpen, /* Focus */ true);

	if (!CurrentInstance || CurrentInstance->GetEditorName() != GetEditorName())
	{
		IMaterialEditorModule& MaterialEditorModule = FModuleManager::LoadModuleChecked<IMaterialEditorModule>("MaterialEditor");

		if (ParentMaterial)
		{
			if (TSharedPtr<IMaterialEditor> NewEditor = MaterialEditorModule.CreateMaterialEditor(EToolkitMode::Standalone, nullptr, ParentMaterial))
			{
				CurrentInstance = NewEditor.Get();
			}
		}
		else if (ParentFunction)
		{
			if (TSharedPtr<IMaterialEditor> NewEditor = MaterialEditorModule.CreateMaterialEditor(EToolkitMode::Standalone, nullptr, ParentFunction))
			{
				CurrentInstance = NewEditor.Get();
			}
		}

		if (!CurrentInstance)
		{
			return;
		}
	}

	// We've ensured that this is a valid cast by checking GetEditorName()
	IMaterialEditor* MaterialAssetEditor = static_cast<IMaterialEditor*>(CurrentInstance);
	UMaterial* PreviewMaterial = Cast<UMaterial>(MaterialAssetEditor->GetMaterialInterface());

	if (!PreviewMaterial)
	{
		return;
	}

	Expression = PreviewMaterial->FindExpressionByGUID<UMaterialExpression>(InGuid);

	if (Expression)
	{
		MaterialAssetEditor->JumpToExpression(Expression);
	}
}

void FMaterialEditor::UpdateDetailView()
{
	GetDetailView()->InvalidateCachedState();
}

void FMaterialEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	if (bFocusSelectionInProgress)
	{
		return;
	}

	TArray<UObject*> SelectedObjects;

	UObject* EditObject = Material;
	if (MaterialFunction)
	{
		EditObject = MaterialFunction;
	}

	bSelectRegularNode = false;
	if( NewSelection.Num() == 0 )
	{
		SelectedObjects.Add(EditObject);
	}
	else
	{
		for(TSet<class UObject*>::TConstIterator SetIt(NewSelection);SetIt;++SetIt)
		{
			if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(*SetIt))
			{
				bSelectRegularNode = true;
				SelectedObjects.Add(GraphNode->MaterialExpression);
			}
			else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(*SetIt))
			{
				SelectedObjects.Add(CommentNode->MaterialExpressionComment);
			}
			else
			{
				SelectedObjects.Add(EditObject);
			}
		}
	}

	GetDetailView()->SetObjects( SelectedObjects, true );
	FocusDetailsPanel();
	
	if (SelectedObjects.Num() == 1 && SelectedObjects[0] == Material)
	{
		// Update when we display the material details, in case any MF parameter was modified
		UpdateMaterialExpressionGroups();
	}

	if (bHideUnrelatedNodes)
	{
		if (!bLockNodeFadeState)
		{
			if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
			{
				FocusedGraphEd->ResetAllNodesUnrelatedStates();
			}

			if (bSelectRegularNode)
			{
				HideUnrelatedNodes();
			}
		}
		else if(bSelectFocusedNodes)
		{
			// We still want to select nodes
			if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
			{
				const TArray<UMaterialGraphNode*> NodesToSelect = GetSelectionRelatedNodes();
				SelectMaterialGraphNodes(*FocusedGraphEd, NodesToSelect);
			}
		}
	}
}

void FMaterialEditor::OnNodeDoubleClicked(class UEdGraphNode* Node)
{
	UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Node);

	if (GraphNode && GraphNode->MaterialExpression)
	{
		TOptional<FLinearColor> InitialColor;
		bool bCanEditAlpha = true;
		if(UMaterialExpressionConstant3Vector* Constant3Expression = Cast<UMaterialExpressionConstant3Vector>(GraphNode->MaterialExpression))
		{
			InitialColor = Constant3Expression->Constant;
			bCanEditAlpha = false;
		}
		else if(UMaterialExpressionConstant4Vector* Constant4Expression = Cast<UMaterialExpressionConstant4Vector>(GraphNode->MaterialExpression))
		{
			InitialColor = Constant4Expression->Constant;
		}
		else if (UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(GraphNode->MaterialExpression))
		{
			InitialColor = InputExpression->PreviewValue;
		}
		else if (UMaterialExpressionVectorParameter* VectorExpression = Cast<UMaterialExpressionVectorParameter>(GraphNode->MaterialExpression))
		{
			InitialColor = VectorExpression->DefaultValue;
		}

		if (InitialColor.IsSet())
		{
			TWeakObjectPtr<UObject> ColorPickerObject = GraphNode->MaterialExpression;

			// Open a color picker 
			FColorPickerArgs PickerArgs = FColorPickerArgs(InitialColor.GetValue(), FOnLinearColorValueChanged::CreateSP(this, &FMaterialEditor::OnColorPickerCommitted, ColorPickerObject));
			PickerArgs.ParentWidget = FocusedGraphEdPtr.Pin();
			PickerArgs.bUseAlpha = bCanEditAlpha;
			PickerArgs.bOnlyRefreshOnOk = false;
			PickerArgs.bOnlyRefreshOnMouseUp = true;
			PickerArgs.bExpandAdvancedSection = true;
			PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );
			PickerArgs.OptionalOwningDetailsView = MaterialDetailsView;
			OpenColorPicker(PickerArgs);
		}

		UMaterialExpressionTextureSample* TextureExpression = Cast<UMaterialExpressionTextureSample>(GraphNode->MaterialExpression);
		UMaterialExpressionTextureSampleParameter* TextureParameterExpression = Cast<UMaterialExpressionTextureSampleParameter>(GraphNode->MaterialExpression);
		UMaterialExpressionMaterialFunctionCall* FunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(GraphNode->MaterialExpression);
		UMaterialExpressionCollectionParameter* CollectionParameter = Cast<UMaterialExpressionCollectionParameter>(GraphNode->MaterialExpression);
		UMaterialExpressionCollectionTransform* CollectionTransform = Cast<UMaterialExpressionCollectionTransform>(GraphNode->MaterialExpression);

		TArray<UObject*> ObjectsToView;
		UObject* ObjectToEdit = NULL;

		if (TextureExpression && TextureExpression->Texture)
		{
			ObjectsToView.Add(TextureExpression->Texture);
		}
		else if (TextureParameterExpression && TextureParameterExpression->Texture)
		{
			ObjectsToView.Add(TextureParameterExpression->Texture);
		}
		else if (FunctionExpression && FunctionExpression->MaterialFunction)
		{
			ObjectToEdit = FunctionExpression->MaterialFunction;
		}
		else if (CollectionParameter && CollectionParameter->Collection)
		{
			ObjectToEdit = CollectionParameter->Collection;
		}
		else if (CollectionTransform && CollectionTransform->Collection)
		{
			ObjectToEdit = CollectionTransform->Collection;
		}

		if (ObjectsToView.Num() > 0)
		{
			GEditor->SyncBrowserToObjects(ObjectsToView);
		}
		if (ObjectToEdit)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ObjectToEdit);
		}

		// Double click actions for named reroute nodes
		if (GraphNode->MaterialExpression->IsA<UMaterialExpressionNamedRerouteDeclaration>())
		{
			OnSelectNamedRerouteUsages();
		}
		else if (GraphNode->MaterialExpression->IsA<UMaterialExpressionNamedRerouteUsage>())
		{
			OnSelectNamedRerouteDeclaration();
		}
		else if (GraphNode->MaterialExpression->IsA<UMaterialExpressionComposite>())
		{
			if (UMaterialGraphNode_Composite* CompositeGraphNode = Cast<UMaterialGraphNode_Composite>(GraphNode))
			{
				if (CompositeGraphNode->GetJumpTargetForDoubleClick())
				{
					if (UObject* HyperlinkTarget = CompositeGraphNode->GetJumpTargetForDoubleClick())
					{
						FMaterialEditorUtilities::BringFocusAttentionOnObject(HyperlinkTarget);
					}
				}
			}
		}
	}
}

void FMaterialEditor::OnRenameNode()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd)
	{
		return;
	}

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt);
		if (SelectedNode != nullptr && SelectedNode->GetCanRenameNode())
		{
			bool ToRename = true;
			FocusedGraphEd->IsNodeTitleVisible(SelectedNode, ToRename);
			break;
		}
	}
}

bool FMaterialEditor::CanRenameNodes() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != nullptr) && Node->GetCanRenameNode())
		{
			return true;
		}
	}
	return false;
}

void FMaterialEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		FString NewName = NewText.ToString().TrimStartAndEnd();
		if (NewName.IsEmpty())
		{
			// Ignore empty names
			return;
		}
		else if (NewName.Len() >= NAME_SIZE) {
			UE_LOGF(LogMaterialEditor, Warning, "New material graph node name '%ls...' exceeds maximum length of %d and thus was truncated.", *NewName.Left(8), NAME_SIZE - 1);
			NewName = NewName.Left(NAME_SIZE - 1);
		}

		const FScopedTransaction Transaction( LOCTEXT( "RenameNode", "Rename Node" ) );
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewName);
		UpdateGenerator();
		SetEditorInstanceForWidgets();

		if (NamedReroutesPanel.IsValid())
		{
			NamedReroutesPanel->RequestRefresh();
		}
	}
}

bool FMaterialEditor::OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage)
{
	bool bValid = true;

	UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(NodeBeingChanged);
	if( MaterialNode && MaterialNode->MaterialExpression && MaterialNode->MaterialExpression->IsA<UMaterialExpressionParameter>() )
	{
		if( NewText.ToString().Len() >= NAME_SIZE )
		{
			OutErrorMessage = FText::Format( LOCTEXT("MaterialEditorExpressionError_NameTooLong", "Parameter names must be less than {0} characters"), FText::AsNumber(NAME_SIZE));
			bValid = false;
		}
	}

	return bValid;
}

FReply FMaterialEditor::OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph)
{
	UEdGraph* Graph = InGraph;
	if (FMaterialEditorSpawnNodeCommands::IsRegistered())
	{
		TSharedPtr< FEdGraphSchemaAction > Action = FMaterialEditorSpawnNodeCommands::Get().GetGraphActionByChord(InChord, InGraph);

		if (Action.IsValid())
		{
			TArray<UEdGraphPin*> DummyPins;
			Action->PerformAction(Graph, DummyPins, InPosition);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

void FMaterialEditor::UpdateStatsMaterials()
{
	if (bShowBuiltinStats && bStatsFromPreviewMaterial)
	{	
		UMaterial* StatsMaterial = Material;
		FString EmptyMaterialName = FString(TEXT("MEStatsMaterial_Empty_")) + Material->GetName();
		EmptyMaterial = (UMaterial*)StaticDuplicateObject(Material, GetTransientPackage(), *EmptyMaterialName, ~RF_Standalone, UPreviewMaterial::StaticClass());

		EmptyMaterial->GetExpressionCollection().Empty();

		//Disconnect all properties from the expressions
		for (int32 PropIdx = 0; PropIdx < MP_MAX; ++PropIdx)
		{
			FExpressionInput* ExpInput = EmptyMaterial->GetExpressionInputForProperty((EMaterialProperty)PropIdx);
			if(ExpInput)
			{
				ExpInput->Expression = NULL;
			}
		}
		EmptyMaterial->bAllowDevelopmentShaderCompile = Material->bAllowDevelopmentShaderCompile;
		EmptyMaterial->PreEditChange(NULL);
		EmptyMaterial->PostEditChange();
	}

	// Also request to update the Substrate slab.
	SubstrateWidget->UpdateFromMaterial();
}

void FMaterialEditor::NotifyExternalMaterialChange()
{
	MaterialStatsManager->SignalMaterialChanged();
}

void FMaterialEditor::RefreshGraphFromOriginal()
{
	// Rebuild inside an in-flight transaction would clobber it.
	if (GIsTransacting)
	{
		return;
	}

	const bool bIsFunctionEditor = (MaterialFunction != nullptr);

	if (bIsFunctionEditor)
	{
		if (!MaterialFunction->ParentFunction || !Material)
		{
			return;
		}
	}
	else if (!OriginalMaterial)
	{
		return;
	}

	// Surface the swap so external rebuilds don't look like a glitch. Routine toasts suppress
	// if one is still on screen (no notification stacking); the discarded-edits toast bypasses
	// suppression because losing user work silently is worse than stacking.
	const bool bDiscardingDirty = bMaterialDirty;
	if (bDiscardingDirty || !ExternalRefreshNotificationPtr.IsValid())
	{
		FNotificationInfo Info(NSLOCTEXT("MaterialEditor", "ExternalRefreshToast", "Material refreshed from external edit."));
		Info.ExpireDuration = 4.0f;
		Info.bFireAndForget = true;
		if (bDiscardingDirty)
		{
			Info.SubText = NSLOCTEXT("MaterialEditor", "ExternalRefreshToastDirty", "Pending in-editor changes were discarded.");
		}
		ExternalRefreshNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	}

	// Carrying PreviewExpression across the swap would leave it pointing into the orphaned old asset.
	if (PreviewExpression)
	{
		SetPreviewExpression(nullptr);
	}

	// Cached preview material samples Material's state at lazy init; drop it so the next preview picks up the new asset.
	ExpressionPreviewMaterial = nullptr;

	if (bIsFunctionEditor)
	{
		UMaterialFunction* SourceFunction = MaterialFunction->ParentFunction;

		UMaterialFunction* NewPreview = (UMaterialFunction*)StaticDuplicateObject(SourceFunction, GetTransientPackage(), NAME_None, ~RF_Standalone, UMaterialFunction::StaticClass());
		NewPreview->ParentFunction = SourceFunction;
		NewPreview->SetFlags(RF_Transactional);

		// Function-mode wiring: throwaway Material shadows the function's expression collection (see InitMaterialEditor).
		NewPreview->MaterialGraph = Material->MaterialGraph;
		NewPreview->EditorMaterial = Material;
		Material->AssignExpressionCollection(NewPreview->GetExpressionCollection());
		Material->bEnableNewHLSLGenerator = NewPreview->bEnableNewHLSLGenerator;

		Material->MaterialGraph->MaterialFunction = NewPreview;
		MaterialFunction = NewPreview;

		RestoreExpressionBackReferences(Material);
		RebindExpressionsToMaterial(Material);
	}
	else
	{
		if (OriginalMaterial->IsSelected())
		{
			GEditor->GetSelectedObjects()->Deselect(OriginalMaterial);
		}

		UMaterial* OldMaterial = Material;
		UMaterialGraph* ExistingGraph = Material ? Material->MaterialGraph : nullptr;

		UMaterial* NewPreview = DuplicatePreviewMaterial(OriginalMaterial);
		CopyMaterialUsageFlags(OriginalMaterial, NewPreview);

		// Reparent the existing UMaterialGraph onto the new preview so the user's pan/zoom survives the swap.
		if (ExistingGraph)
		{
			ExistingGraph->Rename(nullptr, NewPreview, REN_DontCreateRedirectors | REN_DoNotDirty);
			ExistingGraph->Material = NewPreview;
			NewPreview->MaterialGraph = ExistingGraph;
		}

		Material = NewPreview;
		// Drop the old preview from the toolkit's strong-rooted EditingObjects so it can GC; register the new one.
		if (OldMaterial)
		{
			RemoveEditingObject(OldMaterial);
		}
		AddEditingObject(NewPreview);
		SetPreviewMaterial(Material);
		FMaterialEditorUtilities::InitExpressions(Material);
	}

	Material->BuildEditorParameterList();
	// RebuildGraph appends to SubGraphs but never empties it; clear before the rebuild so composite-expression
	// materials don't accumulate stale subgraph entries pointing at the orphaned old preview.
	Material->MaterialGraph->SubGraphs.Empty();
	Material->MaterialGraph->RebuildGraph();

	if (MaterialEditorInstance)
	{
		if (!bIsFunctionEditor)
		{
			MaterialEditorInstance->PreviewMaterial = Material;
		}
		SetEditorInstanceForWidgets();
		UpdateGenerator();
	}

	if (TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin())
	{
		// RebuildGraph created fresh subgraph instances; an open subgraph document tab is now showing an orphan.
		// Close the stale tab and reinvoke the root graph editor (matches PostUndo).
		UEdGraph* CurrentGraph = FocusedGraphEd->GetCurrentGraph();
		if (CurrentGraph && CurrentGraph != Material->MaterialGraph)
		{
			CloseDocumentTab(CurrentGraph);
			DocumentManager->CleanInvalidTabs();
			DocumentManager->RefreshAllTabs();
			GetTabManager()->TryInvokeTab(FMaterialEditorTabs::GraphEditor);
		}

		// Active graph can change above, re-acquire ptr
		if (TSharedPtr<SGraphEditor> RefocusedGraphEd = FocusedGraphEdPtr.Pin())
		{
			RefocusedGraphEd->ClearSelectionSet();
			RefocusedGraphEd->NotifyGraphChanged();
		}
	}

	UpdatePreviewMaterial(/*bForce=*/true);
	ForceRefreshExpressionPreviews();
	bMaterialDirty = false;
}

void FMaterialEditor::FocusDetailsPanel()
{
	if (SpawnedDetailsTab.IsValid() && !SpawnedDetailsTab.Pin()->IsForeground())
	{
		SpawnedDetailsTab.Pin()->DrawAttention();
	}
}

void FMaterialEditor::RebuildInheritanceList()
{
	if (!MaterialFunction)
	{
		MaterialChildList.Empty();

		UMaterialEditingLibrary::GetChildInstances(OriginalMaterial, MaterialChildList);
	}
}

FMaterialEditor::FScopedMaterialChecker::FScopedMaterialChecker(FMaterialEditor* InEditor, const char* InDebugEditName)
	: Editor{ InEditor }
	, DebugEditName{ InDebugEditName }
{
}

FMaterialEditor::FScopedMaterialChecker::~FScopedMaterialChecker()
{
	if (Editor->Material->GetEditorOnlyData()->ExpressionCollection.Expressions.Remove(nullptr) > 0)
	{
		UE_LOGF(LogMaterialEditor, Warning, 
			"Material Editor internal error after operation '%s': null expressions found in material '%ls' expressions list.", 
			DebugEditName, *Editor->Material->GetName());
	}
}

#undef LOCTEXT_NAMESPACE
