// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Stats/Stats.h"
#include "Misc/Guid.h"
#include "UObject/GCObject.h"
#include "Misc/NotifyHook.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/InputChord.h"
#include "EditorUndoClient.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Toolkits/IToolkitHost.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "IMaterialEditor.h"
#include "IDetailsView.h"
#include "SMaterialEditorViewport.h"
#include "Materials/Material.h"
#include "Tickable.h"
#include "UObject/WeakFieldPtr.h"

struct FAssetData;
struct FEdGraphEditAction;
struct FToolMenuSection;
class FCanvas;
class FMaterialCompiler;
class FScopedTransaction;
class IMessageLogListing;
class SFindInMaterial;
class SGraphEditor;
class SMaterialPalette;
class UEdGraph;
class UEdGraphPin;
class UFactory;
class UMaterialEditorOptions;
class UMaterialExpressionComment;
class UMaterialExpressionComposite;
class UMaterialExpressionNamedRerouteDeclaration;
class UMaterialInstance;
class UMaterialGraphNode;
struct FGraphAppearanceInfo;
class UMaterialFunctionInstance;
struct FMaterialCachedExpressionData;
class SMaterialEditorSubstrateWidget;

typedef TSet<class UObject*> FGraphPanelSelectionSet;

/**
 * Class for rendering previews of material expressions in the material editor's linked object viewport.
 */
class FMatExpressionPreview : public FEditorMaterialResourceBase, public FMaterialRenderProxy
{
public:
	FMatExpressionPreview();
	FMatExpressionPreview(UMaterialExpression* InExpression);

	virtual ~FMatExpressionPreview();

	virtual bool PrepareDestroy_GameThread() override
	{
		FMaterial::PrepareDestroy_GameThread();
		return true; // always need render thread callback
	}

	virtual void PrepareDestroy_RenderThread() override
	{
		FMaterial::PrepareDestroy_RenderThread();
		ReleaseResource();
	}

	void AddReferencedObjects(FReferenceCollector& Collector);

	/**
	 * Should the shader for this material with the given platform, shader type and vertex 
	 * factory type combination be compiled
	 *
	 * @param ShaderType	Which shader is being compiled
	 * @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	 *
	 * @return true if the shader should be compiled
	 */
	virtual bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const override;
	
	////////////////
	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		if (GetRenderingThreadShaderMap())
		{
			return this;
		}
		return nullptr;
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override
	{
		if (Expression.IsValid() && Expression->Material)
		{
			return Expression->Material->GetRenderProxy()->GetParameterValue(Type, ParameterInfo, OutValue, Context);
		}
		return false;
	}

	// Material properties.
	/** Entry point for compiling a specific material property.  This must call SetMaterialProperty. */
	virtual int32 CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const override;

	virtual UMaterialExpression* GetMaterialGraphNodePreviewExpression() const override { return Expression.Get(); }

	virtual EMaterialDomain GetMaterialDomain() const override { return MD_Surface; }
	virtual FString GetMaterialUsageDescription() const override { return FString::Printf(TEXT("FMatExpressionPreview %s"), Expression.IsValid() ? *Expression->GetName() : TEXT("NULL")); }
	virtual bool IsPreview() const override { return true; }
	virtual bool IsTwoSided() const override { return false; }
	virtual bool IsThinSurface() const override { return false; }
	virtual bool IsDitheredLODTransition() const override { return false; }
	virtual bool IsLightFunction() const override { return false; }
	virtual bool IsDeferredDecal() const override { return false; }
	virtual bool IsUIMaterial() const override { return false; }
	virtual bool IsPostProcessMaterial() const override { return false; }
	virtual bool IsVolumetricPrimitive() const override { return false; }
	virtual bool IsSpecialEngineMaterial() const override { return false; }
	virtual bool IsWireframe() const override { return false; }
	virtual bool IsMasked() const override { return false; }
	virtual enum EBlendMode GetBlendMode() const override { return BLEND_Translucent; }
	virtual bool GetRootNodeOverridesDefaultRefraction()const override { return false; } 
	virtual FMaterialShadingModelField GetShadingModels() const override { return MSM_Unlit; }
	virtual bool IsShadingModelFromMaterialExpression() const override { return false; }
	virtual float GetOpacityMaskClipValue() const override { return 0.5f; }
	virtual bool GetCastDynamicShadowAsMasked() const override { return false; }
	virtual FString GetFriendlyName() const override { return FString::Printf(TEXT("FMatExpressionPreview %s"), Expression.IsValid() ? *Expression->GetName() : TEXT("NULL")); }
	/**
	 * Should shaders compiled for this material be saved to disk?
	 */
	virtual bool IsPersistent() const override { return false; }
	virtual FGuid GetMaterialId() const override { return Id; }
	const UMaterialExpression* GetExpression() const
	{
		return Expression.Get();
	}

	// This material interface is solely needed for the translator to be able to parse the graph for the Substrate tree.
	virtual UMaterialInterface* GetMaterialInterface() const override;

	virtual void NotifyCompilationFinished() override;
	virtual bool IsUsingNewHLSLGenerator() const override;

	friend FArchive& operator<< ( FArchive& Ar, FMatExpressionPreview& V )
	{
		return Ar << V.Expression;
	}

	virtual void GatherExpressionsForCustomInterpolators(TArray<UMaterialExpression*>& OutExpressions) const override
	{
		if(Expression.IsValid() && Expression->Material)
		{
			Expression->Material->GetAllExpressionsForCustomInterpolators(OutExpressions);
		}
	}

	/**
	 * Checks that no pre-compilation errors have been detected and if so it reports them using specified compiler.
	 * @return whether no errors occurred.
	 */
	virtual bool CheckInValidStateForCompilation(class FMaterialCompiler* Compiler) const override;

	float UnrelatedNodesOpacity;

	int32 GetCompiledPreviewNumChannels() const { return CompiledPreviewNumChannels; }

private:
	TWeakObjectPtr<UMaterialExpression> Expression;
	FGuid Id;
	mutable int32 CompiledPreviewNumChannels = 0;
};

/** Wrapper for each material expression, including a trimmed name */
struct FMaterialExpression
{
	FString Name;
	UClass* MaterialClass;
	FText CreationDescription;
	FText CreationName;

	friend bool operator==(const FMaterialExpression& A,const FMaterialExpression& B)
	{
		return A.MaterialClass == B.MaterialClass;
	}
};

/** Static array of categorized material expression classes. */
struct FCategorizedMaterialExpressionNode
{
	FText	CategoryName;
	TArray<FMaterialExpression> MaterialExpressions;
};

/** Used to display material information, compile errors etc. */
struct FMaterialInfo
{
	FString Text;
	FLinearColor Color;

	FMaterialInfo(const FString& InText, const FLinearColor& InColor)
		: Text(InText)
		, Color(InColor)
	{}
};

/**
 * Material Editor class
 */
class FMaterialEditor : public IMaterialEditor, public FGCObject, public FTickableGameObject, public FEditorUndoClient, public FNotifyHook
{
public:
	// @todo This is a hack for now until we reconcile the default toolbar with application modes
	void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
public:
	/** Initializes the editor to use a material. Should be the first thing called. */
	void InitEditorForMaterial(UMaterial* InMaterial);

	/** Initializes the editor to use a material function. Should be the first thing called. */
	void InitEditorForMaterialFunction(UMaterialFunction* InMaterialFunction);

	/**
	 * Edits the specified material object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The material or material function to edit
	 */
	void InitMaterialEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit );

	/** Constructor */
	FMaterialEditor();

	virtual ~FMaterialEditor();
	
	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMaterialEditor");
	}

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void InitToolMenuContext(struct FToolMenuContext& MenuContext) override;


	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("Engine/Rendering/Materials"));
	}

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** The material instance applied to the preview mesh. */
	virtual UMaterialInterface* GetMaterialInterface() const override;
	
	/**
	 * Draws messages on the specified viewport and canvas.
	 */
	virtual void DrawMessages( FViewport* Viewport, FCanvas* Canvas ) override;

	/**
	 * Recenter the editor to either the material inputs or the first material function output
	 */
	void RecenterEditor();

	/** Passes instructions to the preview viewport */
	bool SetPreviewAsset(UObject* InAsset);
	bool SetPreviewAssetByName(const TCHAR* InAssetName);
	void SetPreviewMaterial(UMaterialInterface* InMaterialInterface);
	
	/**
	 * Refreshes the viewport containing the preview mesh.
	 */
	void RefreshPreviewViewport();

	/** Pushes the currently set custom primitive data to the preview mesh component. */
	void UpdatePreviewCustomPrimitiveData();

	/** Regenerates the code view widget with new text */
	void RegenerateCodeView(bool bForce=false);
	
	/**
	 * Recompiles the material used in the preview window.
	 */
	void UpdatePreviewMaterial(bool bForce=false);

	/**
	 * Updates the original material with the changes made in the editor
	 * @return true if the update was successful.  False if update was canceled (eg attempted to update Default Material with errors, which would cause a crash).
	 */
	bool UpdateOriginalMaterial();

	/**
	 * Updates list of Material Info used to show stats
	 */
	void UpdateMaterialInfoLog();
	void UpdateMaterialPlatformStats();

	/**
	 * Updates flags on the Material Nodes to avoid expensive look up calls when rendering
	 */
	void UpdateGraphNodeStates();

	/**
	 * Updates flags on the Material Nodes for a single graph and it's subgraphs.
	 * 
	 * @param	Graph					Material graph to update
	 * @param	ErrorMaterialResource	Material Resource containing known errors to draw for graph
	 * @param	VisibleExpressions		List of expressions that should be visible (As in not grayed out / disabled).
	 * @param	bShowAllNodes           True implies all nodes should be visible.
	 * @return	True if we updated the error state of this material during graph update.
	 */
	bool UpdateGraphNodeState(UEdGraph* Graph, const FMaterialResource* ErrorMaterialResource, TArray<UMaterialExpression*>& VisibleExpressions, bool bShowAllNodes);
	
	// Widget Accessors
	TSharedRef<class IDetailsView> GetDetailView() const {return MaterialDetailsView.ToSharedRef();}
	
	virtual void UpdateDetailView() override;

	// FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual TStatId GetStatId() const override;

	/** Pushes the PreviewMesh assigned the the material instance to the thumbnail info */
	static void UpdateThumbnailInfoPreviewMesh(UMaterialInterface* MatInterface);

	/** Sets the expression to be previewed. */
	void SetPreviewExpression(UMaterialExpression* NewPreviewExpression);

	/** Pan the view to center on a particular node */
	void JumpToNode(const UEdGraphNode* Node);

	/** Called when graph editor focus is changed */
	virtual void OnGraphEditorFocused(const TSharedRef<class SGraphEditor>& InGraphEditor);

	/** Called when the graph editor tab is backgrounded */
	virtual void OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor);

	// Finds any open tabs containing the specified document and adds them to the specified array; returns true if at least one is found
	bool FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results);

	/** Open workflow document tab */
	TSharedPtr<SDockTab> OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause);

	/** Close workflow document tab */
	void CloseDocumentTab(const UObject* DocumentID);

	// IMaterial Editor Interface
	virtual UMaterialExpression* CreateNewMaterialExpression(UClass* NewExpressionClass, const FVector2D& NodePos, bool bAutoSelect, bool bAutoAssignResource, const class UEdGraph* Graph = nullptr) override;
	virtual UMaterialExpressionComposite* CreateNewMaterialExpressionComposite(const FVector2D& NodePos, const class UEdGraph* Graph = nullptr) override;
	virtual UMaterialExpressionComment* CreateNewMaterialExpressionComment(const FVector2D& NodePos, const class UEdGraph* Graph = nullptr) override;
	virtual void ForceRefreshExpressionPreviews() override;
	virtual void AddToSelection(UMaterialExpression* Expression) override;
	virtual void JumpToExpression(UMaterialExpression* Expression) override;
	virtual void DeleteSelectedNodes() override;
	virtual FText GetOriginalObjectName() const override;
	virtual TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const override;
	virtual void UpdateMaterialAfterGraphChange() override;
	virtual void MarkMaterialDirty() override;
	virtual void JumpToHyperlink(const UObject* ObjectReference) override; 
	virtual bool CanPasteNodes() const override;
	virtual void PasteNodesHere(const FVector2D& Location, const class UEdGraph* Graph = nullptr) override;
	virtual int32 GetNumberOfSelectedNodes() const override;
	virtual TSet<UObject*> GetSelectedNodes() const override;
	virtual void GetBoundsForNode(const UEdGraphNode* InNode, class FSlateRect& OutRect, float InPadding) const override;
	virtual FMatExpressionPreview* GetExpressionPreview(UMaterialExpression* InExpression) override;
	virtual void DeleteNodes(const TArray<class UEdGraphNode*>& NodesToDelete) override;
	virtual void GenerateInheritanceMenu(class UToolMenu* Menu) override;
	virtual void RefreshStatsMaterials() override;

	void DeleteSelectedNodes(bool bShowConfirmation);
	void DeleteNodes(const TArray<class UEdGraphNode*>& NodesToDelete, bool bShowConfirmation);
	FString CopyNodesToBuffer(const FGraphPanelSelectionSet& Nodes);
	FString CopyNodesToBuffer(const TSet<UEdGraphNode*>& Nodes);
	void PasteNodesHereFromBuffer(const FVector2D& Location, const class UEdGraph* Graph, const FString& TextToImport, TMap<FGuid, FGuid>* OutOldToNewGuids);
	void UpdateStatsMaterials();

	/** Gets the extensibility managers for outside entities to extend material editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() { return ToolBarExtensibilityManager; }

	FORCEINLINE bool IsShowingBuiltinStats()
	{
		return bShowBuiltinStats;
	}

	/** call this to notify the editor that the edited material changed from outside */
	virtual void NotifyExternalMaterialChange() override;

	/** Called to bring focus to the details panel */
	void FocusDetailsPanel();

	/** Rebuilds the inheritance list for this material. */
	void RebuildInheritanceList();

	/** Add entry to hierarchy menu */
	static void AddInheritanceMenuEntry(FToolMenuSection& Section, const FAssetData& AssetData, bool bIsFunctionPreviewMaterial);

	virtual void AddGraphEditorPinActionsToContextMenu(FToolMenuSection& InSection) const override;

	/** Overrides function in FEditorUndoClient. Called to see if the context of the current undo/redo operation is a match for the client. */
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;

	/**
	 * Notifies all other editors that the given material with a UserSceneTexture output was loaded or unloaded, so they can refresh their preview window
	 * if needed.  Utility function shared between UMaterial and UMaterialInstance editors.
	 */
	static void NotifyAllUserSceneTextureLoadOrUnload(UMaterialInterface* MaterialInterface);

public:
	/** Find all usages of the given Named Reroute Declaration and show them in the Find Results tab */
	void SelectNamedRerouteUsages(UMaterialExpressionNamedRerouteDeclaration* Declaration);

	/** Set to true when modifications have been made to the material */
	bool bMaterialDirty;

	/** Set to true if stats should be displayed from the preview material. */
	bool bStatsFromPreviewMaterial;

	/** Set to true when editor is being destructed */
	bool bDestructing = false;

	/** The material applied to the preview mesh. */
	TObjectPtr<UMaterial> Material;

	TArray<TObjectPtr<UMaterialInstance>> DerivedMaterialInstances;
	TArray<TObjectPtr<UMaterialInstance>> OriginalDerivedMaterialInstances;
	
	/** The source material being edited by this material editor. Only will be updated when Material's settings are copied over this material */
	TObjectPtr<UMaterial> OriginalMaterial;
	
	/** The material applied to the preview mesh when previewing an expression. */
	TObjectPtr<UMaterial> ExpressionPreviewMaterial;

	/** An empty copy of the preview material. Allows displaying of stats about the built in cost of the current material. */
	TObjectPtr<UMaterial> EmptyMaterial;

	/** The expression currently being previewed.  This is NULL when not in expression preview mode. */
	TObjectPtr<UMaterialExpression> PreviewExpression;

	/** 
	 * Material function being edited.  
	 * If this is non-NULL, a function is being edited and Material is being used to preview it.
	 */
	TObjectPtr<UMaterialFunction> MaterialFunction;
	
	/** The original material or material function being edited by this material editor.. */
	UObject* OriginalMaterialObject;

	/** Configuration class used to store editor settings across sessions. */
	TObjectPtr<UMaterialEditorOptions> EditorOptions;
	
	/** Document manager for workflow tabs */
	TSharedPtr<FDocumentTracker> DocumentManager;

	/** Factory that spawns graph editors; used to look up all tabs spawned by it. */
	TWeakPtr<FDocumentTabFactory> GraphEditorTabFactoryPtr;
	
protected:
	/**
	 * Helper struct used to validate the state of the material after the material editor
	 * performs an operation, searching for any invalid state.
	 */
	struct FScopedMaterialChecker
	{
		UE_NONCOPYABLE(FScopedMaterialChecker);

		/** The material editor performing the operation */
		FMaterialEditor* Editor;

		/** Name of the operation being performed, logged when an issue is reported */
		const char* DebugEditName;
		
		FScopedMaterialChecker(FMaterialEditor* Editor, const char* DebugEditName);
		~FScopedMaterialChecker();
	};

protected:
	//~ FAssetEditorToolkit interface
	virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	virtual void SaveAsset_Execute() override;
	virtual void SaveAssetAs_Execute() override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;

protected:
	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);

	/**
	 * Called when a node is double clicked
	 *
	 * @param	Node	The Node that was clicked
	 */
	void OnNodeDoubleClicked(class UEdGraphNode* Node);

	/**
	 * Called when a node's title is committed for a rename
	 *
	 * @param	NewText				New title text
	 * @param	CommitInfo			How text was committed
	 * @param	NodeBeingChanged	The node being changed
	 */
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/**
	 * Verifies that the node text entered is valid for the node
	 *
	 * @param	NewText			New node text
	 * @param	NodeBeingChanged	The node being changed
	 * @param	OutErrorMessage		Error message to display if text is invalid
	 * @return	True if the text is valid, false otherwise
	 */
	bool OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage);

	/**
	 * Handles spawning a graph node in the current graph using the passed in chord
	 *
	 * @param	InChord		Chord that was just performed
	 * @param	InPosition	Current cursor position
	 * @param	InGraph		Graph that chord was performed in
	 *
	 * @return	FReply	Whether chord was handled
	 */
	FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph);

	/** Calls Modify() on the Material and its editor only data, as well as MaterialFunction (and its editor only data) if present */
	void ModifyMaterial();

	/** Select every node in the graph */
	void SelectAllNodes();
	/** Whether we can select every node */
	bool CanSelectAllNodes() const;

	/** Whether we are able to delete the currently selected nodes */
	bool CanDeleteNodes() const;
	/** Delete only the currently selected nodes that can be duplicated */
	void DeleteSelectedDuplicatableNodes();
	/** Recursively deletes nodes if needed */
	void DeleteNodesInternal(const TArray<class UEdGraphNode*>& NodesToDelete, bool& bHaveExpressionsToDelete, bool& bPreviewExpressionDeleted, bool bForceDeletion = false);

	/** Copy the currently selected nodes */
	void CopySelectedNodes();
	/** Whether we are able to copy the currently selected nodes */
	bool CanCopyNodes() const;

	/** Paste the contents of the clipboard */
	void PasteNodes();

	/** Handle transient properties, and other things that can't be done in PostPasteNode */
	void PostPasteMaterialExpression(UMaterialExpression* NewExpression);

	/** Cut the currently selected nodes */
	void CutSelectedNodes();
	/** Whether we are able to cut the currently selected nodes */
	bool CanCutNodes() const;

	/** Duplicate the currently selected nodes */
	void DuplicateNodes();
	/** Whether we are able to duplicate the currently selected nodes */
	bool CanDuplicateNodes() const;

	/** Called to undo the last action */
	void UndoGraphAction();

	/** Called to redo the last undone action */
	void RedoGraphAction();

	/** On starting to rename node */
	void OnRenameNode();

	/** Check if node can be renamed */
	bool CanRenameNodes() const;

	/** Collapse node group */
	void OnCollapseNodes();
	/** Check if nodes can be collapsed */
	bool CanCollapseNodes() const;

	/** Expand node group */
	void OnExpandNodes();
	/** Check if node can be expanded */
	bool CanExpandNodes() const;

	/** Opens collapsed nodes in a new tab */
	void OnOpenSelectedNodesInNewTab();
	/** Check if selected nodes can be opened in a new tab*/
	bool CanOpenSelectedNodesInNewTab() const;

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();

	void OnStraightenConnections();

	void OnDistributeNodesH();
	void OnDistributeNodesV();

private:
	void OnMessageLogLinkActivated(const class TSharedRef<IMessageToken>& Token);

	/** Builds the toolbar widget for the material editor */
	void ExtendToolbar();
	void RegisterToolBar();

	/** Creates the toolbar buttons. Bound by ExtendToolbar*/
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);

	void GeneratePreviewMenuContent(class UToolMenu* Menu);

	/** Allows editor to veto the setting of a preview asset */
	virtual bool ApproveSetPreviewAsset(UObject* InAsset) override;

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();
	
	/** Updates and returns all groups for all material expressions */
	void UpdateAndGetMaterialExpressionGroups(TArray<FString>* OutGroups);

	/** Updates all groups for all material expressions*/
	void UpdateMaterialExpressionGroups();

	/** Updates the 3D and UI preview viewport visibility based on material domain */
	void UpdatePreviewViewportsVisibility();

	/** Called to notify other material editors when a post process material with a UserSceneTexture output is loaded or unloaded, as the material may be used in their preview */
	void NotifyUserSceneTextureLoadOrUnload();

	//@TODO: these methods are mostly C&P from BlueprintEditor, consider consolidating logic to graph editor. Note: We don't support macros / functions / tunnels / split pins, and also have material expression specific considerations. */
	// void CollapseNodesIntoGraph(UEdGraphNode* InGatewayNode, UMaterialGraphNode* InEntryNode, UMaterialGraphNode* InResultNode, UEdGraph* InSourceGraph, UEdGraph* InDestinationGraph, TSet<UEdGraphNode*>& InCollapsableNodes);
	// void CollapseNodes(TSet<class UEdGraphNode*>& InCollapsableNodes);
	// static void ExpandNode(UEdGraphNode* InNodeToExpand, UEdGraph* InSourceGraph, TSet<UEdGraphNode*>& OutExpandedNodes);
	// void MoveNodesToAveragePos(TSet<UEdGraphNode*>& AverageNodes, FVector2D SourcePos, bool bExpandedNodesNeedUniqueGuid = false) const;
	// static void MoveNodesToGraph(TArray<UEdGraphNode*>& SourceNodes, UEdGraph* DestinationGraph, TSet<UEdGraphNode*>& OutExpandedNodes, UEdGraphNode** OutEntry, UEdGraphNode** OutResult, const bool bIsCollapsedGraph = false);
	// bool CollapseGatewayNode(UK2Node* InNode, UEdGraphNode* InEntryNode, UEdGraphNode* InResultNode, TSet<UEdGraphNode*>* OutExpandedNodes = nullptr) const;

	/**
	 * Collapses a selection of nodes into a graph for composite.
	 *
	 * @param InGatewayNode				The node replacing the selection of nodes
	 * @param InEntryNode				The entry node in the graph
	 * @param InResultNode				The result node in the graph
	 * @param InSourceGraph				The graph the selection is from
	 * @param InDestinationGraph		The destination graph to move the selected nodes to
	 * @param InCollapsableNodes		The selection of nodes being collapsed
	 */
	void CollapseNodesIntoGraph(UEdGraphNode* InGatewayNode, UMaterialGraphNode* InEntryNode, UMaterialGraphNode* InResultNode, UEdGraph* InSourceGraph, UEdGraph* InDestinationGraph, TSet<UEdGraphNode*>& InCollapsableNodes);

	/** Called when a selection of nodes are being collapsed into a sub-graph */
	void CollapseNodes(TSet<class UEdGraphNode*>& InCollapsableNodes);

	/**
	 * Expands passed in node 
	 * 
	 * @param InNodeToExpand			The node containing the selection of nodes that ill be removed
	 * @param InSourceGraph				The graph containing the original node, 
	 * @param OutExpandedNodes			The nodes expanded into the source graph
	 */
	void ExpandNode(UEdGraphNode* InNodeToExpand, UEdGraph* InSourceGraph, TSet<UEdGraphNode*>& OutExpandedNodes);

	/**
	* Move the given set of nodes to an average spot near the Source position
	*
	* @param AverageNodes					The nodes to move
	* @param SourcePos						The source position used to average the nodes around
	* @param bExpandedNodesNeedUniqueGuid	If true then a new Guid will be generated for each node in the set
	*/
	void MoveNodesToAveragePos(TSet<UEdGraphNode*>& AverageNodes, FVector2D SourcePos, bool bExpandedNodesNeedUniqueGuid = false) const;

	/**
	* Move every node from the source graph to the destination graph. Add Each node that is moved to the OutExpandedNodes set.
	* If the source graph is a function graph, keep track of the entry and result nodes in the given Out Parameters.
	*
	* @param SourceNodes		Nodes to move
	* @param DestinationGraph	Graph to move nodes to
	* @param OutExpandedNodes	Set of each node that was moved from the source to destination graph
	* @param OutEntry			Pointer to the function entry node
	* @param OutResult			Pointer to the function result node
	* @param bIsCollapsedGraph	Whether or not the source graph is collapsed
	**/
	static void MoveNodesToGraph(TArray<UEdGraphNode*>& SourceNodes, UEdGraph* DestinationGraph, TSet<UEdGraphNode*>& OutExpandedNodes, UEdGraphNode** OutEntry, UEdGraphNode** OutResult, const bool bIsCollapsedGraph = false);

	/**
	 * Makes connections into/or out of the gateway node, connect directly to the associated networks on the opposite side of the tunnel
	 * When done, none of the pins on the gateway node will be connected to anything.
	 * Requires both this gateway node and it's associated node to be in the same graph already (post-merging)
	 *
	 * @param InGatewayNode			The function or tunnel node
	 * @param InEntryNode			The entry node in the inner graph
	 * @param InResultNode			The result node in the inner graph
	 *
	 * @return						Returns TRUE if successful
	 */
	static bool CollapseGatewayNode(UEdGraphNode* InNode, UEdGraphNode* InEntryNode, UEdGraphNode* InResultNode, TSet<UEdGraphNode*>* OutExpandedNodes = nullptr);

	/** Helper functions for the quality level node display toggling */
	void SetQualityPreview(EMaterialQualityLevel::Type NewQuality);
	bool IsQualityPreviewChecked(EMaterialQualityLevel::Type TestQuality);
	EMaterialQualityLevel::Type GetQualityPreviewLevel() const;

	/** Helper functions for the feature level node display toggling */
	void SetFeaturePreview(ERHIFeatureLevel::Type NewFeatureLevel);
	bool IsFeaturePreviewChecked(ERHIFeatureLevel::Type TestFeatureLevel) const;
	bool IsFeaturePreviewAvailable(ERHIFeatureLevel::Type TestFeatureLevel) const;

	/** Update Substrate topology preview */
	void UpdateSubstrateTopologyPreview();

	/** Create array of derived material instances used in conjunction with preview material in material stats */
	void CreateDerivedMaterialInstancesPreviews();

	/** Returns nullptr if nothing selected */
	UMaterialGraphNode* GetFirstSelectedGraphNode() const;

public:
private:
	/**
	 * Load editor settings from disk (docking state, window pos/size, option state, etc).
	 */
	void LoadEditorSettings();

	/**
	 * Saves editor settings to disk (docking state, window pos/size, option state, etc).
	 */
	void SaveEditorSettings();

	/**
	* Rebuilds dependant Material Instance Editors
	* @param		MatInst	Material Instance to search dependent editors and force refresh of them.
	*/
	//void RebuildMaterialInstanceEditors();
	
	/**
	 * Binds our UI commands to delegates
	 */
	void BindCommands();
	
	/** Command for the apply button */
	void OnApply();
	bool OnApplyEnabled() const;
	/** Command for the camera home button */
	void OnCameraHome();
	/** Command for the hide unused connectors button */
	void OnHideConnectors();
	bool IsOnHideConnectorsChecked() const;
	/** Command for the Toggle Live Preview button */
	void ToggleLivePreview();
	bool IsToggleLivePreviewChecked() const;
	/** Command for the Toggle Real Time button */
	void ToggleRealTimeExpressions();
	bool IsToggleRealTimeExpressionsChecked() const;
	/** Command for the Refresh all previews button */
	void OnAlwaysRefreshAllPreviews();
	bool IsOnAlwaysRefreshAllPreviews() const;

	/** Make nodes which are unrelated to the selected nodes fade out */
	void ToggleHideUnrelatedNodes();
	bool IsToggleHideUnrelatedNodesChecked() const;
	bool IsSelectFocusedNodesEnabled() const;
	void CollectDownstreamNodes(UMaterialGraphNode* CurrentNode, TArray<UMaterialGraphNode*>& CollectedNodes);
	void CollectUpstreamNodes(UMaterialGraphNode* CurrentNode, TArray<UMaterialGraphNode*>& CollectedNodes);
	void HideUnrelatedNodes();
	void SelectMaterialGraphNodes(SGraphEditor& InGraphEd, const TArray<UMaterialGraphNode*>& InNodes);

	/** Make a drop down menu to control the opacity of unrelated nodes */
	void MakeHideUnrelatedNodesOptionsMenu(class UToolMenu* Menu);
	void OnLockNodeStateCheckStateChanged(ECheckBoxState NewCheckedState);
	void OnFocusWholeChainCheckStateChanged(ECheckBoxState NewCheckedState);
	void OnSelectFocusedNodesCheckStateChanged(ECheckBoxState NewCheckedState);

	/** Command for using currently selected texture */
	void OnUseCurrentTexture();
	/** Command for converting nodes to objects */
	void OnConvertObjects();
	/** Command for converting nodes to textures */
	void OnConvertTextures();
	/** Command for collapsing nodes to a function */
	void OnCollapseToFunction();
	bool CanCollapseToFunction() const;
	/** Command for expanding a function */
	void OnExpandMaterialFunctionNode();
	bool CanExpandMaterialFunctionNode() const;
	/** Command for promoting nodes to double precision */
	void OnPromoteObjects();
	/** Command to select local variable declaration */
	void OnSelectNamedRerouteDeclaration();
	/** Command to select local variable usages */
	void OnSelectNamedRerouteUsages();
	/** Command to create reroute usage node from declaration */
	void OnCreateRerouteUsageFromDeclaration();
	/** Command to convert a reroute node to local variables */
	void OnConvertRerouteToNamedReroute();
	/** Command to convert local variables to a reroute node */
	void OnConvertNamedRerouteToReroute();
	/** Command for previewing a selected node */
	void OnPreviewNode();
	/** Command for previewing the currently-hovered node */
	void OnPreviewHoveredNode();

	/** 
	 *	Call to Preview a InGraphNode.
	 *	If InGraphNode is the currently-previewed node, will stop previewing
	 */
	virtual void PreviewNode(UMaterialGraphNode* InGraphNode) override;

	/** Set the custom primitive data to use in the preview display area */
	virtual void SetCustomPrimitiveData(TConstArrayView<float> CustomFloats) override;

	/** Command for toggling real time preview of selected node */
	void OnToggleRealtimePreview();
	/** Command to select nodes downstream of selected node */
	void OnSelectDownstreamNodes();
	/** Command to select nodes upstream of selected node */
	void OnSelectUpstreamNodes();
	/** Command to force a refresh of all previews (triggered by space bar) */
	void OnForceRefreshPreviews();
	/** Create comment node on graph */
	void OnCreateComment();
	/** Create ComponentMask node on graph */
	void OnCreateComponentMaskNode();
	/** Bring up the search tab */
	void OnFindInMaterial();

	/** Will promote selected pin to a parameter of the pin type */
	void OnPromoteToParameter(const FToolMenuContext& InMenuContext) const;
	
	/** Used to know if we can promote selected pin to a parameter of the pin type */
	bool OnCanPromoteToParameter(const FToolMenuContext& InMenuContext) const;

	/** Will  return the UClass to create from the Pin Type */
	UClass* GetOnPromoteToParameterClass(const UEdGraphPin* TargetPin) const;

	/** Used to know if we can reset the selected pin to it's default value */
	bool OnCanResetToDefault(const FToolMenuContext& InMenuContext) const;

	/** Will reset selected pin to it's default value */
	void OnResetToDefault(const FToolMenuContext& InMenuContext) const;

	/** Used to know if we can delete the selected pin */
	bool OnCanDeletePin(const FToolMenuContext& InMenuContext) const;

	/** Will delete the selected pin */
	void OnDeletePin(const FToolMenuContext& InMenuContext) const;
	
	enum class ESubstrateNodeForPin : uint8
	{
		Slab,
		HorizontalMix,
		VerticalLayer,
		Weight,
		Select
	};
	/** Will create a Substrate node as input to the pin */
	void OnCreateSubstrateNodeForPin(const FToolMenuContext& InMenuContext, ESubstrateNodeForPin NodeForPin) const;
	/** Used to know if we can create a Substrate node as input to the pin */
	bool OnCanCreateSubstrateNodeForPin(const FToolMenuContext& InMenuContext, ESubstrateNodeForPin NodeForPin) const;

	/** Open documentation for the selected node class */
	void OnGoToDocumentation();
	/** Can we open documentation for the selected node */
	bool CanGoToDocumentation();

	/** Open C++ header for the selected node class */
	void OnGoToDefinition();
	/** Can we open the C++ header for the selected node */
	bool CanGoToDefinition() const;

	/** Util to try and get doc link for the currently selected node */
	FString GetDocLinkForSelectedNode();

	/** Util to try and get the base URL for the doc link for the currently selected node */
	FString GetDocLinkBaseUrlForSelectedNode();

	/** Callback from the Asset Registry when an asset is renamed. */
	void RenameAssetFromRegistry(const FAssetData& InAddedAssetData, const FString& InNewName);

	/** Callback to tell the Material Editor that a materials usage flags have been changed */
	void OnMaterialUsageFlagsChanged(class UMaterial* MaterialThatChanged, int32 FlagThatChanged);

	/** Callback when an asset is imported */
	void OnAssetPostImport(UFactory* InFactory, UObject* InObject);

	void OnNumericParameterDefaultChanged(class UMaterialExpression*, EMaterialParameterType Type, FName ParameterName, const UE::Shader::FValue& Value);
	void OnParameterDefaultChanged();

	void SetNumericParameterDefaultOnDependentMaterials(EMaterialParameterType Type, FName ParameterName, const UE::Shader::FValue& Value);
	void ClearNumericParameterDefaultOnDependentMaterials(EMaterialParameterType Type, FName ParameterName);

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

	// FNotifyHook interface
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	/** Flags the material as dirty */
	void SetMaterialDirty() {bMaterialDirty = true;}

	/** Toggles the collapsed flag of a Material Expression and updates preview */
	void ToggleCollapsed(UMaterialExpression* MaterialExpression);

	/**
	 * Refreshes material expression previews.  Refreshes all previews if bAlwaysRefreshAllPreviews is true.
	 * Otherwise, refreshes only those previews that have a bRealtimePreview of true.
	 */
	void RefreshExpressionPreviews(bool bForceRefreshAll = false);

	virtual void RefreshGraphFromOriginal() override;

	/**
	 * Refreshes the preview for the specified material expression.  Does nothing if the specified expression
	 * has a bRealtimePreview of false.
	 *
	 * @param	MaterialExpression		The material expression to update.
	 */
	void RefreshExpressionPreview(UMaterialExpression* MaterialExpression, bool bRecompile);

	/**
	 * Returns the expression preview for the specified material expression.
	 */
	FMatExpressionPreview* GetExpressionPreview(UMaterialExpression* MaterialExpression, bool& bNewlyCreated);


	/** Called whenever the color picker is used and accepted. */
	void OnColorPickerCommitted(FLinearColor LinearColor, TWeakObjectPtr<UObject> ColorPickerObject);

	/** Create new graph editor widget */
	TSharedRef<class SGraphEditor> CreateGraphEditorWidget(TSharedRef<class FTabInfo> InTabInfo, class UEdGraph* InGraph);

	/** Gets the current Material Graph's appearance */
	FGraphAppearanceInfo GetGraphAppearance() const;

	/* Gets the upstream and downstream nodes related to the current selection. **/
	TArray<UMaterialGraphNode*> GetSelectionRelatedNodes();

	/**
	 * Deletes any disconnected material expressions.
	 */
	void CleanUnusedExpressions();

	/**
	* Fixes up the samplers for all broken texture sample expressions.
	*/
	void FixupMismatchedSamplerExpressions();

	/**
	 * Cleans up any referenced resources.
	 */
	void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	/**
	 * Refreshes editor after world cleanup.
	 */
	void OnPostWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	/**
	 * Perform a deep copy of all expressions within the given graph.
	 *
	 * @param	CopyGraph, graph whose expression's need a deep copy
	 * @param	NewSubgraphExpression, material expression that will be used as the subgraph expression for all
	 *			deep copied expressions, and will become the graph's new subgraph expression
	 *
	 */
	void DeepCopyExpressions(UMaterialGraph* CopyGraph, UMaterialExpression* NewSubgraphExpression);

	/**
	 * Displays a warning message to the user if the expressions to remove would cause any issues
	 *
	 * @param NodesToRemove The expression nodes we wish to remove
	 *
	 * @return Whether the user agrees to remove these expressions
	 */
	bool CheckExpressionRemovalWarnings(const TArray<UEdGraphNode*>& NodesToRemove);

	/** Removes the selected expression from the favorites list. */
	void RemoveSelectedExpressionFromFavorites();
	/** Adds the selected expression to the favorites list. */
	void AddSelectedExpressionToFavorites();

	/* Updates internal material editor widgets */
	void SetEditorInstanceForWidgets() const;

	/* Jumps to and selects the node corresponding to the given guid, even if it exists in another asset. */
	void JumpToNodeByGuid(const FGuid& InGuid);

private:
	TSharedRef<SDockTab> SpawnTab_Preview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_MaterialProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Palette(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_NamedReroutes(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Find(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ParameterDefaults(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_CustomPrimitiveData(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_LayerProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Substrate(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_MaterialExpressionsOverview(const FSpawnTabArgs& Args);

	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	void OnGraphChanged(const FEdGraphEditAction& Action);
	void OnFinishedChangingParametersFromOverview(const FPropertyChangedEvent& PropertyChangedEvent);
	void OnChangeBreadCrumbGraph(class UEdGraph* InGraph);
	void GeneratorRowsRefreshed();
	void UpdateGenerator();
	void NavigateTab(FDocumentTracker::EOpenDocumentCause InCause);
private:
	/** Property View */
	TSharedPtr<class IDetailsView> MaterialDetailsView;

	/** Currently focused graph editor */
	TWeakPtr<class SGraphEditor> FocusedGraphEdPtr;

	/** Outstanding "material refreshed from external edit" toast, kept so back-to-back refreshes don't stack. */
	TWeakPtr<class SNotificationItem> ExternalRefreshNotificationPtr;

	/** Preview Viewport widget */
	TSharedPtr<class SMaterialEditor3DPreviewViewport> PreviewViewport;

	/** Preview viewport widget used for UI materials */
	TSharedPtr<class SMaterialEditorUIPreviewViewport> PreviewUIViewport;

	/** Hashed error code use to refresh the error displaying widget when this will change */
	FString MaterialErrorHash;

	/** Palette of Material Expressions and functions */
	TSharedPtr<class SMaterialPalette> Palette;

	/** Panel listing Named Reroute Declaration nodes */
	TSharedPtr<class SMaterialNamedReroutesPanel> NamedReroutesPanel;

	/** Handle for the material graph's OnGraphChanged delegate (used to refresh the Named Reroutes panel) */
	FDelegateHandle OnGraphChangedDelegateHandle;

	/** The Substrate control tab */
	TSharedPtr<class SMaterialEditorSubstrateWidget> SubstrateWidget;

	/** Stats log, with the log listing that it reflects */
	TSharedPtr<class SWidget> Stats;
	TSharedPtr<class IMessageLogListing> StatsListing;

	/** Find results log as well as the search filter */
	TSharedPtr<class SFindInMaterial> FindResults;

	/** Parameter overview list View */
	TSharedPtr<class SMaterialParametersOverviewPanel> MaterialParametersOverviewWidget;

	/** Shared texture samplers list view **/
	TSharedPtr<class SMaterialExpressionsOverviewPanel> MaterialExpressionsOverviewWidget;

	/** Custom primitive data widget **/
	TSharedPtr<class SMaterialCustomPrimitiveDataPanel> MaterialCustomPrimitiveDataWidget;

	/** Layer Properties View */
	TSharedPtr<class SMaterialLayersFunctionsMaterialWrapper> MaterialLayersFunctionsInstance;
	
	/** The current transaction. */
	FScopedTransaction* ScopedTransaction;

	/** If true, always refresh all expression previews.  This overrides UMaterialExpression::bRealtimePreview. */
	bool bAlwaysRefreshAllPreviews;

	/** Material expression previews. */
	TArray<TRefCountPtr<FMatExpressionPreview>> ExpressionPreviews;

	/** Used to store material errors */
	TArray<TSharedPtr<FMaterialInfo>> MaterialInfoList;

	TSet<TTuple<EMaterialParameterType, FName>> OverriddenNumericParametersToRevert;

	/** If true, don't render connectors that are not connected to anything. */
	bool bHideUnusedConnectors;

	/** If true, the preview material is compiled on every edit of the material. If false, only on Apply. */
	bool bLivePreview;

	/** Just storing this choice for now, not sure what difference it will make to Graph Editor */
	bool bIsRealtime;

	/** If true, show stats for an empty material. Helps artists to judge the cost of their changes to the graph. */
	bool bShowBuiltinStats;

	/** If true, fade out nodes which are unrelated to the selected nodes automatically. */
	bool bHideUnrelatedNodes;

	/** Lock the current fade state of each node */
	bool bLockNodeFadeState;

	/** Focus all nodes in the same output chain  */
	bool bFocusWholeChain;

	/** Select focused nodes. */
	bool bSelectFocusedNodes;

	/** Used to avoid selection operations while automated selection takes place. */
	bool bFocusSelectionInProgress;

	/** If a regular node (not a comment node, not the output node) has been selected */
	bool bSelectRegularNode;

	/** Command list for this editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	/** Object that stores all of the possible parameters we can edit. */
	TObjectPtr<class UMaterialEditorPreviewParameters> MaterialEditorInstance;

	/** Object used as material statistics manager */
	TSharedPtr<class FMaterialStats> MaterialStatsManager;

	/** Tab that holds the details panel */
	TWeakPtr<SDockTab> SpawnedDetailsTab;

	/** Stores the quality level used to preview the material graph */
	EMaterialQualityLevel::Type NodeQualityLevel;

	/** Stores the feature level used to preview the material graph */
	ERHIFeatureLevel::Type NodeFeatureLevel;

	/** True if we want to preview static switches, disabling inactive nodes in the graph */
	bool bPreviewStaticSwitches;

	/** True if the quality level or feature level to preview has been changed */
	bool bPreviewFeaturesChanged;

	/** List of children used to populate the inheritance list chain. */
	TArray< FAssetData > MaterialChildList;

	/** List of children used to populate the inheritance list chain. */
	TArray< FAssetData > FunctionChildList;

	TSharedPtr<class IPropertyRowGenerator> Generator;

	/** Delegate handles for FWorldDelegates registrations. */
	FDelegateHandle OnWorldCleanupDelegateHandle;
	FDelegateHandle OnPostWorldCleanupDelegateHandle;

	/** The text for the number of shaders in the material. */
	TSharedPtr<STextBlock> NumShadersText;

	/** Current custom primitive data set */
	TArray<float> CustomPrimitiveDataFloats;
};
