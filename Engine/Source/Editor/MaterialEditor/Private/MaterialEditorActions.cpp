// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorActions.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionComment.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

struct FEdGraphSchemaAction;

#define LOCTEXT_NAMESPACE "MaterialEditorCommands"

UE_DEFINE_TCOMMANDS(FMaterialEditorCommands)

void FMaterialEditorCommands::RegisterCommands()
{
	UI_COMMAND( Apply, "Apply", "Apply changes to original material and its use in the world.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( Flatten, "Flatten", "Flatten the material to a texture for mobile devices.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( ShowAllMaterialParameters, "Show Inactive", "Show the material instance parameters currently hidden behind static switches.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( SetCylinderPreview, "Cylinder", "Sets the preview mesh to a cylinder primitive.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetSpherePreview, "Sphere", "Sets the preview mesh to a sphere primitive.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetPlanePreview, "Plane", "Sets the preview mesh to a plane primitive.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetCubePreview, "Cube", "Sets the preview mesh to a cube primitive.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetShaderBallPreview, "ShaderBall", "Sets the preview mesh to a shader ball primitive.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetPreviewMeshFromSelection, "Mesh", "Sets the preview mesh based on the current content browser selection.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SelectPreviewMeshMaterialSlots, "Mesh Slots", "Sets the preview mesh slots to apply the material to.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( TogglePreviewBackground, "Background", "Toggles the preview pane's background.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( CameraHome, "Home", "Goes home on the canvas.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CleanUnusedExpressions, "Clean Up", "Cleans up any unused Expressions.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FixupMismatchedSamplers, "Fixup mismatched samplers", "Automatically fixes all texture samplers with the wrong type for their texture.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ShowHideConnectors, "Hide Unconnected Pins", "Hides all pins with no connections", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleLivePreview, "Preview Material", "Toggles real time update of the preview material.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleRealtimeExpressions, "Realtime Nodes", "Nodes impacted by time-based functions such as panners, etc. will update realtime.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( AlwaysRefreshAllPreviews, "All Node Previews", "All node previews are updated upon any change to the graph.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleMaterialStats, "Stats", "Toggles displaying of the material's stats.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( TogglePlatformStats, "Platform Stats", "Toggles the window that shows material stats and compilation errors for multiple platforms.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleHideUnrelatedNodes, "Hide Unrelated", "Toggles hiding nodes which are unrelated to the selected nodes automatically.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( NewComment, "New Comment", "Creates a new comment node.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MatertialPasteHere, "Paste Here", "Pastes copied items at this location.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( UseCurrentTexture, "Use Current Texture", "Uses the current texture selected in the content browser.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertObjects, "Convert to Parameter", "Converts the objects to parameters.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( PromoteToDouble, "Promote to Double", "Converts the parameters to double precision.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( PromoteToFloat, "Promote to Float", "Converts the parameters to float precision.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ConvertToConstant, "Convert to Constant", "Converts the parameters to constants.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectNamedRerouteDeclaration, "Select Named Reroute Declaration", "Select this named reroute's declaration node.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectNamedRerouteUsages, "Select Named Reroute Usages", "Select this named reroute's usage nodes.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateRerouteUsageFromDeclaration, "Create Reroute Usage", "Create Reroute Usage From Declaration.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertRerouteToNamedReroute, "Convert to Named Reroute", "Replace this reroute with a named reroute.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertNamedRerouteToReroute, "Convert to Reroute", "Replace this named reroute with an (unnamed) reroute node.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertToTextureObjects, "Convert to Texture Object", "Converts the objects to texture objects.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertToTextureSamples, "Convert to Texture Sample", "Converts the objects to texture samples.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( StopPreviewNode, "Stop Previewing Node", "Stops the preview viewport from previewing this node", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( StartPreviewNode, "Start Previewing Node", "Makes the preview viewport start previewing this node", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND(PreviewHoveredNode, "Preview Hovered Node", "Previews the currently-hovered node", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::SpaceBar) );
	UI_COMMAND( EnableRealtimePreviewNode, "Enable Realtime Preview", "Enables realtime previewing of this expression node", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DisableRealtimePreviewNode, "Disable Realtime Preview", "Disables realtime previewing of this expression node", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BreakAllLinks, "Break All Links", "Breaks all links leading out of this node.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DuplicateObjects, "Duplicate Object(s)", "Duplicates the selected objects.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DeleteObjects, "Delete Object(s)", "Deletes the selected objects.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectDownstreamNodes, "Select Downstream Nodes", "Selects all nodes that use this node's outgoing links.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectUpstreamNodes, "Select Upstream Nodes", "Selects all nodes that feed links into this node.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RemoveFromFavorites, "Remove From Favorites", "Removes this expression from your favorites.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddToFavorites, "Add To Favorites", "Adds this expression to your favorites.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BreakLink, "Break Link", "Deletes this link.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ForceRefreshPreviews, "Force Refresh Previews", "Forces a refresh of all previews", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar) );
	UI_COMMAND( CreateComponentMaskNode, "Create ComponentMask Node", "Creates a ComponentMask node at the current cursor position.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::C));
	UI_COMMAND( FindInMaterial, "Search", "Finds expressions and comments in the current Material", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
	
	// Pin Actions
	UI_COMMAND( PromoteToParameter, "Promote to Parameter", "Promote selected Pin to parameter of pin type", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ResetToDefault, "Reset Pin to Default Value", "Reset pin value to default for that type", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeletePin, "Delete Pin", "Delete Selected Pin", EUserInterfaceActionType::Button, FInputChord());

	// Substrate Pin Actions
	UI_COMMAND( CreateSlabNode, "Create Slab Node", "Create a Slab Node linked to the pin", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( CreateHorizontalMixNode, "Create Horizontal Mix Node", "Create a Horizontal Mix Node linked to the pin", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( CreateVerticalLayerNode, "Create Vertical Layer Node", "Create a Vertical Layer Node linked to the pin", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( CreateWeightNode, "Create Weight Node", "Create a Weight Node linked to the pin", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( CreateSelectNode, "Create Select Node", "Create a Select Node linked to the pin", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(QualityLevel_All, "All", "Sets node preview to show all quality levels.)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(QualityLevel_Epic, "Epic", "Sets node preview to Epic quality.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(QualityLevel_High, "High", "Sets node preview to high quality.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(QualityLevel_Medium, "Medium", "Sets node preview to medium quality.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(QualityLevel_Low, "Low", "Sets node preview to low quality.", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(FeatureLevel_All, "All", "Sets node preview to show all feature levels.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FeatureLevel_Mobile, "Mobile", "Sets node preview to show the Mobile feature level.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FeatureLevel_SM5, "SM5", "Sets node preview to show the SM5 feature level.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FeatureLevel_SM6, "SM6", "Sets node preview to show the SM6 feature level.", EUserInterfaceActionType::RadioButton, FInputChord());
}

//////////////////////////////////////////////////////////////////////////
// FExpressionSpawnInfo

TSharedPtr< FEdGraphSchemaAction > FExpressionSpawnInfo::GetAction(UEdGraph* InDestGraph)
{
	if (MaterialExpressionClass == UMaterialExpressionComment::StaticClass())
	{
		return MakeShareable(new FMaterialGraphSchemaAction_NewComment);
	}
	else
	{
		TSharedPtr<FMaterialGraphSchemaAction_NewNode> NewNodeAction(new FMaterialGraphSchemaAction_NewNode);
		NewNodeAction->MaterialExpressionClass = MaterialExpressionClass;
		return NewNodeAction;
	}
}

//////////////////////////////////////////////////////////////////////////
// FMaterialEditorSpawnNodeCommands

void FMaterialEditorSpawnNodeCommands::RegisterCommands()
{
	const FString ConfigSection = TEXT("MaterialEditorSpawnNodes");
	const FString SettingName = TEXT("Node");
	TArray< FString > NodeSpawns;
	GConfig->GetArray(*ConfigSection, *SettingName, NodeSpawns, GEditorPerProjectIni);

	for(int32 x = 0; x < NodeSpawns.Num(); ++x)
	{
		FString ClassName;
		if(!FParse::Value(*NodeSpawns[x], TEXT("Class="), ClassName))
		{
			// Could not find a class name, cannot continue with this line
			continue;
		}

		FString CommandLabel;
		UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName, EFindFirstObjectOptions::ExactClass);
		TSharedPtr< FExpressionSpawnInfo > InfoPtr;

		if(FoundClass && FoundClass->IsChildOf(UMaterialExpression::StaticClass()))
		{
			// The class name matches that of a UMaterialExpression, so setup a spawn info that can generate one
			CommandLabel = FoundClass->GetName();

			InfoPtr = MakeShareable(new FExpressionSpawnInfo(FoundClass));
		}

		// If spawn info was created, setup a UI Command for keybinding.
		if(InfoPtr.IsValid())
		{
			TSharedPtr< FUICommandInfo > CommandInfo;

			FKey Key;
			bool bShift = false;
			bool bCtrl = false;
			bool bAlt = false;

			// Parse the keybinding information
			FString KeyString;
			if( FParse::Value(*NodeSpawns[x], TEXT("Key="), KeyString) )
			{
				Key = *KeyString;
			}

			if( Key.IsValid() )
			{
				FParse::Bool(*NodeSpawns[x], TEXT("Shift="), bShift);
				FParse::Bool(*NodeSpawns[x], TEXT("Alt="), bAlt);
				FParse::Bool(*NodeSpawns[x], TEXT("Ctrl="), bCtrl);
			}

			FInputChord Chord(Key, EModifierKey::FromBools(bCtrl, bAlt, bShift, false));

			const FText CommandLabelText = FText::FromString( CommandLabel );
			const FText Description = FText::Format( NSLOCTEXT("MaterialEditor", "NodeSpawnDescription", "Hold down the bound keys and left click in the graph panel to spawn a {0} node."), CommandLabelText);

			FUICommandInfo::MakeCommandInfo( this->AsShared(), CommandInfo, FName(*NodeSpawns[x]), CommandLabelText, Description, FSlateIcon(FAppStyle::GetAppStyleSetName(), *FString::Printf(TEXT("%s.%s"), *this->GetContextName().ToString(), *NodeSpawns[x])), EUserInterfaceActionType::Button, Chord );

			InfoPtr->CommandInfo = CommandInfo;

			NodeCommands.Add(InfoPtr);
		}
	}
}

TSharedPtr< FEdGraphSchemaAction > FMaterialEditorSpawnNodeCommands::GetGraphActionByChord(FInputChord& InChord, UEdGraph* InDestGraph) const
{
	if(InChord.IsValidChord())
	{
		for(int32 x = 0; x < NodeCommands.Num(); ++x)
		{
			if (NodeCommands[x]->CommandInfo->HasActiveChord(InChord))
			{
				return NodeCommands[x]->GetAction(InDestGraph);
			}
		}
	}

	return TSharedPtr< FEdGraphSchemaAction >();
}

const TSharedPtr<const FInputChord> FMaterialEditorSpawnNodeCommands::GetChordByClass(UClass* MaterialExpressionClass) const
{
	for(int32 Index = 0; Index < NodeCommands.Num(); ++Index)
	{
		if (NodeCommands[Index]->GetClass() == MaterialExpressionClass && NodeCommands[Index]->CommandInfo->GetFirstValidChord()->IsValidChord())
		{
			// Just return the first valid chord
			return NodeCommands[Index]->CommandInfo->GetFirstValidChord();
		}
	}

	return TSharedPtr< const FInputChord >();
}

#undef LOCTEXT_NAMESPACE
