// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

struct FGuid;

DECLARE_DELEGATE_OneParam(FSelectMaterialNode, const FGuid& InNodeGuid);
