// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorDetailCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"
#include "Materials/Material.h"
#include "Materials/MaterialParameterCollection.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionColorRamp.h"
#include "Materials/MaterialExpressionOperator.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialLayersFunctionsCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyRestriction.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "Widgets/Input/SButton.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "IDetailGroup.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialEditor/DEditorFontParameterValue.h"
#include "MaterialEditor/DEditorRuntimeVirtualTextureParameterValue.h"
#include "MaterialEditor/DEditorScalarParameterValue.h"
#include "MaterialEditor/DEditorStaticSwitchParameterValue.h"
#include "MaterialEditor/DEditorStaticComponentMaskParameterValue.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorVectorParameterValue.h"
#include "MaterialPropertyHelpers.h"
#include "MaterialEditor/DEditorMaterialLayersParameterValue.h"
#include "PropertyCustomizationHelpers.h"
#include "Curves/CurveLinearColor.h"
#include "IPropertyUtilities.h"
#include "MaterialDomain.h"
#include "Engine/Texture.h"
#include "Materials/MaterialExpressionCurveAtlasRowParameter.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "RenderUtils.h"
#include "MaterialShared.h"
#include "SColorGradientEditor.h"
#include "SWarningOrErrorBox.h"
#include "IDetailChildrenBuilder.h"

static TAutoConsoleVariable<bool> CVarMaterialEdUseGroupCustomUI(
	TEXT("r.MaterialEditor.UseGroupsCustomUI"),
	true,
	TEXT("Whether Custom UI should be used for Groups Priorities."),
	ECVF_Default);


#define LOCTEXT_NAMESPACE "MaterialEditor"

// Update the blend mode names based on what is supported in legacy mode or Substrate mode
void UpdateBlendModeEnum()
{
	UEnum* BlendModeEnum = StaticEnum<EBlendMode>();
	if (Substrate::IsSubstrateEnabled())
	{
		// BLEND_Translucent & BLEND_TranslucentGreyTransmittance are mapped onto the same enum index
		BlendModeEnum->SetMetaData(TEXT("DisplayName"), TEXT("TranslucentGreyTransmittance"), BLEND_Translucent);

		// BLEND_Modulate & BLEND_ColoredTransmittanceOnly are mapped onto the same enum index
		BlendModeEnum->SetMetaData(TEXT("DisplayName"), TEXT("ColoredTransmittanceOnly"), BLEND_Modulate);

		// BLEND_TranslucentColoredTransmittance is only supported in Substrate mode
		BlendModeEnum->SetMetaData(TEXT("DisplayName"), TEXT("TranslucentColoredTransmittance"), BLEND_TranslucentColoredTransmittance);

		static const FString ToolTipName = TEXT("ToolTip");
		static const FString TranslucentToolTip = TEXT("Translucent surface accounting for coverage. Colored transmittance over the background is not possible and will be converted to grey scale transmittance.\nThis is the same as the legacy \"Translucent\" blend mode.");
		static const FString ModulateToolTip = TEXT("Surface where only transmittance is applied over the background.\nThis is similar to the legacy \"Modulate\" blend mode.");
		BlendModeEnum->SetMetaData(*ToolTipName, TEXT("Opaque surface"), BLEND_Opaque);
		BlendModeEnum->SetMetaData(*ToolTipName, TEXT("Masked opaque surface, visible only when opacity is greater than the OpacityMaskClipValue from the material."), BLEND_Masked);
		BlendModeEnum->SetMetaData(*ToolTipName, *TranslucentToolTip, BLEND_Translucent);
		BlendModeEnum->SetMetaData(*ToolTipName, TEXT("Surface where only emissive and coverage are accounted for."), BLEND_Additive);
		BlendModeEnum->SetMetaData(*ToolTipName, *ModulateToolTip, BLEND_Modulate);
		BlendModeEnum->SetMetaData(*ToolTipName, TEXT("Translucent surface for which the throughput over the background can be overriden independently from the material itself."), BLEND_AlphaComposite);
		BlendModeEnum->SetMetaData(*ToolTipName, TEXT("Alpha hold out blend mode use to punch holes in translucent surfaces."), BLEND_AlphaHoldout);
		BlendModeEnum->SetMetaData(*ToolTipName, TEXT("Translucent surface accounting for coverage as well as colored transmittance over the background.\nThis is similar to the legacy \"ThinTranslucent\" shading model."), BLEND_TranslucentColoredTransmittance);
		BlendModeEnum->SetMetaData(*ToolTipName, *TranslucentToolTip, BLEND_TranslucentGreyTransmittance);
		BlendModeEnum->SetMetaData(*ToolTipName, *ModulateToolTip, BLEND_ColoredTransmittanceOnly);
	}
	else
	{
		// BLEND_TranslucentColoredTransmittance is not supported in legacy mode
		BlendModeEnum->SetMetaData(TEXT("Hidden"), TEXT("True"), BLEND_TranslucentColoredTransmittance);
	}
}

TSharedRef<IDetailCustomization> FMaterialExpressionParameterDetails::MakeInstance(FOnCollectParameterGroups InCollectGroupsDelegate)
{
	return MakeShareable( new FMaterialExpressionParameterDetails(InCollectGroupsDelegate) );
}

FMaterialExpressionParameterDetails::FMaterialExpressionParameterDetails(FOnCollectParameterGroups InCollectGroupsDelegate)
	: CollectGroupsDelegate(InCollectGroupsDelegate)
{
}

FMaterialExpressionParameterDetails::FMaterialExpressionParameterDetails()
{
}

void FMaterialExpressionParameterDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	// for expression parameters all their properties are in one category based on their class name.
	FName DefaultCategory = NAME_None;
	IDetailCategoryBuilder& Category = DetailLayout.EditCategory( DefaultCategory );
	
	TArray< TWeakObjectPtr<UObject> > Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	DefaultValueHandles.Reset();
	ScalarParameterObjects.Reset();

	const FName MaterialExpressionCategory = TEXT("MaterialExpression");
	IDetailCategoryBuilder& ExpressionCategory = DetailLayout.EditCategory(MaterialExpressionCategory);

	for (const auto& WeakObjectPtr : Objects)
	{
		UObject* Object = WeakObjectPtr.Get();

		UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(Object);

		if (ScalarParameter)
		{
			// Store these for OnSliderMinMaxEdited
			ScalarParameterObjects.Emplace(WeakObjectPtr);

			TSharedPtr<IPropertyHandle> DefaultValueHandle = DetailLayout.GetProperty("DefaultValue", UMaterialExpressionScalarParameter::StaticClass());
			DefaultValueHandles.Emplace(DefaultValueHandle);

			if (DefaultValueHandle.IsValid() && DefaultValueHandle->IsValidHandle())
			{
				if (!ScalarParameter->bUseCustomPrimitiveData && ScalarParameter->ControlType == EMaterialScalarParameterControlType::Enumeration && !ScalarParameter->Enumeration.IsNull())
				{
					// Display enumeration widget for DefaultValue
					IDetailPropertyRow& PropertyRow = *DetailLayout.EditDefaultProperty(DefaultValueHandle);
					PropertyRow.CustomWidget()
					.NameContent()
					[
						DefaultValueHandle->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						FMaterialPropertyHelpers::CreateScalarEnumerationParameterValueWidget(ScalarParameter->Enumeration, DefaultValueHandle)
					];
				}
			}

			TSharedPtr<IPropertyHandle> SliderMinHandle = DetailLayout.GetProperty("SliderMin", UMaterialExpressionScalarParameter::StaticClass());

			if (SliderMinHandle.IsValid() && SliderMinHandle->IsValidHandle())
			{
				// Setup a callback when SliderMin changes to update the DefaultValue slider
				FSimpleDelegate OnSliderMinMaxEditedDelegate = FSimpleDelegate::CreateSP(this, &FMaterialExpressionParameterDetails::OnSliderMinMaxEdited);
				SliderMinHandle->SetOnPropertyValueChanged(OnSliderMinMaxEditedDelegate);
			}

			TSharedPtr<IPropertyHandle> SliderMaxHandle = DetailLayout.GetProperty("SliderMax", UMaterialExpressionScalarParameter::StaticClass());

			if (SliderMaxHandle.IsValid() && SliderMaxHandle->IsValidHandle())
			{
				FSimpleDelegate OnSliderMinMaxEditedDelegate = FSimpleDelegate::CreateSP(this, &FMaterialExpressionParameterDetails::OnSliderMinMaxEdited);
				SliderMaxHandle->SetOnPropertyValueChanged(OnSliderMinMaxEditedDelegate);
			}

			OnSliderMinMaxEdited();

			// Some properties require a layout rebuild on edit.
			auto ForceRefresh = FSimpleDelegate::CreateLambda([&DetailLayout]() { DetailLayout.ForceRefreshDetails(); });

			TSharedPtr<IPropertyHandle> ControlTypeHandle = DetailLayout.GetProperty("ControlType", UMaterialExpressionScalarParameter::StaticClass());
			if (ControlTypeHandle.IsValid() && ControlTypeHandle->IsValidHandle())
			{
				ControlTypeHandle->SetOnPropertyValueChanged(ForceRefresh);
			}

			TSharedPtr<IPropertyHandle> EnumerationHandle = DetailLayout.GetProperty("Enumeration", UMaterialExpressionScalarParameter::StaticClass());
			if (EnumerationHandle.IsValid() && EnumerationHandle->IsValidHandle())
			{
				EnumerationHandle->SetOnPropertyValueChanged(ForceRefresh);
			}

			TSharedPtr<IPropertyHandle> UseCustomPrimitiveDataPropertyHandle = DetailLayout.GetProperty("bUseCustomPrimitiveData", UMaterialExpressionScalarParameter::StaticClass());
			if (UseCustomPrimitiveDataPropertyHandle.IsValid() && UseCustomPrimitiveDataPropertyHandle->IsValidHandle())
			{
				UseCustomPrimitiveDataPropertyHandle->SetOnPropertyValueChanged(ForceRefresh);
			}

			if (ScalarParameter->IsUsedAsAtlasPosition())
			{
				UMaterialExpressionCurveAtlasRowParameter* ColorCurveParameter = Cast<UMaterialExpressionCurveAtlasRowParameter>(ScalarParameter);
				TSharedPtr<IPropertyHandle> CurveHandle = DetailLayout.GetProperty("Curve", UMaterialExpressionCurveAtlasRowParameter::StaticClass());
				if (CurveHandle.IsValid() && CurveHandle->IsValidHandle())
				{
					const FName AtlasCategoryName = TEXT("MaterialExpressionCurveAtlasRowParameter");
					IDetailCategoryBuilder& AtlasCategory = DetailLayout.EditCategory(AtlasCategoryName);
					TSoftObjectPtr<UCurveLinearColorAtlas> Atlas = TSoftObjectPtr<UCurveLinearColorAtlas>(FSoftObjectPath(ColorCurveParameter->Atlas->GetPathName()));
					CurveHandle->MarkHiddenByCustomization();
					AtlasCategory.AddCustomRow(CurveHandle->GetPropertyDisplayName())
						.NameContent()
						[
							SNew(STextBlock)
							.Text(CurveHandle->GetPropertyDisplayName())
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						.ValueContent()
						.HAlign(HAlign_Fill)
						.MaxDesiredWidth(400.0f)
						[
							SNew(SObjectPropertyEntryBox)
							.PropertyHandle(CurveHandle)
							.AllowedClass(UCurveLinearColor::StaticClass())
							.NewAssetFactories(TArray<UFactory*>())
							.DisplayThumbnail(true)
							.ThumbnailPool(DetailLayout.GetThumbnailPool())
							.OnShouldFilterAsset(FOnShouldFilterAsset::CreateStatic(&FMaterialPropertyHelpers::OnShouldFilterCurveAsset, Atlas))
							.OnShouldSetAsset(FOnShouldSetAsset::CreateStatic(&FMaterialPropertyHelpers::OnShouldSetCurveAsset, Atlas))
						];


						TSharedPtr<IPropertyHandle> AtlasHandle = DetailLayout.GetProperty("Atlas", UMaterialExpressionCurveAtlasRowParameter::StaticClass());
						if (AtlasHandle.IsValid() && AtlasHandle->IsValidHandle())
						{
							AtlasHandle->SetOnPropertyValueChanged(ForceRefresh);
						}

				}

				DetailLayout.HideCategory(NAME_UI);
				DetailLayout.HideCategory(TEXT("CustomPrimitiveData"));
			}

			if (ScalarParameter->bUseCustomPrimitiveData)
			{
				DetailLayout.HideCategory(MaterialExpressionCategory);
				DetailLayout.HideCategory(NAME_UI);
			}
		}

		UMaterialExpressionVectorParameter* VectorParameter = Cast<UMaterialExpressionVectorParameter>(Object);

		if (VectorParameter)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty("bUseCustomPrimitiveData", UMaterialExpressionVectorParameter::StaticClass());
			if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				// Rebuild the layout when the bUseCustomPrimitiveData property changes
				PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{
					DetailLayout.ForceRefreshDetails();
				}));
			}

			TSharedPtr<IPropertyHandle> ChannelHandle = DetailLayout.GetProperty("ChannelNames", UMaterialExpressionVectorParameter::StaticClass());
			TSharedPtr<IPropertyHandle> ValueHandle = DetailLayout.GetProperty("DefaultValue", UMaterialExpressionVectorParameter::StaticClass());
			if (ChannelHandle.IsValid() && ChannelHandle->IsValidHandle())
			{
				static const FName Red("R");
				static const FName Green("G");
				static const FName Blue("B");
				static const FName Alpha("A");
				// Rebuild the layout when the ChannelNames property changes
				ChannelHandle->GetChildHandle(Red)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{
	
					DetailLayout.ForceRefreshDetails();
				}));
				ChannelHandle->GetChildHandle(Green)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{

					DetailLayout.ForceRefreshDetails();
				}));
				ChannelHandle->GetChildHandle(Blue)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{

					DetailLayout.ForceRefreshDetails();
				}));
				ChannelHandle->GetChildHandle(Alpha)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{

					DetailLayout.ForceRefreshDetails();
				}));
			}

			if (VectorParameter->bUseCustomPrimitiveData)
			{
				DetailLayout.HideCategory(TEXT("MaterialExpressionVectorParameter"));
				DetailLayout.HideCategory(MaterialExpressionCategory);
			}

			if (ValueHandle.IsValid() && ValueHandle->IsValidHandle() && !VectorParameter->IsUsedAsChannelMask())
			{
				static const FName Red("R");
				static const FName Green("G");
				static const FName Blue("B");
				static const FName Alpha("A");
				if (!VectorParameter->ChannelNames.R.IsEmpty())
				{
					ValueHandle->GetChildHandle(Red)->SetPropertyDisplayName(VectorParameter->ChannelNames.R);
				}
				if (!VectorParameter->ChannelNames.G.IsEmpty())
				{
					ValueHandle->GetChildHandle(Green)->SetPropertyDisplayName(VectorParameter->ChannelNames.G);
				}
				if (!VectorParameter->ChannelNames.B.IsEmpty())
				{
					ValueHandle->GetChildHandle(Blue)->SetPropertyDisplayName(VectorParameter->ChannelNames.B);
				}
				if (!VectorParameter->ChannelNames.A.IsEmpty())
				{
					ValueHandle->GetChildHandle(Alpha)->SetPropertyDisplayName(VectorParameter->ChannelNames.A);
				}
			}

			if (VectorParameter->IsUsedAsChannelMask())
			{
				TSharedPtr<IPropertyHandle> ChannelNameHandle = DetailLayout.GetProperty("ChannelNames", UMaterialExpressionVectorParameter::StaticClass());
				ChannelNameHandle->MarkHiddenByCustomization();
			}
		}

		UMaterialExpressionTextureSampleParameter* TextureParameter = Cast<UMaterialExpressionTextureSampleParameter>(Object);

		if (TextureParameter)
		{
			TSharedPtr<IPropertyHandle> ChannelHandle = DetailLayout.GetProperty("ChannelNames", UMaterialExpressionTextureSampleParameter::StaticClass());
			TSharedPtr<IPropertyHandle> ValueHandle = DetailLayout.GetProperty("Texture", UMaterialExpressionTextureBase::StaticClass());
			
			if (ChannelHandle.IsValid() && ChannelHandle->IsValidHandle())
			{
				static const FName Red("R");
				static const FName Green("G");
				static const FName Blue("B");
				static const FName Alpha("A");
				// Rebuild the layout when the ChannelNames property changes
				ChannelHandle->GetChildHandle(Red)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{
					DetailLayout.ForceRefreshDetails();
				}));
				ChannelHandle->GetChildHandle(Green)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{
					DetailLayout.ForceRefreshDetails();
				}));
				ChannelHandle->GetChildHandle(Blue)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{
					DetailLayout.ForceRefreshDetails();
				}));
				ChannelHandle->GetChildHandle(Alpha)->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
				{
					DetailLayout.ForceRefreshDetails();
				}));
			}

			if (ValueHandle.IsValid() && ValueHandle->IsValidHandle())
			{
				IDetailPropertyRow& PropertyRow = *DetailLayout.EditDefaultProperty(ValueHandle);
				TSharedPtr<SWidget> NameWidget;
				TSharedPtr<SWidget> ValueWidget;
				FDetailWidgetRow DefaultRow;
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, DefaultRow);

				FDetailWidgetRow &DetailWidgetRow = PropertyRow.CustomWidget();
				TSharedPtr<SVerticalBox> NameVerticalBox;
				DetailWidgetRow.NameContent()
					[
						SAssignNew(NameVerticalBox, SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Text(FText::FromName(TextureParameter->ParameterName))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					];

				DetailWidgetRow.ValueContent()
					.MinDesiredWidth(DefaultRow.ValueWidget.MinWidth)
					.MaxDesiredWidth(DefaultRow.ValueWidget.MaxWidth)
					[
						ValueWidget.ToSharedRef()
					];

				static const FName Red("R");
				static const FName Green("G");
				static const FName Blue("B");
				static const FName Alpha("A");

				if (!TextureParameter->ChannelNames.R.IsEmpty())
				{
					NameVerticalBox->AddSlot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(20.0, 2.0, 4.0, 2.0)
						[
							SNew(STextBlock)
							.Text(FText::FromName(Red))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.Padding(4.0, 2.0)
						[
							SNew(STextBlock)
							.Text(TextureParameter->ChannelNames.R)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					];
				}
				if (!TextureParameter->ChannelNames.G.IsEmpty())
				{
					NameVerticalBox->AddSlot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(20.0, 2.0, 4.0, 2.0)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(FText::FromName(Green))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.Padding(4.0, 2.0)
						[
							SNew(STextBlock)
							.Text(TextureParameter->ChannelNames.G)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					];
				}
				if (!TextureParameter->ChannelNames.B.IsEmpty())
				{
					NameVerticalBox->AddSlot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(20.0, 2.0, 4.0, 2.0)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(FText::FromName(Blue))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.Padding(4.0, 2.0)
						[
							SNew(STextBlock)
							.Text(TextureParameter->ChannelNames.B)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					];
				}
				if (!TextureParameter->ChannelNames.A.IsEmpty())
				{
					NameVerticalBox->AddSlot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(20.0, 2.0, 4.0, 2.0)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(FText::FromName(Alpha))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.Padding(4.0, 2.0)
						[
							SNew(STextBlock)
							.Text(TextureParameter->ChannelNames.A)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					];
				}
			}
		}
	}

	check(ScalarParameterObjects.Num() == DefaultValueHandles.Num());
	
	Category.AddProperty("ParameterName");



	// Get a handle to the property we are about to edit
	GroupPropertyHandle = DetailLayout.GetProperty( "Group" );

	TSharedPtr<SComboButton> NewComboButton;
	TSharedPtr<SEditableText> NewEditBox;
	TSharedPtr<SListView<TSharedPtr<FString>>> NewListView;

	if (GroupPropertyHandle->IsValidHandle())
	{
		GroupPropertyHandle->MarkHiddenByCustomization();

		PopulateGroups();	

		ExpressionCategory.AddCustomRow(GroupPropertyHandle->GetPropertyDisplayName())
			.NameContent()
			[
				SNew(STextBlock)
				.Text(GroupPropertyHandle->GetPropertyDisplayName())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(NewComboButton, SComboButton)
				.ContentPadding(FMargin(2.0f, 2.0f))
				.ButtonContent()
				[
					SAssignNew(NewEditBox, SEditableText)
					.Text(this, &FMaterialExpressionParameterDetails::OnGetText)
					.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
					.OnTextCommitted(this, &FMaterialExpressionParameterDetails::OnTextCommitted)
				]
				.MenuContent()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.MaxHeight(400.0f)
					[
						SAssignNew(NewListView, SListView<TSharedPtr<FString>>)
						.ListItemsSource(&GroupsSource)
						.OnGenerateRow(this, &FMaterialExpressionParameterDetails::MakeDetailsGroupViewWidget)
						.OnSelectionChanged(this, &FMaterialExpressionParameterDetails::OnSelectionChanged)
					]
				]
			];

		ExpressionCategory.AddProperty("SortPriority");
	}

	GroupComboButton = NewComboButton;
	GroupEditBox = NewEditBox;
	GroupListView = NewListView;
}

void FMaterialExpressionParameterDetails::PopulateGroups()
{
	TArray<FString> Groups;
	CollectGroupsDelegate.ExecuteIfBound(&Groups);
	Groups.Sort([&](const FString& A, const FString& B) {
		return A.ToLower() < B.ToLower();
	});

	GroupsSource.Empty();
	for (int32 GroupIdx = 0; GroupIdx < Groups.Num(); ++GroupIdx)
	{
		GroupsSource.Add(MakeShareable(new FString(Groups[GroupIdx])));
	}
}

TSharedRef< ITableRow > FMaterialExpressionParameterDetails::MakeDetailsGroupViewWidget( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock) .Text(FText::FromString(*Item.Get()))
		];
}

void FMaterialExpressionParameterDetails::OnSelectionChanged( TSharedPtr<FString> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	if (ProposedSelection.IsValid())
	{
		GroupPropertyHandle->SetValue(*ProposedSelection.Get());
		GroupListView.Pin()->ClearSelection();
		GroupComboButton.Pin()->SetIsOpen(false);
	}
}

void FMaterialExpressionParameterDetails::OnTextCommitted( const FText& InText, ETextCommit::Type /*CommitInfo*/)
{
	GroupPropertyHandle->SetValue(InText.ToString());
	PopulateGroups();
}

FString FMaterialExpressionParameterDetails::OnGetString() const
{
	FString OutText;
	if (GroupPropertyHandle->GetValue(OutText) == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
	return OutText;
}

FText FMaterialExpressionParameterDetails::OnGetText() const
{
	FString NewString = OnGetString();
	return FText::FromString(NewString);
}

void FMaterialExpressionParameterDetails::OnSliderMinMaxEdited()
{
	check(ScalarParameterObjects.Num() == DefaultValueHandles.Num());
	for (int32 Index = 0; Index < ScalarParameterObjects.Num(); Index++)
	{
		UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(ScalarParameterObjects[Index].Get());

		if (ScalarParameter && DefaultValueHandles[Index].IsValid() && DefaultValueHandles[Index]->IsValidHandle())
		{
			if (ScalarParameter->SliderMax > ScalarParameter->SliderMin)
			{
				// Update the values that SPropertyEditorNumeric reads
				// Unfortuantly there is no way to recreate the widget to actually update the UI with these new values
				DefaultValueHandles[Index]->SetInstanceMetaData("UIMin", FString::Printf(TEXT("%f"), ScalarParameter->SliderMin));
				DefaultValueHandles[Index]->SetInstanceMetaData("UIMax", FString::Printf(TEXT("%f"), ScalarParameter->SliderMax));
			}
			else
			{
				DefaultValueHandles[Index]->SetInstanceMetaData("UIMin", TEXT(""));
				DefaultValueHandles[Index]->SetInstanceMetaData("UIMax", TEXT(""));
			}
		}
	}
}

TSharedRef<IDetailCustomization> FMaterialExpressionCollectionParameterDetails::MakeInstance(bool bInIncludeScalars)
{
	return MakeShareable( new FMaterialExpressionCollectionParameterDetails(bInIncludeScalars) );
}

FMaterialExpressionCollectionParameterDetails::FMaterialExpressionCollectionParameterDetails(bool bInIncludeScalars)
	: bIncludeScalars(bInIncludeScalars)
{
}

FText FMaterialExpressionCollectionParameterDetails::GetToolTipText() const
{
	if (ParametersSource.Num() == 1)
	{
		return LOCTEXT("SpecifyCollection", "Specify a Collection to get parameter options");
	}
	else
	{
		return LOCTEXT("ChooseParameter", "Choose a parameter from the collection");
	}
}

FText FMaterialExpressionCollectionParameterDetails::GetParameterNameString() const
{
	FString CurrentParameterName;

	FPropertyAccess::Result Result = ParameterNamePropertyHandle->GetValue(CurrentParameterName);
	if( Result == FPropertyAccess::MultipleValues )
	{
		return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	return FText::FromString(CurrentParameterName);
}

bool FMaterialExpressionCollectionParameterDetails::IsParameterNameComboEnabled() const
{
	UMaterialParameterCollection* Collection = nullptr;
	
	if (CollectionPropertyHandle->IsValidHandle())
	{
		UObject* CollectionObject = nullptr;
		FPropertyAccess::Result Result = CollectionPropertyHandle->GetValue(CollectionObject);

		// No name combo enabled if multiple parameter collection nodes are selected.
		if (Result == FPropertyAccess::MultipleValues)
		{
			return false;
		}

		verify(Result == FPropertyAccess::Success);
		Collection = Cast<UMaterialParameterCollection>(CollectionObject);
	}

	return Collection != nullptr;
}

void FMaterialExpressionCollectionParameterDetails::OnCollectionChanged()
{
	PopulateParameters(TEXT(""));
}

void FMaterialExpressionCollectionParameterDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	// for expression parameters all their properties are in one category based on their class name.
	FName DefaultCategory = NAME_None;
	IDetailCategoryBuilder& Category = DetailLayout.EditCategory( DefaultCategory );

	// Get a handle to the property we are about to edit
	ParameterNamePropertyHandle = DetailLayout.GetProperty( "ParameterName" );
	check(ParameterNamePropertyHandle.IsValid());
	CollectionPropertyHandle = DetailLayout.GetProperty( "Collection" );
	check(CollectionPropertyHandle.IsValid());

	// Register a changed callback on the collection property since we need to update the PropertyName vertical box when it changes
	FSimpleDelegate OnCollectionChangedDelegate = FSimpleDelegate::CreateSP( this, &FMaterialExpressionCollectionParameterDetails::OnCollectionChanged );
	CollectionPropertyHandle->SetOnPropertyValueChanged( OnCollectionChangedDelegate );

	ParameterNamePropertyHandle->MarkHiddenByCustomization();
	CollectionPropertyHandle->MarkHiddenByCustomization();

	PopulateParameters(TEXT(""));

	TSharedPtr<SComboButton> NewComboButton;
	TSharedPtr<SListView<TSharedPtr<FString>>> NewListView;

	// This isn't strictly speaking customized, but we need it to appear before the "Parameter Name" property, 
	// so we manually add it and set MarkHiddenByCustomization on it to avoid it being automatically added
	Category.AddProperty(CollectionPropertyHandle);

	Category.AddCustomRow( ParameterNamePropertyHandle->GetPropertyDisplayName() )
	.NameContent()
	[
		SNew( STextBlock )
		.Text( ParameterNamePropertyHandle->GetPropertyDisplayName() )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	]
	.ValueContent()
	[
		SAssignNew(NewComboButton, SComboButton)
		.IsEnabled(this, &FMaterialExpressionCollectionParameterDetails::IsParameterNameComboEnabled)
		.ContentPadding(0)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FMaterialExpressionCollectionParameterDetails::GetParameterNameString )
		]
		.MenuContent()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(4.0f))
			.AutoHeight()
			[
				SAssignNew(ParameterSearchBox, SSearchBox)
				.OnTextChanged(this, &FMaterialExpressionCollectionParameterDetails::OnFilterTextChanged)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(400.0f)
			[
				SAssignNew(NewListView, SListView<TSharedPtr<FString>>)
				.ListItemsSource(&ParametersSource)
				.OnGenerateRow(this, &FMaterialExpressionCollectionParameterDetails::MakeDetailsGroupViewWidget)
				.OnSelectionChanged(this, &FMaterialExpressionCollectionParameterDetails::OnSelectionChanged)
			]
		]
	];

	ParameterComboButton = NewComboButton;
	ParameterListView = NewListView;

	NewComboButton->SetToolTipText(GetToolTipText());
}

void FMaterialExpressionCollectionParameterDetails::PopulateParameters(const FString& TextFilter)
{
	UMaterialParameterCollection* Collection = nullptr;
	
	if (CollectionPropertyHandle->IsValidHandle())
	{
		UObject* CollectionObject = nullptr;
		FPropertyAccess::Result Result = CollectionPropertyHandle->GetValue(CollectionObject);

		// Return an empty set of parameters if multiple parameter collection nodes are selected.
		if (Result == FPropertyAccess::MultipleValues)
		{
			return;
		}

		verify(Result == FPropertyAccess::Success);
		Collection = Cast<UMaterialParameterCollection>(CollectionObject);
	}

	ParametersSource.Empty();

	if (Collection)
	{
		TArray<FName> NameList;
		if (bIncludeScalars)
		{
			Collection->GetParameterNames(NameList, /*bVectorParameters=*/ false);
		}
		Collection->GetParameterNames(NameList, /*bVectorParameters=*/ true);

		NameList.Sort([](const FName& A, const FName& B) 
		{
			return A.LexicalLess(B);
		});

		for (const FName& Name : NameList)
		{
			TSharedPtr<FString> ParamNameString = MakeShareable(new FString(Name.ToString()));
			if (TextFilter.Len() == 0 || ParamNameString->Contains(TextFilter))
			{
				ParametersSource.Add(ParamNameString);
			}
		}
	}

	if (ParametersSource.Num() == 0)
	{
		ParametersSource.Add(MakeShareable(new FString(LOCTEXT("NoParameter", "None").ToString())));
	}
}

TSharedRef< ITableRow > FMaterialExpressionCollectionParameterDetails::MakeDetailsGroupViewWidget( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock) .Text(FText::FromString(*Item.Get()))
		];
}

void FMaterialExpressionCollectionParameterDetails::OnSelectionChanged( TSharedPtr<FString> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	if (ProposedSelection.IsValid())
	{
		ParameterNamePropertyHandle->SetValue(*ProposedSelection.Get());
		ParameterListView.Pin()->ClearSelection();
		ParameterComboButton.Pin()->SetIsOpen(false);
	}
}

void FMaterialExpressionCollectionParameterDetails::OnFilterTextChanged(const FText& InFilterText)
{
	PopulateParameters(InFilterText.ToString());
	ParameterListView.Pin()->RequestListRefresh();
}

TSharedRef<class IDetailCustomization> FMaterialDetailCustomization::MakeInstance()
{
	return MakeShareable( new FMaterialDetailCustomization );
}

static void HandleNewMaterialTranslatorToggleProperty(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& MaterialCategory, TSharedRef<IPropertyHandle>& PropertyHandle)
{
	static const auto CVarMaterialTranslatorEnableNew = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.Material.Translator.EnableNew"));
	
	// If cvar isn't enabled, hide the property instead.
	if (!CVarMaterialTranslatorEnableNew->GetValueOnAnyThread())
	{
		DetailLayout.HideProperty(PropertyHandle);
		return;
	}
	
	const FText WarningText = LOCTEXT(
		"NewMaterialTranslatorWarning",
		"The new material translator is enabled for this material. This path is still experimental and incomplete. Please do not use it for production."
	);

	MaterialCategory.AddCustomRow(WarningText)
		.WholeRowContent()
		[
			SNew(SBox)
				.Padding(FMargin(0.f, 4.f))
				.Visibility_Lambda([PropertyHandle]()
					{
						bool bEnabled = false;
						PropertyHandle->GetValue(bEnabled);
						return bEnabled ? EVisibility::Visible : EVisibility::Collapsed;
					})
				[
					SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(WarningText)
				]
		];
}

class FDetailGroupsPrioritiesBuilder : public FDetailArrayBuilder, public TSharedFromThis<FDetailGroupsPrioritiesBuilder>
{
public:

	FDetailGroupsPrioritiesBuilder(TSharedRef<IPropertyHandle> InBaseProperty, bool InGenerateHeader = true)
		: FDetailArrayBuilder(InBaseProperty, InGenerateHeader, false /* InDisplayResetToDefault */, false /* InDisplayElementNum */)
	{}

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override
	{
		if (GetShouldGenerateHeader())
		{
			TSharedPtr<SHorizontalBox> ContentHorizontalBox;
			SAssignNew(ContentHorizontalBox, SHorizontalBox);

			ContentHorizontalBox->AddSlot()
			[
				SNew(SSearchBox)
				.OnTextChanged(this, &FDetailGroupsPrioritiesBuilder::OnFilterTextChanged)
			];

			NodeRow
			.FilterString(LOCTEXT("ParameterGroupsName", "Parameter Groups Priorities"))
			.NameContent()
			[
				GetPropertyHandle()->CreatePropertyNameWidget(LOCTEXT("ParameterGroupsName", "Parameter Groups Priorities"), FText::GetEmpty())
			]
			.ValueContent()
			[
				ContentHorizontalBox.ToSharedRef()
			];
		}
	}

	void OnFilterTextChanged(const FText& InFilterText)
	{
		CurrentFilterText = InFilterText.ToString();
		RefreshChildren();
	}

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		TSharedPtr<IPropertyHandleArray> LocalArrayProperty = GetPropertyHandle()->AsArray();

		uint32 NumChildren = 0;
		LocalArrayProperty->GetNumElements(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ElementHandle = LocalArrayProperty->GetElement(ChildIndex);

			GenerateSortingGroupsWidget(ElementHandle, ChildIndex, ChildrenBuilder);
		}
	}

	void GenerateSortingGroupsWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
	{
		const TSharedPtr<IPropertyHandle> GroupNameProp = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParameterGroupData, GroupName));

		FString PropDisplayValue;
		GroupNameProp->GetValueAsDisplayString(PropDisplayValue);

		if (!CurrentFilterText.IsEmpty())
		{
			if (!PropDisplayValue.Contains(CurrentFilterText))
			{
				return;
			}
		}

		IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(PropertyHandle);

		const TSharedPtr<IPropertyHandle> GroupPrioProp = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParameterGroupData, GroupSortPriority));

		FUIAction CopyAction;
		FUIAction PasteAction;
		GroupPrioProp->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

		PropertyRow.CustomWidget(false)
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
			true,
			FSimpleDelegate::CreateLambda([GroupPrioProp]()
			{
				if (GroupPrioProp)
				{
					GroupPrioProp->ResetToDefault();
				}
			})))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(FText::FromString(PropDisplayValue))
			]
			.ValueContent()
			[
				GroupPrioProp->CreatePropertyValueWidget()
			]
			.CopyAction(CopyAction)
			.PasteAction(PasteAction);
	}

	private:
	FString CurrentFilterText;
	TSharedPtr<SSearchBox> GroupSearchBox;
};

void FMaterialDetailCustomization::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{	
	TArray<TWeakObjectPtr<UObject> > Objects;
	DetailLayout.GetObjectsBeingCustomized( Objects );

	bool bUIMaterial = true;
	bool bIsShadingModelFromMaterialExpression = false;
	bool bIsAlphaHoldout = false;
	for( TWeakObjectPtr<UObject>& Object : Objects )
	{
		UMaterial* Material = Cast<UMaterial>( Object.Get() );
		if( Material )
		{
			bUIMaterial &= Material->IsUIMaterial();

			// If any Object has its shading model from material expression
			bIsShadingModelFromMaterialExpression |= Material->IsShadingModelFromMaterialExpression();
			
			bIsAlphaHoldout |= Material->BlendMode == BLEND_AlphaHoldout;
		}
		else
		{
			// this shouldn't happen but just in case, let all properties through
			bUIMaterial = false;
		}
	}

	// Material category
	{
		IDetailCategoryBuilder& MaterialCategory = DetailLayout.EditCategory( TEXT("Material") );

		TArray<TSharedRef<IPropertyHandle>> AllProperties;
		MaterialCategory.GetDefaultProperties( AllProperties );

		for( TSharedRef<IPropertyHandle>& PropertyHandle : AllProperties )
		{
			FProperty* Property = PropertyHandle->GetProperty();
			FName PropertyName = Property->GetFName();

			if (bUIMaterial)
			{
				if(		PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, MaterialDomain) 
					&&	PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, BlendMode) 
					&&	PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, OpacityMaskClipValue) 
					&& 	PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, NumCustomizedUVs)
					&&	PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, bEnableNewHLSLGenerator))
				{
					DetailLayout.HideProperty( PropertyHandle );
				}
			}

			if (!Substrate::IsSubstrateEnabled())
			{
				if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterial, bIsThinSurface))
				{
					DetailLayout.HideProperty(PropertyHandle);
				}
			}

			// Patch blend mode displayed names
			if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterial, BlendMode))
			{
				// Override blend mode appearance based on Substrate being enabled or not.
				UpdateBlendModeEnum();

				PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailLayout]()
					{
						// Refresh to update translucency pass restrictions below
						DetailLayout.ForceRefreshDetails();
					}));
			}

			if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterial, bEnableNewHLSLGenerator))
			{
				HandleNewMaterialTranslatorToggleProperty(DetailLayout, MaterialCategory, PropertyHandle);
			}

#if WITH_EDITORONLY_DATA
			// Hide the shading model 
			if (!bIsShadingModelFromMaterialExpression && PropertyName == GET_MEMBER_NAME_CHECKED(UMaterial, UsedShadingModels))
			{
				 DetailLayout.HideProperty(PropertyHandle);
			}
#endif
		}
	}

	// Translucency category
	{
		IDetailCategoryBuilder& TranslucencyCategory = DetailLayout.EditCategory(TEXT("Translucency"));

		TArray<TSharedRef<IPropertyHandle>> AdvancedProperties;
		TranslucencyCategory.GetDefaultProperties(AdvancedProperties, false, true);

		for (TSharedRef<IPropertyHandle>& PropertyHandle : AdvancedProperties)
		{
			FProperty* Property = PropertyHandle->GetProperty();
			FName PropertyName = Property->GetFName();

			if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterial, TranslucencyPass))
			{
				if (bIsAlphaHoldout)
				{
					static FText RestrictReason = NSLOCTEXT("PropertyEditor", "TranslucencyPassRestriction", "Separate translucency pass should not be used with Alpha Holdout blend mode.");
					TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShared<FPropertyRestriction>(RestrictReason);

					FByteProperty* TranslucencyPassProperty = (FByteProperty*)Property;
					EnumRestriction->AddDisabledValue(TranslucencyPassProperty->Enum->GetNameStringByValue(static_cast<int64>(MTP_AfterDOF)));
					EnumRestriction->AddDisabledValue(TranslucencyPassProperty->Enum->GetNameStringByValue(static_cast<int64>(MTP_AfterMotionBlur)));

					PropertyHandle->AddRestriction(EnumRestriction.ToSharedRef());
				}
			}
		}
	}


	if( bUIMaterial )
	{
		DetailLayout.HideCategory( TEXT("TranslucencySelfShadowing") );
		DetailLayout.HideCategory( TEXT("Translucency") );
		DetailLayout.HideCategory( TEXT("Tessellation") );
		DetailLayout.HideCategory( TEXT("PostProcessMaterial") );
		DetailLayout.HideCategory( TEXT("Lightmass") );
		DetailLayout.HideCategory( TEXT("Thumbnail") );
		DetailLayout.HideCategory( TEXT("MaterialInterface") );
		DetailLayout.HideCategory( TEXT("PhysicalMaterial") );

		// Usage Category
		{
			IDetailCategoryBuilder& UsageCategory = DetailLayout.EditCategory(TEXT("Usage"));

			TArray<TSharedRef<IPropertyHandle>> AllProperties;
			UsageCategory.GetDefaultProperties(AllProperties);

			// Hide all Usage except bUsedWithStaticMesh on UI
			for (TSharedRef<IPropertyHandle>& PropertyHandle : AllProperties)
			{
				FProperty* Property = PropertyHandle->GetProperty();
				FName PropertyName = Property->GetFName();

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, bUsedWithStaticMesh))
				{
					DetailLayout.HideProperty(PropertyHandle);
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		// Mobile category
		{
			IDetailCategoryBuilder& MobileCategory = DetailLayout.EditCategory(TEXT("Mobile"));
			
			TArray<TSharedRef<IPropertyHandle>> AllProperties;
			MobileCategory.GetDefaultProperties(AllProperties);

			for (TSharedRef<IPropertyHandle>& PropertyHandle : AllProperties)
			{
				FProperty* Property = PropertyHandle->GetProperty();
				FName PropertyName = Property->GetFName();

				if (PropertyName != GET_MEMBER_NAME_CHECKED(UMaterial, FloatPrecisionMode))
				{
					DetailLayout.HideProperty(PropertyHandle);
				}
			}
		}
	}

	if (CVarMaterialEdUseGroupCustomUI.GetValueOnGameThread())
	{
		IDetailCategoryBuilder& GroupSortingCategory = DetailLayout.EditCategory("Group Sorting");

		TSharedPtr<IPropertyHandle> SortingGroupDataProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialEditorOnlyData, ParameterGroupData), UMaterialEditorOnlyData::StaticClass());
		check(SortingGroupDataProperty);

		TSharedRef<FDetailGroupsPrioritiesBuilder> SortingGroupDataBuilder = MakeShared<FDetailGroupsPrioritiesBuilder>(SortingGroupDataProperty.ToSharedRef());
		GroupSortingCategory.AddCustomBuilder(SortingGroupDataBuilder);
	}
}

TSharedRef<class IDetailCustomization> FMaterialFunctionDetailCustomization::MakeInstance()
{
	return MakeShareable(new FMaterialFunctionDetailCustomization);
}

void FMaterialFunctionDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// MaterialFunction category
	{
		IDetailCategoryBuilder& MaterialCategory = DetailLayout.EditCategory(TEXT("MaterialFunction"));

		TArray<TSharedRef<IPropertyHandle>> AllProperties;
		MaterialCategory.GetDefaultProperties(AllProperties);

		for (TSharedRef<IPropertyHandle>& PropertyHandle : AllProperties)
		{
			FProperty* Property = PropertyHandle->GetProperty();
			FName PropertyName = Property->GetFName();

			if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterial, bEnableNewHLSLGenerator))
			{
				HandleNewMaterialTranslatorToggleProperty(DetailLayout, MaterialCategory, PropertyHandle);
				break;
			}
		}
	}

	// Hide Thumbnail category if material function preview domain is set to UserInterface
	{
		TArray<TWeakObjectPtr<UObject> > Objects;
		DetailLayout.GetObjectsBeingCustomized(Objects);
		bool bUIMaterial = true;
		for (TWeakObjectPtr<UObject>& Object : Objects)
		{
			UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>(Object.Get());
			if (MaterialFunction)
			{
				bUIMaterial &= MaterialFunction->PreviewMaterialDomain == MD_UI;
			}
			else
			{
				// this shouldn't happen but just in case, let all properties through
				bUIMaterial = false;
				break;
			}
		}

		if (bUIMaterial)
		{
			DetailLayout.HideCategory(TEXT("Thumbnail"));
		}
	}
}

TSharedRef<class IDetailCustomization> FMaterialExpressionLayersParameterDetails::MakeInstance(FOnCollectParameterGroups InCollectGroupsDelegate)
{
	return MakeShareable(new FMaterialExpressionLayersParameterDetails(InCollectGroupsDelegate));
}

FMaterialExpressionLayersParameterDetails::FMaterialExpressionLayersParameterDetails(FOnCollectParameterGroups InCollectGroupsDelegate)
{
	CollectGroupsDelegate = InCollectGroupsDelegate;
}

void FMaterialExpressionLayersParameterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FMaterialExpressionParameterDetails::CustomizeDetails(DetailLayout);
	// for expression parameters all their properties are in one category based on their class name.

	FName LayerCategory = FName("Layers");
	IDetailCategoryBuilder& Category = DetailLayout.EditCategory(LayerCategory);
	// Get a handle to the property we are about to edit
	GroupPropertyHandle = DetailLayout.GetProperty("DefaultLayers");
	GroupPropertyHandle->MarkHiddenByCustomization();
	TSharedRef<FMaterialLayersFunctionsCustomization> MaterialLayersFunctionsCustomization = MakeShareable(new FMaterialLayersFunctionsCustomization(GroupPropertyHandle, &DetailLayout));
	Category.AddCustomBuilder(MaterialLayersFunctionsCustomization);

}

TSharedRef<class IDetailCustomization> FMaterialExpressionCompositeDetails::MakeInstance()
{
	return MakeShareable(new FMaterialExpressionCompositeDetails());
}

void FMaterialExpressionCompositeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FName DefaultCategory = NAME_None;
	IDetailCategoryBuilder& Category = DetailLayout.EditCategory(DefaultCategory);
	Category.AddProperty("SubgraphName");

	const FName MaterialExpressionCategory = TEXT("MaterialExpression");
	IDetailCategoryBuilder& ExpressionCategory = DetailLayout.EditCategory(MaterialExpressionCategory);

	TArray< TWeakObjectPtr<UObject> > Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	
	for (const auto& WeakObjectPtr : Objects)
	{
		UObject* Object = WeakObjectPtr.Get();

		UMaterialExpressionComposite* Composite = Cast<UMaterialExpressionComposite>(Object);

		if (Composite)
		{
			if (IDetailPropertyRow* PinBaseRow = ExpressionCategory.AddExternalObjectProperty({Composite->InputExpressions}, GET_MEMBER_NAME_CHECKED(UMaterialExpressionPinBase, ReroutePins)))
			{
				PinBaseRow->DisplayName(LOCTEXT("InputPinsComposite", "Input Pins"));
				PinBaseRow->Visibility(EVisibility::Visible);
			}

			if (IDetailPropertyRow* PinBaseRow = ExpressionCategory.AddExternalObjectProperty({Composite->OutputExpressions }, GET_MEMBER_NAME_CHECKED(UMaterialExpressionPinBase, ReroutePins)))
			{
				PinBaseRow->DisplayName(LOCTEXT("OutPinsComposite", "Output Pins"));
				PinBaseRow->Visibility(EVisibility::Visible);
			}
		}
	}
}

TSharedRef<class IDetailCustomization> FMaterialExpressionColorRampCustomization::MakeInstance()
{
	return MakeShareable(new FMaterialExpressionColorRampCustomization);
}

void FMaterialExpressionColorRampCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("MaterialExpressionColorRamp");

	// Hide default ColorCurve details
	TSharedPtr<IPropertyHandle> GradientProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialExpressionColorRamp, ColorCurve));
	DetailBuilder.HideProperty(GradientProperty);

	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);
	if (CustomizedObjects.Num() > 0)
	{
		UMaterialExpressionColorRamp* ColorRamp = Cast<UMaterialExpressionColorRamp>(CustomizedObjects[0].Get());
		if (ColorRamp)
		{
			TSharedPtr<SColorGradientEditor> GradientEditor = SNew(SColorGradientEditor)
				.ViewMinInput(0.0f)
				.ViewMaxInput(1.0f)
				.ClampStopsToViewRange(true);
			GradientEditor->SetCurveOwner(ColorRamp->ColorCurve);

			// Recompile shader when the curve is modified
			ColorRamp->ColorCurve->OnUpdateCurve.AddUObject(ColorRamp, &UMaterialExpressionColorRamp::HandleCurvePropertyChanged);
			
			Category.AddCustomRow(FText::FromString("Color Gradient"))
				.NameContent()
				[
					SNew(STextBlock)
						.Text(FText::FromString("Color Gradient"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
				.ValueContent()
				.MinDesiredWidth(300.f)
				[
					GradientEditor.ToSharedRef()
				];
		}
	}
}

TSharedRef<IDetailCustomization> FMaterialExpressionOperatorCustomization::MakeInstance()
{
	return MakeShareable(new FMaterialExpressionOperatorCustomization);
}

void FMaterialExpressionOperatorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Hides default detail panel property for input array
    TSharedRef<IPropertyHandle> DynamicInputsHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialExpressionOperator, DynamicInputs));
	DetailLayout.HideProperty(DynamicInputsHandle);

	// Get Material Expression Operater ref
    UMaterialExpressionOperator* MaterialExpOperator = nullptr;
    TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
    for (TWeakObjectPtr<UObject>& Obj : Objects)
    {
        if (auto* Test = Cast<UMaterialExpressionOperator>(Obj.Get()))
        {
			MaterialExpOperator = Test;
            break;
        }
    }

	if (MaterialExpOperator)
	{
		DetailLayout.EditCategory("MaterialExpressionOperator");
		IDetailCategoryBuilder& InputCategory = DetailLayout.EditCategory("MaterialExpressionOperatorInputs", LOCTEXT("InputsCategory", "Material Expression Operator Inputs"));

		// Add an entry for each input in the input array
		uint32 NumInputs = 0;
		DynamicInputsHandle->GetNumChildren(NumInputs);
		for (uint32 Index = 0; Index < NumInputs; Index++)
		{
			TSharedPtr<IPropertyHandle> ElementHandle = DynamicInputsHandle->GetChildHandle(Index);
			TSharedPtr<IPropertyHandle> ConstValueHandle = ElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMaterialExpressionOperatorInput, ConstValue));
			FText DisplayName = FText::FromName(MaterialExpOperator->GetInputName(Index));

			InputCategory.AddCustomRow(DisplayName)
			.IsEnabled(true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(DisplayName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.MinDesiredWidth(125.0f)
			[
				ConstValueHandle->CreatePropertyValueWidget()
			];
		}
	}

}

#undef LOCTEXT_NAMESPACE
