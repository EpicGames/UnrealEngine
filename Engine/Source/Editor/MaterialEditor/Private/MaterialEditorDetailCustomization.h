// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

class IDetailLayoutBuilder;
class SComboButton;
class SEditableText;

DECLARE_DELEGATE_OneParam(FOnCollectParameterGroups, TArray<FString>*);

class FMaterialExpressionParameterDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(FOnCollectParameterGroups InCollectGroupsDelegate);
	
	/** Constructor requires a delegate to populate group names with */
	FMaterialExpressionParameterDetails(FOnCollectParameterGroups InCollectGroupsDelegate);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

protected:
	/** Default constructor for child overrides */
	FMaterialExpressionParameterDetails();

	/** Populates the group name that this parameter details uses */
	void PopulateGroups();

	/** Functions used to make the custom combo button with editable text */
	TSharedRef< ITableRow > MakeDetailsGroupViewWidget( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable );
	void OnSelectionChanged( TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo );
	void OnTextCommitted( const FText& InText, ETextCommit::Type CommitInfo);
	FString OnGetString() const;
	FText OnGetText() const;
	void OnSliderMinMaxEdited();

protected:
	/** The property handle to the groups */
	TSharedPtr<class IPropertyHandle> GroupPropertyHandle;

	/** Custom widgets (combo button with editable text) to access the property with */
	TWeakPtr<SComboButton> GroupComboButton;
	TWeakPtr<SEditableText> GroupEditBox;
	TWeakPtr<SListView<TSharedPtr<FString>>> GroupListView;

	/** Delegate to call to collect a list of groups with */
	FOnCollectParameterGroups CollectGroupsDelegate;

	/** A list of all group names to choose from */
	TArray<TSharedPtr<FString>> GroupsSource;

	TArray<TWeakObjectPtr<UObject>> ScalarParameterObjects;
	TArray<TSharedPtr<IPropertyHandle>> DefaultValueHandles;
};

class FMaterialExpressionLayersParameterDetails : public FMaterialExpressionParameterDetails
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(FOnCollectParameterGroups InCollectGroupsDelegate);

	/** Constructor requires a delegate to populate group names with */
	FMaterialExpressionLayersParameterDetails(FOnCollectParameterGroups InCollectGroupsDelegate);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	void ResetCategory(IDetailLayoutBuilder* DetailLayout);
};

/** 
 * Customizes the details of a CollectionParameter node, 
 * specifically creating a vertical box for ParameterName with only valid entries based on the current collection. 
 */
class FMaterialExpressionCollectionParameterDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(bool bInIncludeScalars);

	/** Constructor requires a delegate to populate group names with */
	FMaterialExpressionCollectionParameterDetails(bool bInIncludeScalars);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:
	/** Populates the group name that this parameter details uses */
	void PopulateParameters(const FString& FilterText);

	/** Functions used to make the custom combo button with editable text */
	TSharedRef< ITableRow > MakeDetailsGroupViewWidget( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable );
	void OnSelectionChanged( TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo );
	void OnFilterTextChanged(const FText& InFilterText);

	FText GetToolTipText() const;
	FText GetParameterNameString() const;
	bool IsParameterNameComboEnabled() const;
	void OnCollectionChanged();

private:

	TSharedPtr<class IPropertyHandle> CollectionPropertyHandle;

	/** The property handle to the parameter name */
	TSharedPtr<class IPropertyHandle> ParameterNamePropertyHandle;

	TWeakPtr<SComboButton> ParameterComboButton;
	TWeakPtr<SListView<TSharedPtr<FString>>> ParameterListView;
	/** Text entry to filter parameters */
	TSharedPtr<class SSearchBox> ParameterSearchBox;

	/** A list of all parameter names to choose from */
	TArray<TSharedPtr<FString>> ParametersSource;

	/** Whether to include scalars in this detail customization */
	bool bIncludeScalars;
};


/**
 * Customizes the details of a Composite node,
 * specifically exposing pinbases.
 */
class FMaterialExpressionCompositeDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

/**
 * Customizes the details of a Color Ramp node, adding a gradient slider to the panel
 */
class FMaterialExpressionColorRampCustomization : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

/**
 * Customizes the details of Operator node
 */
class FMaterialExpressionOperatorCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

class FMaterialDetailCustomization : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
};

class FMaterialFunctionDetailCustomization : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};
