// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorGraphPanelPinFactory.h"

#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "KismetPins/SGraphPinEnum.h"
#include "HAL/PlatformCrt.h"
#include "KismetPins/SGraphPinBool.h"
#include "KismetPins/SGraphPinColor.h"
#include "KismetPins/SGraphPinExec.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialEditor/MaterialPins/SGraphPinMaterialInput.h"
#include "Misc/Optional.h"
#include "SGraphPin.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "KismetPins/SGraphPinNum.h"
#include "KismetPins/SGraphPinVector2D.h"
#include "KismetPins/SGraphPinVector.h"
#include "KismetPins/SGraphPinVector4.h"
#include "KismetPins/SGraphPinInteger.h"
#include "MaterialValueType.h"

TSharedPtr<class SGraphPin> FMaterialEditorGraphPanelPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (const UMaterialGraphSchema* MaterialGraphSchema = Cast<const UMaterialGraphSchema>(InPin->GetSchema()))
	{
		UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(InPin->GetOwningNode()->GetGraph());
		if (InPin->PinType.PinCategory == MaterialGraphSchema->PC_Exec)
		{
			return SNew(SGraphPinExec, InPin);
		}
		else if (!MaterialGraph->IsInputActive(InPin) && InPin->PinType.PinCategory == MaterialGraphSchema->PC_MaterialInput)
		{
			return SNew(SGraphPinMaterialInput, InPin);
		}
		else
		{
			// get metadata and pass it along
			UObject* NodeOwner = CastChecked<UMaterialGraphNode_Base>(InPin->GetOuter())->GetMaterialNodeOwner();

			TArray<FProperty*> InputProperties;
			if (NodeOwner->IsA<UMaterialExpression>())
			{
				InputProperties = CastChecked<UMaterialExpression>(NodeOwner)->GetInputPinProperty(InPin->SourceIndex);
			}
			FProperty* InProperty = InputProperties.Num() > 0 ? InputProperties[0] : nullptr;

			if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_Red || InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_Float)
			{
				return SNew(SGraphPinNum<double>, InPin);	
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_RG)
			{
				return SNew(SGraphPinVector2D<float>, InPin);
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_RGB)
			{
				return SNew(SGraphPinVector<float>, InPin);
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_RGBA)
			{
				return SNew(SGraphPinColor, InPin);
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_Vector4)
			{
				return SNew(SGraphPinVector4<float>, InPin);
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_Int)
			{
				return SNew(SGraphPinInteger, InPin);
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_Byte)
			{
				// Check for valid enum object reference
				if ((InPin->PinType.PinSubCategoryObject != NULL) && (InPin->PinType.PinSubCategoryObject->IsA(UEnum::StaticClass())))
				{
					// The property may have a ValidEnumValues metadata to filter out invalid values. This array contains the
					// extracted valid values if any, or remains empty if the ValidEnumValues metadata was not specified,
					// which will cause no values to be filtered out.
					TArray<FString> ValidEnumValues;

					// We know the property type is an enum so it is safe to cast here.
					static const FName ValidEnumValuesName("ValidEnumValues");

					// If the property has the ValidEnumValues metadata, it will be in the form of a string of elements separated by
					// comma (with potentially space in between that needs trimming).
					if (InProperty && InProperty->HasMetaData(ValidEnumValuesName))
					{
						// The property has the metadata. Parse the values into an array of strings.
						TArray<FString> ValidEnumValuesAsString;
						InProperty->GetMetaData(ValidEnumValuesName).ParseIntoArray(ValidEnumValuesAsString, TEXT(","));

						// For each value, trim it and then push to ValidEnumValues.
						for(FString& Value : ValidEnumValuesAsString)
						{
							Value.TrimStartAndEndInline();
							ValidEnumValues.Add(Value);
						}
					}

					// Create the SGraphPinEnum with the allowed list of values (empty if no metadata was found).
					return SNew(SGraphPinEnum, InPin)
						.ValidEnumValues(ValidEnumValues);
				}
				else
				{
					return SNew(SGraphPinInteger, InPin);
				}
			}
			else if (InPin->PinType.PinSubCategory == MaterialGraphSchema->PSC_Bool)
			{
				return SNew(SGraphPinBool, InPin);
			}
			else if (InPin->PinType.PinCategory == MaterialGraphSchema->PC_MaterialInput)
			{
				return SNew(SGraphPinMaterialInput, InPin);
			}
		}

		return SNew(SGraphPin, InPin);
	}

	return nullptr;
}
