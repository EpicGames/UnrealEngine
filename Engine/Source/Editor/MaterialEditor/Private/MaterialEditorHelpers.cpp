// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorHelpers.h"
#include "MaterialEditor.h"
#include "AssetToolsModule.h"
#include "MaterialEditorUtilities.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "Factories/MaterialFunctionFactoryNew.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialFunction.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MaterialEditor"

void FMaterialEditorHelpers::CollapseToFunction(FMaterialEditor& MaterialEditor)
{
	if (!ensure(MaterialEditor.OriginalMaterialObject))
	{
		return;
	}

	TSet<UEdGraphNode*> NodesToCollapse;
	for (UObject* NodeObject : MaterialEditor.GetSelectedNodes())
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(NodeObject);
		if (GraphNode && GraphNode->CanDuplicateNode())
		{
			NodesToCollapse.Add(GraphNode);
		}
	}
	
	// Output pin outside the nodes to collapse -> all pins inside the nodes to collapse to connect to that new function input
	TMap<UEdGraphPin*, TArray<UEdGraphPin*>> Inputs;
	// Output pin inside the nodes to collapse -> all pins outside the nodes to collapse to connect to that new function output
	TMap<UEdGraphPin*, TArray<UEdGraphPin*>> Outputs;

	UEdGraph* OldGraph = nullptr;
	
	for (UEdGraphNode* Node : NodesToCollapse)
	{
		OldGraph = Node->GetGraph();

		for (UEdGraphPin* Pin : Node->Pins)
		{
			for (UEdGraphPin* LinkedToPin : Pin->LinkedTo)
			{
				if (!NodesToCollapse.Contains(LinkedToPin->GetOwningNode()))
				{
					if (Pin->Direction == EGPD_Input)
					{
						Inputs.FindOrAdd(LinkedToPin).Add(Pin);
					}
					else
					{
						check(Pin->Direction == EGPD_Output);
						Outputs.FindOrAdd(Pin).Add(LinkedToPin);
					}
				}
			}
		}
	}
	
	// Expand the bounds to ensure the function input/output nodes don't overlap with the pasted nodes in the middle
	FBox2D NodeBounds = GetNodesBounds(MaterialEditor, NodesToCollapse);
	NodeBounds = NodeBounds.ExpandBy(300);

	// Sort inputs & outputs by the average position of the pins inside the function
	// This reduces the risk of having crossing links
	{
		const auto GetPinSortValue = [](UEdGraphPin* Pin)
		{
			return 100 * Pin->GetOwningNode()->NodePosY + Pin->GetOwningNode()->GetPinIndex(Pin);
		};

		Inputs.ValueSort([&](const TArray<UEdGraphPin*>& PinsA, const TArray<UEdGraphPin*>& PinsB)
		{
			float PositionA = 0;
			for (UEdGraphPin* Pin : PinsA)
			{
				PositionA += GetPinSortValue(Pin);
			}
			PositionA /= PinsA.Num();

			float PositionB = 0;
			for (UEdGraphPin* Pin : PinsB)
			{
				PositionB += GetPinSortValue(Pin);
			}
			PositionB /= PinsB.Num();

			return PositionA < PositionB;
		});
		Outputs.KeySort([&](UEdGraphPin& PinA, UEdGraphPin& PinB)
		{
			return GetPinSortValue(&PinA) < GetPinSortValue(&PinB);
		});
	}

	if (Inputs.Num() == 0 && Outputs.Num() == 0)
	{
		// Avoids issues below
		return;
	}

	UMaterialFunction* NewMaterialFunction = nullptr;
	UMaterialGraph* NewGraph = nullptr;
	FMaterialEditor* NewMaterialEditor = nullptr;

	// Show prompt asking the user where the new asset should be placed
	{
		const FString DefaultSuffix = TEXT("_Func");

		FString Name;
		FString PackageName;
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.CreateUniqueAssetName(MaterialEditor.OriginalMaterialObject->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

		UMaterialFunctionFactoryNew* Factory = NewObject<UMaterialFunctionFactoryNew>();
		UObject* FunctionObject = AssetTools.CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunction::StaticClass(), Factory);
		if (!FunctionObject)
		{
			// Cancelled
			return;
		}

		NewMaterialFunction = Cast<UMaterialFunction>(FunctionObject);
		if (!ensure(NewMaterialFunction))
		{
			return;
		}
		
		// Open the asset editor for the material function, so we can paste the nodes in it
		// This is somewhat hacky, but much simpler than dealing with the material expressions themselves
		// Needs to be done outside of the transaction, otherwise internal changes (like creating the graph schema) will be undone as well & will crash
		NewMaterialEditor = OpenMaterialEditorForAsset(NewMaterialFunction);
		if (!ensure(NewMaterialEditor) ||
			!ensure(NewMaterialEditor->Material) ||
			!ensure(NewMaterialEditor->Material->MaterialGraph))
		{
			return;
		}
		
		NewGraph = NewMaterialEditor->Material->MaterialGraph;
	}

	// Create the new function nodes
	{
		// Make sure to not put the asset creation or OpenMaterialEditorForAsset in the transaction - that won't undo properly
		// We need two transactions here, as we need to call UpdateOriginalMaterial before spawning the new function call node
		// and calling UpdateOriginalMaterial inside a transactions seems to create some asset corruption on undo
		// 
		// Example of repro when UpdateOriginalMaterial is in a transaction:
		// Collapse to function -> Ctrl Shift S -> Ctrl Z -> Ctrl Shift S -> the material object function is now deleted
		// However the asset editor for the material function is still open, and references an invalid material function.
		// The asset is also still visible in the content browser, but is nulled in memory (right clicking the asset will crash)
		const FScopedTransaction Transaction(LOCTEXT("OnCollapseToFunctionCreateNewFunction", "Collapse to function - create new function"));

		TMap<FGuid, UMaterialGraphNode*> OldGuidToNewNode;

		// Paste the nodes inside the new function
		{
			// Delete all default nodes
			// Need to take copy as DeleteNodes will remove them from the graph
			TArray<UEdGraphNode*> NodesCopy = NewGraph->Nodes;
			NewMaterialEditor->DeleteNodes(NodesCopy, false);
			ensure(NewGraph->Nodes.Num() == 0);

			// Copy the selected nodes
			const FString Clipboard = MaterialEditor.CopyNodesToBuffer(NodesToCollapse);

			// Paste the nodes, keeping track of their new GUIDs
			// Note that the pins GUIDs don't change on paste - their uniqueness is only guaranteed within their node
			TMap<FGuid, FGuid> OldToNewGuids;
			NewMaterialEditor->PasteNodesHereFromBuffer(FVector2D::ZeroVector, NewGraph, Clipboard, &OldToNewGuids);

			FindNewNodes(*NewMaterialEditor, OldGuidToNewNode, OldToNewGuids);
		}

		TSet<FName> UsedPinNames;
		const auto GetFunctionPinName = [&](UEdGraphPin* Pin)
		{
			// For function inputs/outputs, we try to give them nice names based on the pin they are linked to

			FName Name = Pin->PinName;
			if (Name.IsNone() || Name == "Output")
			{
				// Pin doesn't have a name, fall back to the node title
				
				UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(Pin->GetOwningNode());
				if (ensure(Node) && ensure(Node->MaterialExpression))
				{
					TArray<FString> Captions;
					Node->MaterialExpression->GetCaption(Captions);
					if (Captions.Num() > 0)
					{
						Name = *Captions[0];
					}
				}
			}

			// Ensure the function parameters are unique
			while (UsedPinNames.Contains(Name))
			{
				Name.SetNumber(Name.GetNumber() + 1);
			}
			UsedPinNames.Add(Name);

			return Name;
		};

		{
			int32 InputIndex = 0;
			for (auto& It : Inputs)
			{
				// This pin is outside of the collapsed nodes
				UEdGraphPin* PinLinkedToInput = It.Key;
				UMaterialGraphNode* GraphNodeLinkedToInput = CastChecked<UMaterialGraphNode>(PinLinkedToInput->GetOwningNode());

				EMaterialValueType PinMaterialType = GraphNodeLinkedToInput->GetOutputValueType(PinLinkedToInput);
				EFunctionInputType PinFunctionType = {};

				// This could maybe be shared with UMaterialExpressionFunctionInput, but there are tricky details involved with MCT_Float1 vs MCT_Float
				static_assert(FunctionInput_MAX == 13, "Need to update");
				switch (PinMaterialType)
				{
				case MCT_Float1:             PinFunctionType = FunctionInput_Scalar;             break;
				case MCT_Float2:             PinFunctionType = FunctionInput_Vector2;            break;
				case MCT_Float3:             PinFunctionType = FunctionInput_Vector3;            break;
				case MCT_Float4:             PinFunctionType = FunctionInput_Vector4;            break;
				case MCT_Texture2D:          PinFunctionType = FunctionInput_Texture2D;          break;
				case MCT_TextureCube:        PinFunctionType = FunctionInput_TextureCube;        break;
				case MCT_Texture2DArray:     PinFunctionType = FunctionInput_Texture2DArray;     break;
				case MCT_VolumeTexture:      PinFunctionType = FunctionInput_VolumeTexture;      break;
				case MCT_StaticBool:         PinFunctionType = FunctionInput_StaticBool;         break;
				case MCT_MaterialAttributes: PinFunctionType = FunctionInput_MaterialAttributes; break;
				case MCT_TextureExternal:    PinFunctionType = FunctionInput_TextureExternal;    break;
				case MCT_Bool:				 PinFunctionType = FunctionInput_Bool;				 break;
				case MCT_Substrate:			 PinFunctionType = FunctionInput_Substrate;			 break;
				default:
					// Will happen pretty often with MCT_Float, as the types are rarely fully resolved
					// (eg Add nodes can take float1/2/3/4)
					PinFunctionType = FunctionInput_Scalar;
				}

				const FVector2D Location = FVector2D(-NodeBounds.GetExtent().X, (InputIndex - Inputs.Num() / 2) * 100);

				UMaterialExpressionFunctionInput* FunctionInput = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionFunctionInput>(
					NewGraph,
					Location,
					false);

				FunctionInput->bCollapsed = true;
				FunctionInput->InputName = GetFunctionPinName(PinLinkedToInput);
				FunctionInput->Id = FGuid::NewGuid();
				FunctionInput->SortPriority = 10 * InputIndex;
				FunctionInput->InputType = PinFunctionType;

				// For each of the pins inside of the collapsed nodes, make a link to the new function input

				UMaterialGraphNode* FunctionInputGraphNode = CastChecked<UMaterialGraphNode>(FunctionInput->GraphNode);
				for (UEdGraphPin* OldPin : It.Value)
				{
					UEdGraphPin* NewPin = FindNewPin(OldPin, OldGuidToNewNode);
					if (!ensure(NewPin))
					{
						continue;
					}

					FunctionInputGraphNode->GetOutputPin(0)->MakeLinkTo(NewPin);
				}

				InputIndex++;
			}
		}

		{
			int32 OutputIndex = 0;
			for (auto& It : Outputs)
			{
				// This pin is inside the collapsed nodes
				UEdGraphPin* OutputPin = It.Key;

				const FVector2D Location = FVector2D(NodeBounds.GetExtent().X, (OutputIndex - Outputs.Num() / 2) * 100);

				UMaterialExpressionFunctionOutput* FunctionOutput = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionFunctionOutput>(
					NewGraph,
					Location,
					false);

				FunctionOutput->bCollapsed = true;
				FunctionOutput->OutputName = GetFunctionPinName(OutputPin);
				FunctionOutput->Id = FGuid::NewGuid();
				FunctionOutput->SortPriority = 10 * OutputIndex;

				UMaterialGraphNode* FunctionOutputGraphNode = CastChecked<UMaterialGraphNode>(FunctionOutput->GraphNode);

				UMaterialGraphNode* NewGraphNode = OldGuidToNewNode.FindRef(OutputPin->GetOwningNode()->NodeGuid);
				if (!ensure(NewGraphNode))
				{
					continue;
				}

				// Find OutputPin on the pasted nodes

				UEdGraphPin* NewPin = FindNewPin(OutputPin, OldGuidToNewNode);
				if (!ensure(NewPin))
				{
					continue;
				}
				FunctionOutputGraphNode->GetInputPin(0)->MakeLinkTo(NewPin);

				OutputIndex++;
			}
		}
	}

	// Focus one of the newly spawned nodes - by default it'll focus the now deleted default Result output
	NewMaterialEditor->JumpToNode(NewGraph->Nodes[0]);
	
	// Update the material function to have the correct function pins
	// This MUST NOT be in a transaction
	NewMaterialEditor->UpdateOriginalMaterial();

	const FScopedTransaction Transaction(LOCTEXT("OnCollapseToFunction", "Collapse to function"));

	UMaterialExpressionMaterialFunctionCall* FunctionCall = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionMaterialFunctionCall>(
		OldGraph,
		NodeBounds.GetCenter(),
		true);
	if (!ensure(FunctionCall->SetMaterialFunctionEx(nullptr, NewMaterialFunction)))
	{
		return;
	}
	
	UMaterialGraphNode* FunctionCallGraphNode = CastChecked<UMaterialGraphNode>(FunctionCall->GraphNode);

	// Connect inputs
	{
		int32 InputIndex = 0;
		for (auto& It : Inputs)
		{
			UEdGraphPin* PinLinkedToInput = It.Key;
			UEdGraphPin* InputPin = FunctionCallGraphNode->GetInputPin(InputIndex);
			if (ensure(InputPin))
			{
				InputPin->MakeLinkTo(PinLinkedToInput);
			}
			
			InputIndex++;
		}
	}

	// Connect outputs
	{
		int32 OutputIndex = 0;
		for (auto& It : Outputs)
		{
			for (UEdGraphPin* PinLinkedToOutput : It.Value)
			{
				UEdGraphPin* OutputPin = FunctionCallGraphNode->GetOutputPin(OutputIndex);
				if (ensure(OutputPin))
				{
					OutputPin->MakeLinkTo(PinLinkedToOutput);
				}
			}
			
			OutputIndex++;
		}
	}

	// Remove the old nodes that are now collapsed
	MaterialEditor.DeleteNodes(NodesToCollapse.Array(), false);

	// Make sure the active material editor still is the one focused
	MaterialEditor.FocusWindow();
}

void FMaterialEditorHelpers::ExpandNode(FMaterialEditor& MaterialEditor)
{
	TMap<UMaterialGraphNode*, FMaterialEditor*> FunctionCalls;
	for (UObject* NodeObject : MaterialEditor.GetSelectedNodes())
	{
		UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(NodeObject);
		if (!Node)
		{
			continue;
		}
		
		UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(Node->MaterialExpression);
		if (!FunctionCallExpression)
		{
			continue;
		}

		UMaterialFunctionInterface* MaterialFunction = FunctionCallExpression->MaterialFunction;
		if (!MaterialFunction)
		{
			continue;
		}
		
		FMaterialEditor* FunctionMaterialEditor = OpenMaterialEditorForAsset(MaterialFunction);
		if (!ensure(FunctionMaterialEditor))
		{
			continue;
		}
		FunctionCalls.Add(Node, FunctionMaterialEditor);
	}

	// Do the transaction after all the OpenMaterialEditorForAsset are done
	const FScopedTransaction Transaction(LOCTEXT("ExpandNode", "Expand node"));
	
	for (auto& It : FunctionCalls)
	{
		ExpandNode(MaterialEditor, *It.Value, It.Key);
	}
	
	// Make the the active material editor still is the one focused
	MaterialEditor.FocusWindow();
}

void FMaterialEditorHelpers::ExpandNode(FMaterialEditor& MaterialEditor, FMaterialEditor& FunctionMaterialEditor, UMaterialGraphNode* FunctionCallNode)
{
	if (!ensure(MaterialEditor.Material) ||
		!ensure(MaterialEditor.Material->MaterialGraph))
	{
		return;
	}
	UMaterialGraph* Graph = MaterialEditor.Material->MaterialGraph;
	
	UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = CastChecked<UMaterialExpressionMaterialFunctionCall>(FunctionCallNode->MaterialExpression);
	
	// Copy the function nodes into this graph
	TMap<FGuid, UMaterialGraphNode*> OldGuidToNewNode;
	TMap<FGuid, UMaterialExpressionFunctionInput*> IdsToFunctionInputs;
	TMap<FGuid, UMaterialExpressionFunctionOutput*> IdsToFunctionOutputs;
	FVector2D PastePositionOffset(ForceInit);
	{
		if (!ensure(FunctionMaterialEditor.OriginalMaterial) ||
			!ensure(FunctionMaterialEditor.OriginalMaterial->MaterialGraph))
		{
			return;
		}
		
		TSet<UEdGraphNode*> NodesToCopy(FunctionMaterialEditor.OriginalMaterial->MaterialGraph->Nodes);

		// Make sure to not copy the function input/outputs - these can't be copied into materials
		for (auto It = NodesToCopy.CreateIterator(); It; ++It)
		{
			UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(*It);
			if (!MaterialNode)
			{
				continue;
			}

			UMaterialExpressionFunctionInput* FunctionInput = Cast<UMaterialExpressionFunctionInput>(MaterialNode->MaterialExpression);
			if (FunctionInput)
			{
				ensure(!IdsToFunctionInputs.Contains(FunctionInput->Id));
				IdsToFunctionInputs.Add(FunctionInput->Id, FunctionInput);
				It.RemoveCurrent();
				continue;
			}

			UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(MaterialNode->MaterialExpression);
			if (FunctionOutput)
			{
				ensure(!IdsToFunctionOutputs.Contains(FunctionOutput->Id));
				IdsToFunctionOutputs.Add(FunctionOutput->Id, FunctionOutput);
				It.RemoveCurrent();
				continue;
			}
		}

		const FString Clipboard = FunctionMaterialEditor.CopyNodesToBuffer(NodesToCopy);
		const FVector2D PasteLocation(FunctionCallNode->NodePosX, FunctionCallNode->NodePosY);

		// Compute the offset in a similar way PasteNodesHereFromBuffer does it, if we need to add new node for input previews
		FVector2D AveragePosition = FVector2D::ZeroVector;
		for (UEdGraphNode* Node : NodesToCopy)
		{
			AveragePosition.X += Node->NodePosX;
			AveragePosition.Y += Node->NodePosY;
		}
		AveragePosition /= NodesToCopy.Num();
		PastePositionOffset = PasteLocation - AveragePosition;
		
		TMap<FGuid, FGuid> OldToNewGuids;
		MaterialEditor.PasteNodesHereFromBuffer(PasteLocation, Graph, Clipboard, &OldToNewGuids);

		FindNewNodes(MaterialEditor, OldGuidToNewNode, OldToNewGuids);
	}

	TArray<UEdGraphPin*> FunctionCallInputPins;
	TArray<UEdGraphPin*> FunctionCallOutputPins;
	
	TArray<UEdGraphPin*> Pins = FunctionCallNode->Pins;
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			FunctionCallInputPins.Add(Pins[PinIndex]);
		}
		else
		{
			FunctionCallOutputPins.Add(Pins[PinIndex]);
		}
	}

	if (!ensure(FunctionCallInputPins.Num() == FunctionCallExpression->FunctionInputs.Num()) ||
		!ensure(FunctionCallOutputPins.Num() == FunctionCallExpression->FunctionOutputs.Num()))
	{
		return;
	}

	// Fixup inputs, making sure to handle preview values correctly
	for (int32 InputIndex = 0; InputIndex < FunctionCallInputPins.Num(); InputIndex++)
	{
		UEdGraphPin* FunctionCallInputPin = FunctionCallInputPins[InputIndex];
		const FFunctionExpressionInput& FunctionCallInput = FunctionCallExpression->FunctionInputs[InputIndex];
		UMaterialExpressionFunctionInput* FunctionInput = IdsToFunctionInputs.FindRef(FunctionCallInput.ExpressionInputId);
		UMaterialGraphNode* FunctionInputNode = FunctionInput ? Cast<UMaterialGraphNode>(FunctionInput->GraphNode) : nullptr;

		if (!ensure(FunctionInput) || !ensure(FunctionInputNode))
		{
			continue;
		}

		UEdGraphPin* PinLinkedToInput = nullptr;
		if (FunctionCallInputPin->LinkedTo.Num() > 0)
		{
			// If the input is connected, just use that as input

			ensure(FunctionCallInputPin->LinkedTo.Num() == 1);
			PinLinkedToInput = FunctionCallInputPin->LinkedTo[0];
		}
		else
		{
			// If no input connected, we need to figure out what to do with preview pins
			// if bUsePreviewValueAsDefault is false this is technically a compilation error - but it's better if we just ignore that and use the preview value anyway

			TArray<UEdGraphPin*> InputPins;
			for (int32 PinIndex = 0; PinIndex < FunctionInputNode->Pins.Num(); PinIndex++)
			{
				if (FunctionInputNode->Pins[PinIndex]->Direction == EGPD_Input)
				{
					InputPins.Add(FunctionInputNode->Pins[PinIndex]);
				}
			}
			
			if (!ensure(InputPins.Num() == 1))
			{
				continue;
			}
			
			if (InputPins[0]->LinkedTo.Num() > 0)
			{
				ensure(InputPins[0]->LinkedTo.Num() == 1);
				PinLinkedToInput = FindNewPin(InputPins[0]->LinkedTo[0], OldGuidToNewNode);
			}
			else
			{
				// See UMaterialExpressionFunctionInput::CompilePreviewValue

				const FVector2D NewNodePosition = FVector2D(FunctionInputNode->NodePosX, FunctionInputNode->NodePosY) + PastePositionOffset;

				switch (FunctionInput->InputType)
				{
				case FunctionInput_Scalar:
				{
					UMaterialExpressionConstant* Constant = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionConstant>(
						Graph,
						NewNodePosition,
						false);
					Constant->R = FunctionInput->PreviewValue.X;
					PinLinkedToInput = CastChecked<UMaterialGraphNode>(Constant->GraphNode)->GetOutputPin(0);
					break;
				}
				case FunctionInput_Vector2:
				{
					UMaterialExpressionConstant2Vector* Constant = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionConstant2Vector>(
						Graph,
						NewNodePosition,
						false);
					Constant->R = FunctionInput->PreviewValue.X;
					Constant->G = FunctionInput->PreviewValue.Y;
					PinLinkedToInput = CastChecked<UMaterialGraphNode>(Constant->GraphNode)->GetOutputPin(0);
					break;
				}
				case FunctionInput_Vector3:
				{
					UMaterialExpressionConstant3Vector* Constant = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionConstant3Vector>(
						Graph,
						NewNodePosition,
						false);
					Constant->Constant = FLinearColor(FunctionInput->PreviewValue);
					PinLinkedToInput = CastChecked<UMaterialGraphNode>(Constant->GraphNode)->GetOutputPin(0);
					break;
				}
				case FunctionInput_Vector4:
				{
					UMaterialExpressionConstant4Vector* Constant = FMaterialEditorUtilities::CreateNewMaterialExpression<UMaterialExpressionConstant4Vector>(
						Graph,
						NewNodePosition,
						false);
					Constant->Constant = FLinearColor(FunctionInput->PreviewValue);
					PinLinkedToInput = CastChecked<UMaterialGraphNode>(Constant->GraphNode)->GetOutputPin(0);
					break;
				}
				default: break;
				}
			}
		}

		if (!PinLinkedToInput)
		{
			// Can happen if the preview value is not supported
			continue;
		}

		TArray<UEdGraphPin*> OutputPins;
		for (int32 PinIndex = 0; PinIndex < FunctionInputNode->Pins.Num(); PinIndex++)
		{
			if (FunctionInputNode->Pins[PinIndex]->Direction == EGPD_Output)
			{
				OutputPins.Add(FunctionInputNode->Pins[PinIndex]);
			}
		}

		if (!ensure(OutputPins.Num() == 1))
		{
			continue;
		}

		for (UEdGraphPin* LinkedToInMaterialFunction : OutputPins[0]->LinkedTo)
		{
			UEdGraphPin* LinkedToInPastedNodes = FindNewPin(LinkedToInMaterialFunction, OldGuidToNewNode);
			if (ensure(LinkedToInPastedNodes))
			{
				PinLinkedToInput->MakeLinkTo(LinkedToInPastedNodes);
			}
		}
	}
	
	// Fixup outputs
	for (int32 OutputIndex = 0; OutputIndex < FunctionCallOutputPins.Num(); OutputIndex++)
	{
		UEdGraphPin* FunctionCallOutputPin = FunctionCallOutputPins[OutputIndex];
		const FFunctionExpressionOutput& FunctionCallOutput = FunctionCallExpression->FunctionOutputs[OutputIndex];
		UMaterialExpressionFunctionOutput* FunctionOutput = IdsToFunctionOutputs.FindRef(FunctionCallOutput.ExpressionOutputId);
		UMaterialGraphNode* FunctionOutputNode = FunctionOutput ? Cast<UMaterialGraphNode>(FunctionOutput->GraphNode) : nullptr;

		if (!ensure(FunctionOutput) || !ensure(FunctionOutputNode))
		{
			continue;
		}

		TArray<UEdGraphPin*> InputPins;
		for (int32 PinIndex = 0; PinIndex < FunctionOutputNode->Pins.Num(); PinIndex++)
		{
			if (FunctionOutputNode->Pins[PinIndex]->Direction == EGPD_Input)
			{
				InputPins.Add(FunctionOutputNode->Pins[PinIndex]);
			}
		}

		if (!ensure(InputPins.Num() == 1))
		{
			continue;
		}
		if (InputPins[0]->LinkedTo.Num() == 0)
		{
			// Function output is not connected
			continue;
		}
		ensure(InputPins[0]->LinkedTo.Num() == 1);

		UEdGraphPin* LinkedToInPastedNodes = FindNewPin(InputPins[0]->LinkedTo[0], OldGuidToNewNode);
		if (!ensure(LinkedToInPastedNodes))
		{
			continue;
		}
		
		for (UEdGraphPin* PinLinkedToOutput : FunctionCallOutputPin->LinkedTo)
		{
			PinLinkedToOutput->MakeLinkTo(LinkedToInPastedNodes);
		}
	}

	// Finally, delete the function call node
	MaterialEditor.DeleteNodes({ FunctionCallNode }, false);
}

FBox2D FMaterialEditorHelpers::GetNodesBounds(FMaterialEditor& MaterialEditor, const TSet<UEdGraphNode*>& Nodes)
{
	FBox2D Bounds(ForceInit);
	for (UEdGraphNode* Node : Nodes)
	{
		FSlateRect Rect;
		MaterialEditor.GetBoundsForNode(Node, Rect, 0);

		Bounds += Rect.GetBottomLeft();
		Bounds += Rect.GetTopRight();
	}
	return Bounds;
}

void FMaterialEditorHelpers::FindNewNodes(FMaterialEditor& MaterialEditor, TMap<FGuid, UMaterialGraphNode*>& OutOldGuidToNewNode, const TMap<FGuid, FGuid>& OldToNewGuids)
{
	if (!ensure(MaterialEditor.Material) ||
		!ensure(MaterialEditor.Material->MaterialGraph))
	{
		return;
	}
	UMaterialGraph* Graph = MaterialEditor.Material->MaterialGraph;
	
	TMap<FGuid, UMaterialGraphNode*> GuidToNode;
	for (UEdGraphNode* Node : Graph->Nodes)
	{
		GuidToNode.Add(Node->NodeGuid, Cast<UMaterialGraphNode>(Node));
	}

	for (auto& It : OldToNewGuids)
	{
		OutOldGuidToNewNode.Add(It.Key, GuidToNode.FindRef(It.Value));
	}
	
}

UEdGraphPin* FMaterialEditorHelpers::FindNewPin(UEdGraphPin* OldPin, const TMap<FGuid, UMaterialGraphNode*>& OldGuidToNewNode)
{
	if (!ensure(OldPin) || !ensure(OldPin->GetOwningNode()))
	{
		return nullptr;
	}
	
	UMaterialGraphNode* NewGraphNode = OldGuidToNewNode.FindRef(OldPin->GetOwningNode()->NodeGuid);
	if (!ensure(NewGraphNode))
	{
		return nullptr;
	}
	
	// Find OldPin on the pasted nodes
	// Pin GUIDs aren't changed on paste

	UEdGraphPin* NewPin = nullptr;
	for (UEdGraphPin* Pin : NewGraphNode->Pins)
	{
		if (Pin->PinId == OldPin->PinId)
		{
			NewPin = Pin;
		}
	}
	ensure(NewPin);
	return NewPin;
}

FMaterialEditor* FMaterialEditorHelpers::OpenMaterialEditorForAsset(UObject* Asset)
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	AssetEditorSubsystem->OpenEditorForAsset(Asset);

	IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(Asset, true);
	if (!ensure(AssetEditor) ||
		// Clumsy type safety check just in case - see FMaterialEditor::GetToolkitFName
		!ensure(AssetEditor->GetEditorName() == "MaterialEditor"))
	{
		return nullptr;
	}

	return static_cast<FMaterialEditor*>(AssetEditor);
}

#undef LOCTEXT_NAMESPACE