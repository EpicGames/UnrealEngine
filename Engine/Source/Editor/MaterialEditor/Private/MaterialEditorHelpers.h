// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UEdGraphPin;
class FMaterialEditor;
class UMaterialGraphNode;
class UEdGraphNode;

class FMaterialEditorHelpers
{
public:
	static void CollapseToFunction(FMaterialEditor& MaterialEditor);
	static void ExpandNode(FMaterialEditor& MaterialEditor);
	static void ExpandNode(FMaterialEditor& MaterialEditor, FMaterialEditor& FunctionMaterialEditor, UMaterialGraphNode* FunctionCallNode);
	static FMaterialEditor* OpenMaterialEditorForAsset(UObject* Asset);
private:
	static FBox2D GetNodesBounds(FMaterialEditor& MaterialEditor, const TSet<UEdGraphNode*>& Nodes);
	static void FindNewNodes(FMaterialEditor& MaterialEditor, TMap<FGuid, UMaterialGraphNode*>& OutOldGuidToNewNode, const TMap<FGuid, FGuid>& OldToNewGuids);
	static UEdGraphPin* FindNewPin(UEdGraphPin* OldPin, const TMap<FGuid, UMaterialGraphNode*>& OldGuidToNewNode);
	
};