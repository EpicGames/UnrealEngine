// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorInstanceDetailCustomization.h"
#include "Misc/MessageDialog.h"
#include "Misc/Guid.h"
#include "UObject/UnrealType.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "MaterialCachedData.h"
#include "Materials/MaterialInterface.h"
#include "MaterialEditor/DEditorParameterCollectionParameterValue.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialParameterCollection.h"
#include "MaterialShared.h"
#include "EditorSupportDelegates.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialPropertyHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Curves/CurveLinearColor.h"
#include "IPropertyUtilities.h"
#include "Engine/Texture.h"
#include "HAL/PlatformApplicationMisc.h"
#include "RenderUtils.h"
#include "Editor.h"
#include "MaterialEditorValidation.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SToolTip.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#define LOCTEXT_NAMESPACE "MaterialInstanceEditor"


TSharedRef<IDetailCustomization> FMaterialInstanceParameterDetails::MakeInstance(UMaterialEditorInstanceConstant* MaterialInstance, SMaterialLayersFunctionsInstanceWrapper* InMaterialLayersFunctionsInstance, FGetShowHiddenParameters InShowHiddenDelegate)
{
	return MakeShareable(new FMaterialInstanceParameterDetails(MaterialInstance, InMaterialLayersFunctionsInstance, InShowHiddenDelegate));
}

FMaterialInstanceParameterDetails::FMaterialInstanceParameterDetails(UMaterialEditorInstanceConstant* MaterialInstance, SMaterialLayersFunctionsInstanceWrapper* InMaterialLayersFunctionsInstance, FGetShowHiddenParameters InShowHiddenDelegate)
	: MaterialEditorInstance(MaterialInstance)
	, MaterialLayersFunctionsInstance(InMaterialLayersFunctionsInstance) 
	, ShowHiddenDelegate(InShowHiddenDelegate)
{
}

FString FMaterialInstanceParameterDetails::GetFunctionParentPath() const
{
	FString PathString;
	if (MaterialEditorInstance->SourceFunction)
	{
		PathString = MaterialEditorInstance->SourceFunction->Parent->GetPathName();
	}
	return PathString;
}

void UpdateBlendModeEnum();

void FMaterialInstanceParameterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	PropertyUtilities = DetailLayout.GetPropertyUtilities();

	// Override blend mode appearance based on Substrate being enabled or not.
	UpdateBlendModeEnum();

	// Create a new category for a custom layout for the MIC parameters at the very top
	FName GroupsCategoryName = TEXT("ParameterGroups");
	IDetailCategoryBuilder& GroupsCategory = DetailLayout.EditCategory(GroupsCategoryName, LOCTEXT("MICParamGroupsTitle", "Parameter Groups"));
	
	FMaterialEditorValidatorContext ValidationContext;
	// Properties are updated when a static switch is modified so need to run once per update
	if (UMaterialEditorValidationSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMaterialEditorValidationSubsystem>())
	{
		const bool bWarn = !Subsystem->Validate(MaterialEditorInstance->GetMaterialInterface(), ValidationContext);
		if (bWarn)
		{
			// Insert optional validation warning row at the top of the category.
			const bool bHasHyperlink = ValidationContext.HyperlinkAction.IsBound();
			const FSimpleDelegate HyperlinkAction = ValidationContext.HyperlinkAction;
			const FText  HyperlinkText = ValidationContext.HyperlinkText;
			GroupsCategory.AddCustomRow(LOCTEXT("ValidationWarningRow", "Validation Warning"))
			.WholeRowContent()
			[
				SNew(SWarningOrErrorBox)
				.Message(ValidationContext.Message)
				[
					SNew(SButton)
					.ContentPadding(FMargin(4.0, 6.0))
					.ToolTipText(HyperlinkText)
					.Visibility(bHasHyperlink ? EVisibility::Visible : EVisibility::Collapsed)
					.OnClicked_Lambda([HyperlinkAction]()
					{
						HyperlinkAction.ExecuteIfBound();
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Search"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
		}
	}
	
	TSharedRef<IPropertyHandle> ParameterGroupsProperty = DetailLayout.GetProperty("ParameterGroups");
	CreateGroupsWidget(ParameterGroupsProperty, GroupsCategory);
	CreateParameterCollectionOverrideCategory(ParameterGroupsProperty, DetailLayout);

	// Create default category for class properties
	const FName DefaultCategoryName = NAME_None;
	IDetailCategoryBuilder& DefaultCategory = DetailLayout.EditCategory(DefaultCategoryName);
	DetailLayout.HideProperty("MaterialLayersParameterValues");
	if (MaterialEditorInstance->bIsFunctionPreviewMaterial)
	{
		// Customize Parent property so we can check for recursively set parents
		bool bShowParent = false;
		if(MaterialEditorInstance->SourceFunction->GetMaterialFunctionUsage() != EMaterialFunctionUsage::Default)
		{
			bShowParent = true;
		}
		if (bShowParent)
		{
			TSharedRef<IPropertyHandle> ParentPropertyHandle = DetailLayout.GetProperty("Parent");
			IDetailPropertyRow& ParentPropertyRow = DefaultCategory.AddProperty(ParentPropertyHandle);
			ParentPropertyHandle->MarkResetToDefaultCustomized();

			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			FDetailWidgetRow Row;

			ParentPropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

			ParentPropertyHandle->ClearResetToDefaultCustomized();

			const bool bShowChildren = true;
			ParentPropertyRow.CustomWidget(bShowChildren)
				.NameContent()
				.MinDesiredWidth(Row.NameWidget.MinWidth)
				.MaxDesiredWidth(Row.NameWidget.MaxWidth)
				[
					NameWidget.ToSharedRef()
				]
			.ValueContent()
				.MinDesiredWidth(Row.ValueWidget.MinWidth)
				.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
				[
					SNew(SObjectPropertyEntryBox)
					.ObjectPath(this, &FMaterialInstanceParameterDetails::GetFunctionParentPath)
					.AllowedClass(UMaterialFunctionInterface::StaticClass())
					.ThumbnailPool(DetailLayout.GetThumbnailPool())
					.AllowClear(true)
					.OnObjectChanged(this, &FMaterialInstanceParameterDetails::OnAssetChanged, ParentPropertyHandle)
					.OnShouldSetAsset(this, &FMaterialInstanceParameterDetails::OnShouldSetAsset)
					.NewAssetFactories(TArray<UFactory*>())
				];

			ValueWidget.Reset();


		}
		else
		{
			DetailLayout.HideProperty("Parent");
		}

		DetailLayout.HideProperty("PostProcessOverrides");
		DetailLayout.HideProperty("PhysMaterial");
		DetailLayout.HideProperty("LightmassSettings");
		DetailLayout.HideProperty("bUseOldStyleMICEditorGroups");
		DetailLayout.HideProperty("ParameterGroups");
		DetailLayout.HideProperty("RefractionDepthBias");
		DetailLayout.HideProperty("bOverrideSubsurfaceProfile");
		DetailLayout.HideProperty("SubsurfaceProfile");
		DetailLayout.HideProperty("bOverrideSpecularProfile");
		DetailLayout.HideProperty("SpecularProfile");
		DetailLayout.HideProperty("bOverrideToonProfile");
		DetailLayout.HideProperty("ToonProfile");
		DetailLayout.HideProperty("BasePropertyOverrides");
	}
	else
	{
		DetailLayout.HideProperty("PostProcessOverrides");
		CreatePostProcessOverrideWidgets(DetailLayout);

		// Customize Parent property so we can check for recursively set parents
		TSharedRef<IPropertyHandle> ParentPropertyHandle = DetailLayout.GetProperty("Parent");
		IDetailPropertyRow& ParentPropertyRow = DefaultCategory.AddProperty(ParentPropertyHandle);

		TSharedRef<SObjectPropertyEntryBox> ParentValueWidget = SNew(SObjectPropertyEntryBox)
			.PropertyHandle(ParentPropertyHandle)
			.AllowedClass(UMaterialInterface::StaticClass())
			.ThumbnailPool(DetailLayout.GetThumbnailPool())
			.AllowClear(true)
			.OnShouldSetAsset(this, &FMaterialInstanceParameterDetails::OnShouldSetAsset);

		float ParentValueMinDesiredWidth, ParentValueMaxDesiredWidth;
		ParentValueWidget->GetDesiredWidth(ParentValueMinDesiredWidth, ParentValueMaxDesiredWidth);

		const bool bShowChildren = true;
		ParentPropertyRow.CustomWidget(bShowChildren)
			.NameContent()
			[
				ParentPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(ParentValueMinDesiredWidth)
			.MaxDesiredWidth(ParentValueMaxDesiredWidth)
			[
				ParentValueWidget
			];

		{
			DetailLayout.HideProperty("PhysMaterial");
			DetailLayout.HideProperty("bOverridePhysMaterial");

			// Set up the override logic for PhysMaterial
			{
				TAttribute<bool> IsPhysMatOverrideEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this]() { return (bool)MaterialEditorInstance->bOverridePhysMaterial; }));
				TSharedRef<IPropertyHandle> PhysMatPropHandle = DetailLayout.GetProperty("PhysMaterial");
				IDetailPropertyRow& PhysMatPropertyRow = DefaultCategory.AddProperty(PhysMatPropHandle);

				FResetToDefaultHandler ResetPhysMatPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
					if (MaterialEditorInstance->Parent)
					{
						InHandle->SetValue(MaterialEditorInstance->Parent->GetPhysicalMaterial());
					}
				});

				FResetToDefaultOverride ResetPhysMatPropertyOverride = FResetToDefaultOverride::Create(ResetPhysMatPropertyHandler);

				PhysMatPropertyRow
					.EditCondition(IsPhysMatOverrideEnabled,
						FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
							MaterialEditorInstance->bOverridePhysMaterial = (uint32)NewValue;
							MaterialEditorInstance->PostEditChange();
							FEditorSupportDelegates::RedrawAllViewports.Broadcast();
							}))
					.OverrideResetToDefault(ResetPhysMatPropertyOverride);
			}
		}

		// Add/hide other properties
		DetailLayout.HideProperty("LightmassSettings");
		CreateLightmassOverrideWidgets(DetailLayout);
		DetailLayout.HideProperty("bUseOldStyleMICEditorGroups");
		DetailLayout.HideProperty("ParameterGroups");

		{
			FIsResetToDefaultVisible IsRefractionDepthBiasPropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
				float BiasValue;
				float ParentBiasValue;
				return MaterialEditorInstance->SourceInstance->GetRefractionSettings(BiasValue) 
					&& MaterialEditorInstance->Parent->GetRefractionSettings(ParentBiasValue)
					&& BiasValue != ParentBiasValue;
			});
			FResetToDefaultHandler ResetRefractionDepthBiasPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
				MaterialEditorInstance->Parent->GetRefractionSettings(MaterialEditorInstance->RefractionDepthBias);
			});
			FResetToDefaultOverride ResetRefractionDepthBiasPropertyOverride = FResetToDefaultOverride::Create(IsRefractionDepthBiasPropertyResetVisible, ResetRefractionDepthBiasPropertyHandler);
			IDetailPropertyRow& PropertyRow = DefaultCategory.AddProperty("RefractionDepthBias");
			PropertyRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::ShouldShowMaterialRefractionSettings)));
			PropertyRow.OverrideResetToDefault(ResetRefractionDepthBiasPropertyOverride);
		}

		{
			// Add the material property override group
			static FName GroupName(TEXT("MaterialPropertyOverrideGroup"));
			IDetailGroup& MaterialPropertyOverrideGroup = DefaultCategory.AddGroup(GroupName, LOCTEXT("MaterialPropertyOverrideGroup", "Material Property Overrides"), false, false);
			
			// Hide the originals, these will be recreated manually
			DetailLayout.HideProperty("bOverrideSubsurfaceProfile");
			DetailLayout.HideProperty("SubsurfaceProfile");
			DetailLayout.HideProperty("bOverrideSpecularProfile");
			DetailLayout.HideProperty("SpecularProfile");
			DetailLayout.HideProperty("bOverrideToonProfile");
			DetailLayout.HideProperty("ToonProfile");
			DetailLayout.HideProperty("BasePropertyOverrides");

			// Set up the override logic for the subsurface profile
			{
				TAttribute<bool> IsParamEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this](){ return (bool)MaterialEditorInstance->bOverrideSubsurfaceProfile; }));

				IDetailPropertyRow& PropertyRow = MaterialPropertyOverrideGroup.AddPropertyRow(DetailLayout.GetProperty("SubsurfaceProfile"));
				PropertyRow
					.EditCondition(IsParamEnabled, 
						FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
							MaterialEditorInstance->bOverrideSubsurfaceProfile = (uint32)NewValue;
							MaterialEditorInstance->PostEditChange();
							FEditorSupportDelegates::RedrawAllViewports.Broadcast();
					}))
					.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::ShouldShowSubsurfaceProfile)));
			}

			// Set up the override logic for the specular and toon profiles
			if (Substrate::IsSubstrateEnabled())
			{
				TAttribute<bool> IsSpecProfileParamEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this](){ return (bool)MaterialEditorInstance->bOverrideSpecularProfile; }));

				IDetailPropertyRow& SpecProfilePropertyRow = MaterialPropertyOverrideGroup.AddPropertyRow(DetailLayout.GetProperty("SpecularProfile"));
				SpecProfilePropertyRow
				.EditCondition(IsSpecProfileParamEnabled,
								FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
									MaterialEditorInstance->bOverrideSpecularProfile = (uint32)NewValue;
									MaterialEditorInstance->PostEditChange();
									FEditorSupportDelegates::RedrawAllViewports.Broadcast();
				}))
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::ShouldShowSpecularProfile)));


				TAttribute<bool> IsToonProfileParamEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this](){ return (bool)MaterialEditorInstance->bOverrideToonProfile; }));

				IDetailPropertyRow& ToonProfilePropertyRow = MaterialPropertyOverrideGroup.AddPropertyRow(DetailLayout.GetProperty("ToonProfile"));
				ToonProfilePropertyRow
				.EditCondition(IsToonProfileParamEnabled,
								FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
									MaterialEditorInstance->bOverrideToonProfile = (uint32)NewValue;
									MaterialEditorInstance->PostEditChange();
									FEditorSupportDelegates::RedrawAllViewports.Broadcast();
				}))
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::ShouldShowToonProfile)));
			}
			
			// Append the base property overrides to the Material Property Override Group
			CreateBasePropertyOverrideWidgets(DetailLayout, MaterialPropertyOverrideGroup);

			// Append the nanite material override.
			MaterialPropertyOverrideGroup.AddPropertyRow(DetailLayout.GetProperty("NaniteOverrideMaterial"));

			// Add the usage flag overrides
			CreateUsageFlagOverrideWidgets(DetailLayout, DefaultCategory);
		}

		{
			// Refresh UI when enumeration objects change.
			IDetailPropertyRow& PropertyRow = DefaultCategory.AddProperty("EnumerationObjects");
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow.GetPropertyHandle();
			if (PropertyHandle->IsValidHandle())
			{
				// Needs to happen on property (add to array) and child (array entries).
				auto ForceRefresh = FSimpleDelegate::CreateLambda([&DetailLayout]() { DetailLayout.ForceRefreshDetails(); });
				PropertyHandle->SetOnPropertyValueChanged(ForceRefresh);
				PropertyHandle->SetOnChildPropertyValueChanged(ForceRefresh);
			}
		}
	}

	// Add the preview mesh property directly from the material instance 
	FName PreviewingCategoryName = TEXT("Previewing");
	IDetailCategoryBuilder& PreviewingCategory = DetailLayout.EditCategory(PreviewingCategoryName, LOCTEXT("MICPreviewingCategoryTitle", "Previewing"));

	TArray<UObject*> ExternalObjects;
	ExternalObjects.Add(MaterialEditorInstance->SourceInstance);

	PreviewingCategory.AddExternalObjectProperty(ExternalObjects, TEXT("PreviewMesh"));

	DefaultCategory.AddExternalObjectProperty(ExternalObjects, TEXT("AssetUserData"), EPropertyLocation::Advanced);
}

void FMaterialInstanceParameterDetails::CreateGroupsWidget(TSharedRef<IPropertyHandle> ParameterGroupsProperty, IDetailCategoryBuilder& GroupsCategory)
{
	bool bShowSaveButtons = false;
	check(MaterialEditorInstance);

	struct FGroupData
	{
		IDetailGroup* Group = nullptr;
		TArray<int32> AffectedGroupsIndices;
	};
	TMap<FName, FGroupData> GroupNameToData;
	
	for (int32 GroupIdx = 0; GroupIdx < MaterialEditorInstance->ParameterGroups.Num(); ++GroupIdx)
	{
		FEditorParameterGroup& ParameterGroup = MaterialEditorInstance->ParameterGroups[GroupIdx];
		if (ParameterGroup.GroupAssociation != EMaterialParameterAssociation::GlobalParameter ||
			ParameterGroup.GroupName == FMaterialPropertyHelpers::LayerParamName)
		{
			continue;
		}

		bShowSaveButtons = true;
		bool bCreateGroup = false;
		for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num() && !bCreateGroup; ++ParamIdx)
		{
			UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
			const bool bIsVisible = MaterialEditorInstance->VisibleExpressions.Contains(Parameter->ParameterInfo) && FMaterialPropertyHelpers::ShouldCreatePropertyRowForParameter(Parameter);
			bCreateGroup = bIsVisible && (!MaterialEditorInstance->bShowOnlyOverrides || FMaterialPropertyHelpers::IsOverriddenExpression(Parameter));
		}

		if (!bCreateGroup)
		{
			continue;
		}

		IDetailGroup* DetailGroup = nullptr;
		TArray<FString> GroupParts;
		ParameterGroup.GroupName.ToString().ParseIntoArray(GroupParts, TEXT("|"));

		FString Path;
		for (FString Part : GroupParts)
		{
			Part.TrimStartAndEndInline();

			if (!Path.IsEmpty())
			{
				Path += "|";
			}
			Path += Part;

			FGroupData& GroupData = GroupNameToData.FindOrAdd(FName(Path));
			if (!GroupData.Group)
			{
				if (!DetailGroup)
				{
					GroupData.Group = &GroupsCategory.AddGroup(FName(Part), FText::FromString(FName::NameToDisplayString(Part, false)), false, true);
				}
				else
				{
					GroupData.Group = &DetailGroup->AddGroup(FName(Part), FText::FromString(FName::NameToDisplayString(Part, false)), true);
				}
			}

			DetailGroup = GroupData.Group;
			GroupData.AffectedGroupsIndices.Add(GroupIdx);
		}

		check(DetailGroup);
		CreateSingleGroupWidget(ParameterGroup, ParameterGroupsProperty->GetChildHandle(GroupIdx), *DetailGroup);
	}

	for (const TPair<FName, FGroupData>& It : GroupNameToData)
	{
		FUIAction CopyAction(
			FExecuteAction::CreateSP(this, &FMaterialInstanceParameterDetails::OnCopyParameterValues, It.Value.AffectedGroupsIndices),
			FCanExecuteAction::CreateSP(this, &FMaterialInstanceParameterDetails::CanCopyParameterValues, It.Value.AffectedGroupsIndices));
		FUIAction PasteAction(
			FExecuteAction::CreateSP(this, &FMaterialInstanceParameterDetails::OnPasteParameterValues, It.Value.AffectedGroupsIndices),
			FCanExecuteAction::CreateSP(this, &FMaterialInstanceParameterDetails::CanPasteParameterValues, It.Value.AffectedGroupsIndices));
		FDetailWidgetRow& HeaderRow = It.Value.Group->HeaderRow()
			.CopyAction(CopyAction)
			.PasteAction(PasteAction)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(FText::FromName(It.Value.Group->GetGroupName()))
			];

		HeaderRow.AddCustomContextMenuAction(FUIAction(
			FExecuteAction::CreateSP(this, &FMaterialInstanceParameterDetails::EnableGroupParameters, It.Value.AffectedGroupsIndices, true)),
			LOCTEXT("ToggleParametersEnable", "Enable All Parameters"),
			LOCTEXT("ToggleParametersEnableTooltip", "Enable All Parameters in group"),
			FSlateIcon());

		HeaderRow.AddCustomContextMenuAction(FUIAction(
			FExecuteAction::CreateSP(this, &FMaterialInstanceParameterDetails::EnableGroupParameters, It.Value.AffectedGroupsIndices, false)),
			LOCTEXT("ToggleParametersDisable", "Disable All Parameters"),
			LOCTEXT("ToggleParametersDisableTooltip", "Disable All Parameters in group"),
			FSlateIcon());
	}




	if (bShowSaveButtons)
	{
		FDetailWidgetRow& SaveInstanceRow = GroupsCategory.AddCustomRow(LOCTEXT("SaveInstances", "Save Instances"));
		FOnClicked OnChildButtonClicked;
		FOnClicked OnSiblingButtonClicked;
		UMaterialInterface* LocalSourceInstance = MaterialEditorInstance->SourceInstance;
		UObject* LocalEditorInstance = MaterialEditorInstance;
		if (!MaterialEditorInstance->bIsFunctionPreviewMaterial)
		{
			OnChildButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewMaterialInstance, LocalSourceInstance, LocalEditorInstance);
			OnSiblingButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewMaterialInstance, ToRawPtr(MaterialEditorInstance->SourceInstance->Parent), LocalEditorInstance);
		}
		else
		{
			OnChildButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewFunctionInstance, 
				ImplicitConv<UMaterialFunctionInterface*>(MaterialEditorInstance->SourceFunction), LocalSourceInstance, LocalEditorInstance);
			OnSiblingButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewFunctionInstance,
				ImplicitConv<UMaterialFunctionInterface*>(MaterialEditorInstance->SourceFunction->Parent), LocalSourceInstance, LocalEditorInstance);
		}
		SaveInstanceRow.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNullWidget::NullWidget
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("SaveSibling", "Save Sibling"))
					.HAlign(HAlign_Center)
					.OnClicked(OnSiblingButtonClicked)
					.ToolTipText(LOCTEXT("SaveToSiblingInstance", "Save to Sibling Instance"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("SaveChild", "Save Child"))
					.HAlign(HAlign_Center)
					.OnClicked(OnChildButtonClicked)
					.ToolTipText(LOCTEXT("SaveToChildInstance", "Save to Child Instance"))
				]
			];
	}
}

void FMaterialInstanceParameterDetails::CreateParameterCollectionOverrideCategory(TSharedRef<IPropertyHandle> ParameterGroupsProperty, IDetailLayoutBuilder& DetailLayout)
{
	if (!MaterialEditorInstance || !MaterialEditorInstance->SourceInstance)
	{
		return;
	}

	struct FParameterEntry
	{
		UDEditorParameterCollectionParameterValue* ParameterValue;
		TSharedPtr<IPropertyHandle> ParameterValueProperty;
	};

	TMap<UMaterialParameterCollection*, TArray<FParameterEntry>> ParameterCollectionSharedOverrideMap;

	for (int32 GroupIdx = 0; GroupIdx < MaterialEditorInstance->ParameterGroups.Num(); ++GroupIdx)
	{
		TSharedPtr<IPropertyHandle> ParameterGroupProperty = ParameterGroupsProperty->GetChildHandle(GroupIdx)->GetChildHandle("Parameters");

		FEditorParameterGroup& ParameterGroup = MaterialEditorInstance->ParameterGroups[GroupIdx];
		for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ++ParamIdx)
		{
			UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
			if (UDEditorParameterCollectionParameterValue* ParameterCollectionParam = Cast<UDEditorParameterCollectionParameterValue>(Parameter))
			{
				// Map each parameter collection to its referencing parameters.
				for (const FMaterialParameterCollectionInfo& CollectionInfo : MaterialEditorInstance->SourceInstance->GetCachedExpressionData().ParameterCollectionInfos)
				{
					FMaterialParameterMetadata Result;
					if (ParameterCollectionParam->GetValue(Result) && ensure(Result.Value.Type == EMaterialParameterType::ParameterCollection) &&
						Result.Value.ParameterCollection && Result.Value.ParameterCollection->IsInstanceOf(CollectionInfo.StateId))
					{
						TSharedPtr<IPropertyHandle> ParameterValueProperty = ParameterGroupProperty->GetChildHandle(ParamIdx)->GetChildHandle("ParameterValue");

						TArray<FParameterEntry>& ParameterIndices = ParameterCollectionSharedOverrideMap.FindOrAdd(CollectionInfo.ParameterCollection);
						ParameterIndices.Add({ ParameterCollectionParam, ParameterValueProperty });
						break;
					}
				}
			}
		}
	}

	if (ParameterCollectionSharedOverrideMap.Num() == 0)
	{
		return;
	}

	// Make a new category for MPC overrides
	FName ParamCollectionOverridesName = TEXT("ParameterCollectionOverrides");
	IDetailCategoryBuilder& DetailGroup = DetailLayout.EditCategory(ParamCollectionOverridesName, LOCTEXT("MICParamCollectionOverridesGroupTitle", "Parameter Collection Overrides"));

	FMaterialParameterPropertyRowOverrides RowOverrides;
	RowOverrides.ShowHiddenDelegate = ShowHiddenDelegate;

	for (auto& Entry : ParameterCollectionSharedOverrideMap)
	{
		UMaterialParameterCollection* ParameterCollection = Entry.Key;
		const TArray<FParameterEntry>& ParameterEntries = Entry.Value;
		int32 NumParameterEntries = ParameterEntries.Num();
		if (NumParameterEntries > 0)
		{
			const FParameterEntry& FirstParameterEntry = ParameterEntries[0];

			if (FirstParameterEntry.ParameterValueProperty)
			{
				// Gather the relevant property handles of the collection's referencing parameters.
				TArray<TSharedRef<IPropertyHandle>> ValuePropertyHandles{ FirstParameterEntry.ParameterValueProperty.ToSharedRef() };
				for (int32 ParameterIndex = 1; ParameterIndex < NumParameterEntries; ++ParameterIndex)
				{
					const FParameterEntry& ParameterEntry = ParameterEntries[ParameterIndex];
					if (ParameterEntry.ParameterValueProperty)
					{
						ValuePropertyHandles.Add(ParameterEntry.ParameterValueProperty.ToSharedRef());
					}
				}

				// Create a parameter value widget for the first parameter.
				IDetailPropertyRow& PropertyRow = DetailGroup.AddProperty(FirstParameterEntry.ParameterValueProperty.ToSharedRef());

				FMaterialPropertyHelpers::ConfigurePropertyRowForParameter(PropertyRow, FirstParameterEntry.ParameterValue, FirstParameterEntry.ParameterValueProperty, MaterialEditorInstance, RowOverrides);
				FMaterialPropertyHelpers::SetPropertyRowParameterCollectionParameterWidget(PropertyRow, FirstParameterEntry.ParameterValue, FirstParameterEntry.ParameterValueProperty, ParameterCollection, MoveTemp(ValuePropertyHandles));
			}
		}
	}
}

void FMaterialInstanceParameterDetails::EnableGroupParameters(const TArray<int32> ParameterGroupIndices, const bool bShouldEnable)
{
	if (!MaterialEditorInstance)
	{
		return;
	}

	// loop through each parameter in the group and toggle to enable/disable them all
	for (const int32 ParameterGroupIndex : ParameterGroupIndices)
	{
		if (!MaterialEditorInstance->ParameterGroups.IsValidIndex(ParameterGroupIndex))
		{
			continue;
		}

		FEditorParameterGroup& ParameterGroup = MaterialEditorInstance->ParameterGroups[ParameterGroupIndex];
		for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ++ParamIdx)
		{
			UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
			Parameter->bOverride = bShouldEnable;
		}
	}
}

void FMaterialInstanceParameterDetails::CreateSingleGroupWidget(FEditorParameterGroup& ParameterGroup, TSharedPtr<IPropertyHandle> ParameterGroupProperty, IDetailGroup& DetailGroup, int32 GroupIndex /*= -1*/, bool bForceShowParam /*= false*/)
{
	TSharedPtr<IPropertyHandle> ParametersArrayProperty = ParameterGroupProperty->GetChildHandle("Parameters");

	FMaterialParameterPropertyRowOverrides RowOverrides;
	RowOverrides.ShowHiddenDelegate = ShowHiddenDelegate;

	// Create a custom widget for each parameter in the group
	for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ++ParamIdx)
	{
		TSharedPtr<IPropertyHandle> ParameterProperty = ParametersArrayProperty->GetChildHandle(ParamIdx);
		TSharedPtr<IPropertyHandle> ParameterValueProperty = ParameterProperty.IsValid() ? ParameterProperty->GetChildHandle("ParameterValue") : nullptr;

		UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
		if (ParameterProperty.IsValid() && ParameterValueProperty.IsValid() && ParameterValueProperty->IsValidHandle() &&
			(GroupIndex == INDEX_NONE || Parameter->ParameterInfo.Index == GroupIndex))
		{
			if (Parameter->ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter || bForceShowParam)
			{
				if (!FMaterialPropertyHelpers::ShouldCreatePropertyRowForParameter(Parameter))
				{
					continue;
				}

				IDetailPropertyRow& PropertyRow = DetailGroup.AddPropertyRow(ParameterValueProperty.ToSharedRef());

				FMaterialPropertyHelpers::ConfigurePropertyRowForParameter(PropertyRow, Parameter, ParameterValueProperty, MaterialEditorInstance, RowOverrides);
				FMaterialPropertyHelpers::SetPropertyRowParameterWidget(PropertyRow, Parameter, ParameterValueProperty, MaterialEditorInstance);
			}
		}
	}
}

bool FMaterialInstanceParameterDetails::OnShouldSetAsset(const FAssetData& AssetData) const
{
	if (MaterialEditorInstance->bIsFunctionPreviewMaterial)
	{
		if (MaterialEditorInstance->SourceFunction->GetMaterialFunctionUsage() == EMaterialFunctionUsage::Default)
		{
			return false;
		}
		else
		{
			UMaterialFunctionInstance* FunctionInstance = Cast<UMaterialFunctionInstance>(AssetData.GetAsset());
			if (FunctionInstance != nullptr)
			{
				bool bIsChild = FunctionInstance->IsDependent(MaterialEditorInstance->SourceFunction);
				if (bIsChild)
				{
					FMessageDialog::Open(
						EAppMsgType::Ok,
						FText::Format(LOCTEXT("CannotSetExistingChildFunctionAsParent", "Cannot set {0} as a parent as it is already a child of this material function instance."), FText::FromName(AssetData.AssetName)));
				}
				return !bIsChild;
			}
		}
	}

	UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(AssetData.GetAsset());

	if (MaterialInstance != nullptr)
	{
		bool bIsChild = MaterialInstance->IsChildOf(MaterialEditorInstance->SourceInstance);
		if (bIsChild)
		{
			FMessageDialog::Open(
				EAppMsgType::Ok,
				FText::Format(LOCTEXT("CannotSetExistingChildAsParent", "Cannot set {0} as a parent as it is already a child of this material instance."), FText::FromName(AssetData.AssetName)));
		}

		if (bIsChild)
		{
			return false;
		}
	}

	return true;
}

void FMaterialInstanceParameterDetails::OnAssetChanged(const FAssetData & InAssetData, TSharedRef<IPropertyHandle> InHandle)
{
	if (MaterialEditorInstance->bIsFunctionPreviewMaterial &&
		MaterialEditorInstance->SourceFunction->GetMaterialFunctionUsage() != EMaterialFunctionUsage::Default)
	{
		UMaterialFunctionInterface* NewParent = Cast<UMaterialFunctionInterface>(InAssetData.GetAsset());
		if (NewParent != nullptr)
		{
			MaterialEditorInstance->SourceFunction->SetParent(NewParent);
			FPropertyChangedEvent ParentChanged = FPropertyChangedEvent(InHandle->GetProperty());
			MaterialEditorInstance->PostEditChangeProperty(ParentChanged);
		}
	}
}

EVisibility FMaterialInstanceParameterDetails::ShouldShowMaterialRefractionSettings() const
{
	return (MaterialEditorInstance->SourceInstance->GetMaterial()->bUsesDistortion && IsTranslucentBlendMode(*MaterialEditorInstance->SourceInstance)) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMaterialInstanceParameterDetails::ShouldShowSubsurfaceProfile() const
{
	FMaterialShadingModelField ShadingModels = MaterialEditorInstance->SourceInstance->GetShadingModels();

	return UseSubsurfaceProfile(ShadingModels) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMaterialInstanceParameterDetails::ShouldShowSpecularProfile() const
{
	const bool bHasProfiles = Substrate::IsSubstrateEnabled() && (MaterialEditorInstance->SourceInstance->NumSpecularProfile_Internal() > 0 || MaterialEditorInstance->SourceInstance->GetSpecularProfileOverride_Internal() != nullptr);
	return bHasProfiles ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMaterialInstanceParameterDetails::ShouldShowToonProfile() const
{
	const bool bHasProfiles = Substrate::IsSubstrateEnabled() && (MaterialEditorInstance->SourceInstance->NumToonProfile_Internal() > 0 || MaterialEditorInstance->SourceInstance->GetToonProfileOverride_Internal() != nullptr);
	return bHasProfiles ? EVisibility::Visible : EVisibility::Collapsed;
}

void FMaterialInstanceParameterDetails::OnCopyParameterValues(const TArray<int32> ParameterGroupIndices)
{
	if (!MaterialEditorInstance)
	{
		return;
	}

	TStringBuilder<4096> CombinedValue;
	for (const int32 ParameterGroupIndex : ParameterGroupIndices)
	{
		if (!MaterialEditorInstance->ParameterGroups.IsValidIndex(ParameterGroupIndex))
		{
			continue;
		}

		FEditorParameterGroup& ParameterGroup = MaterialEditorInstance->ParameterGroups[ParameterGroupIndex];

		const int32 NumParams = ParameterGroup.Parameters.Num();
		for (int32 ParamIdx = 0; ParamIdx < NumParams; ++ParamIdx)
		{
			UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
			const FName ParamName = Parameter->ParameterInfo.Name;

			const TCHAR* Prefix = (CombinedValue.Len() == 0) ? TEXT("") : TEXT(",");

			// Include the value in the result entry only if the parameter is overridden.
			const bool bOverride = FMaterialPropertyHelpers::IsOverriddenExpression(Parameter);
			if (bOverride)
			{
				FProperty* ParameterValueProperty = Parameter->GetClass()->FindPropertyByName("ParameterValue");
				if (ParameterValueProperty != nullptr)
				{
					FString ParameterValueString;
					if (ParameterValueProperty->ExportText_InContainer(0, ParameterValueString, Parameter, Parameter, Parameter, PPF_Copy))
					{
						CombinedValue.Appendf(TEXT("%s%s.Override=True,%s.Value=\"%s\""),
							Prefix,
							*(ParamName.ToString()),
							*(ParamName.ToString()),
							*(ParameterValueString.ReplaceCharWithEscapedChar()));
					}
				}
			}
			else
			{
				CombinedValue.Appendf(TEXT("%s%s.Override=False"), Prefix, *(ParamName.ToString()));
			}
		}
	}

	if (CombinedValue.Len())
	{
		// Copy.
		FPlatformApplicationMisc::ClipboardCopy(*CombinedValue);
	}
}

bool FMaterialInstanceParameterDetails::CanCopyParameterValues(const TArray<int32> ParameterGroupIndices)
{
	if (!MaterialEditorInstance)
	{
		return false;
	}

	for (const int32 ParameterGroupIndex : ParameterGroupIndices)
	{
		if (MaterialEditorInstance->ParameterGroups.IsValidIndex(ParameterGroupIndex) &&
			MaterialEditorInstance->ParameterGroups[ParameterGroupIndex].Parameters.Num() > 0)
		{
			return true;
		}
	}

	return false;
}

void FMaterialInstanceParameterDetails::OnPasteParameterValues(const TArray<int32> ParameterGroupIndices)
{
	if (!MaterialEditorInstance)
	{
		return;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (ClipboardContent.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("PasteMaterialInstanceParameters", "Paste Material Instance Parameters"));

	bool bIsValidTransaction = false;

	for (const int32 ParameterGroupIndex : ParameterGroupIndices)
	{
		if (!MaterialEditorInstance->ParameterGroups.IsValidIndex(ParameterGroupIndex))
		{
			continue;
		}

		if (!bIsValidTransaction)
		{
			bIsValidTransaction = true;
			MaterialEditorInstance->Modify();
		}

		FEditorParameterGroup& ParameterGroup = MaterialEditorInstance->ParameterGroups[ParameterGroupIndex];
		for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ++ParamIdx)
		{
			UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
			Parameter->Modify();

			const FName ParamName = Parameter->ParameterInfo.Name;

			const FString OverrideKey = FString::Printf(TEXT("%s.Override="), *(ParamName.ToString()));
			bool bParsedOverride = false;
			if (FParse::Bool(*ClipboardContent, *OverrideKey, bParsedOverride))
			{
				Parameter->bOverride = bParsedOverride;
				if (bParsedOverride)
				{
					// Paste value.
					const FString ValueKey = FString::Printf(TEXT("%s.Value="), *(ParamName.ToString()));
					FString ParsedValueString;
					if (FParse::Value(*ClipboardContent, *ValueKey, ParsedValueString))
					{
						ParsedValueString = ParsedValueString.ReplaceEscapedCharWithChar();
						FProperty* ParameterValueProperty = Parameter->GetClass()->FindPropertyByName("ParameterValue");
						if (ParameterValueProperty != nullptr)
						{
							ParameterValueProperty->ImportText_InContainer(*ParsedValueString, Parameter, Parameter, PPF_Copy);
						}
					}
				}
			}
		}

	}

	if (!bIsValidTransaction)
	{
		Transaction.Cancel();
		return;
	}

	MaterialEditorInstance->PostEditChange();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

bool FMaterialInstanceParameterDetails::CanPasteParameterValues(const TArray<int32> ParameterGroupIndices)
{
	// First check the same criteria as copying.
	if (!CanCopyParameterValues(ParameterGroupIndices))
	{
		return false;
	}

	// Now see if there's anything to paste from the clipboard.
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	return !ClipboardContent.IsEmpty();
}

void FMaterialInstanceParameterDetails::CreateLightmassOverrideWidgets(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& DetailCategory = DetailLayout.EditCategory(NAME_None);

	static FName GroupName(TEXT("LightmassSettings"));
	IDetailGroup& LightmassSettingsGroup = DetailCategory.AddGroup(GroupName, LOCTEXT("LightmassSettingsGroup", "Lightmass Settings"), false, false);

	TAttribute<bool> IsOverrideCastShadowAsMaskedEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return (bool)MaterialEditorInstance->LightmassSettings.CastShadowAsMasked.bOverride; }));
	TAttribute<bool> IsOverrideEmissiveBoostEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return (bool)MaterialEditorInstance->LightmassSettings.EmissiveBoost.bOverride; }));
	TAttribute<bool> IsOverrideDiffuseBoostEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return (bool)MaterialEditorInstance->LightmassSettings.DiffuseBoost.bOverride; }));
	TAttribute<bool> IsOverrideExportResolutionScaleEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return (bool)MaterialEditorInstance->LightmassSettings.ExportResolutionScale.bOverride; }));
	
	TSharedRef<IPropertyHandle> LightmassSettings = DetailLayout.GetProperty("LightmassSettings");
	TSharedPtr<IPropertyHandle> CastShadowAsMaskedProperty = LightmassSettings->GetChildHandle("CastShadowAsMasked");
	TSharedPtr<IPropertyHandle> EmissiveBoostProperty = LightmassSettings->GetChildHandle("EmissiveBoost");
	TSharedPtr<IPropertyHandle> DiffuseBoostProperty = LightmassSettings->GetChildHandle("DiffuseBoost");
	TSharedPtr<IPropertyHandle> ExportResolutionScaleProperty = LightmassSettings->GetChildHandle("ExportResolutionScale");

	
	FIsResetToDefaultVisible IsCastShadowAsMaskedPropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		return MaterialEditorInstance->Parent != nullptr ? MaterialEditorInstance->LightmassSettings.CastShadowAsMasked.ParameterValue != MaterialEditorInstance->Parent->GetCastShadowAsMasked() : false;
	});
	FResetToDefaultHandler ResetCastShadowAsMaskedPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		if (MaterialEditorInstance->Parent != nullptr)
		{
			MaterialEditorInstance->LightmassSettings.CastShadowAsMasked.ParameterValue = MaterialEditorInstance->Parent->GetCastShadowAsMasked();
		}
	});
	FResetToDefaultOverride ResetCastShadowAsMaskedPropertyOverride = FResetToDefaultOverride::Create(IsCastShadowAsMaskedPropertyResetVisible, ResetCastShadowAsMaskedPropertyHandler);
	IDetailPropertyRow& CastShadowAsMaskedPropertyRow = LightmassSettingsGroup.AddPropertyRow(CastShadowAsMaskedProperty->GetChildHandle(0).ToSharedRef());
	CastShadowAsMaskedPropertyRow
		.DisplayName(CastShadowAsMaskedProperty->GetPropertyDisplayName())
		.ToolTip(CastShadowAsMaskedProperty->GetToolTipText())
		.EditCondition(IsOverrideCastShadowAsMaskedEnabled, FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
		MaterialEditorInstance->LightmassSettings.CastShadowAsMasked.bOverride = (uint32)NewValue;
		MaterialEditorInstance->PostEditChange();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible, IsOverrideCastShadowAsMaskedEnabled)))
		.OverrideResetToDefault(ResetCastShadowAsMaskedPropertyOverride);

	FIsResetToDefaultVisible IsEmissiveBoostPropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		return MaterialEditorInstance->Parent != nullptr ? MaterialEditorInstance->LightmassSettings.EmissiveBoost.ParameterValue != MaterialEditorInstance->Parent->GetEmissiveBoost() : false;
	});
	FResetToDefaultHandler ResetEmissiveBoostPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		if (MaterialEditorInstance->Parent != nullptr)
		{
			MaterialEditorInstance->LightmassSettings.EmissiveBoost.ParameterValue = MaterialEditorInstance->Parent->GetEmissiveBoost();
		}
	});
	FResetToDefaultOverride ResetEmissiveBoostPropertyOverride = FResetToDefaultOverride::Create(IsEmissiveBoostPropertyResetVisible, ResetEmissiveBoostPropertyHandler);
	IDetailPropertyRow& EmissiveBoostPropertyRow = LightmassSettingsGroup.AddPropertyRow(EmissiveBoostProperty->GetChildHandle(0).ToSharedRef());
	EmissiveBoostPropertyRow
		.DisplayName(EmissiveBoostProperty->GetPropertyDisplayName())
		.ToolTip(EmissiveBoostProperty->GetToolTipText())
		.EditCondition(IsOverrideEmissiveBoostEnabled, FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
		MaterialEditorInstance->LightmassSettings.EmissiveBoost.bOverride = (uint32)NewValue;
		MaterialEditorInstance->PostEditChange();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible, IsOverrideEmissiveBoostEnabled)))
		.OverrideResetToDefault(ResetEmissiveBoostPropertyOverride);

	FIsResetToDefaultVisible IsDiffuseBoostPropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		return MaterialEditorInstance->Parent != nullptr ? MaterialEditorInstance->LightmassSettings.DiffuseBoost.ParameterValue != MaterialEditorInstance->Parent->GetDiffuseBoost() : false;
	});
	FResetToDefaultHandler ResetDiffuseBoostPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		if (MaterialEditorInstance->Parent != nullptr)
		{
			MaterialEditorInstance->LightmassSettings.DiffuseBoost.ParameterValue = MaterialEditorInstance->Parent->GetDiffuseBoost();
		}
	});
	FResetToDefaultOverride ResetDiffuseBoostPropertyOverride = FResetToDefaultOverride::Create(IsDiffuseBoostPropertyResetVisible, ResetDiffuseBoostPropertyHandler);
	IDetailPropertyRow& DiffuseBoostPropertyRow = LightmassSettingsGroup.AddPropertyRow(DiffuseBoostProperty->GetChildHandle(0).ToSharedRef());
	DiffuseBoostPropertyRow
		.DisplayName(DiffuseBoostProperty->GetPropertyDisplayName())
		.ToolTip(DiffuseBoostProperty->GetToolTipText())
		.EditCondition(IsOverrideDiffuseBoostEnabled, FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
		MaterialEditorInstance->LightmassSettings.DiffuseBoost.bOverride = (uint32)NewValue;
		MaterialEditorInstance->PostEditChange();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible, IsOverrideDiffuseBoostEnabled)))
		.OverrideResetToDefault(ResetDiffuseBoostPropertyOverride);

	FIsResetToDefaultVisible IsExportResolutionScalePropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		return MaterialEditorInstance->Parent != nullptr ? MaterialEditorInstance->LightmassSettings.ExportResolutionScale.ParameterValue != MaterialEditorInstance->Parent->GetDiffuseBoost() : false;
	});
	FResetToDefaultHandler ResetExportResolutionScalePropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
		if (MaterialEditorInstance->Parent != nullptr)
		{
			MaterialEditorInstance->LightmassSettings.ExportResolutionScale.ParameterValue = MaterialEditorInstance->Parent->GetDiffuseBoost();
		}
	});
	FResetToDefaultOverride ResetExportResolutionScalePropertyOverride = FResetToDefaultOverride::Create(IsExportResolutionScalePropertyResetVisible, ResetExportResolutionScalePropertyHandler);
	IDetailPropertyRow& ExportResolutionScalePropertyRow = LightmassSettingsGroup.AddPropertyRow(ExportResolutionScaleProperty->GetChildHandle(0).ToSharedRef());
	ExportResolutionScalePropertyRow
		.DisplayName(ExportResolutionScaleProperty->GetPropertyDisplayName())
		.ToolTip(ExportResolutionScaleProperty->GetToolTipText())
		.EditCondition(IsOverrideExportResolutionScaleEnabled, FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
		MaterialEditorInstance->LightmassSettings.ExportResolutionScale.bOverride = (uint32)NewValue;
		MaterialEditorInstance->PostEditChange();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible, IsOverrideExportResolutionScaleEnabled)))
		.OverrideResetToDefault(ResetExportResolutionScalePropertyOverride);
}

void FMaterialInstanceParameterDetails::CreatePostProcessOverrideWidgets(IDetailLayoutBuilder& DetailLayout)
{
	if (MaterialEditorInstance->PostProcessOverrides.bIsOverrideable)
	{
		FName PostProcessCategoryName = TEXT("PostProcessOverrides");
		IDetailCategoryBuilder& PostProcessCategory = DetailLayout.EditCategory(PostProcessCategoryName, LOCTEXT("MICPostProcessOverridesTitle", "Post Process Overrides"));
		PostProcessCategory.InitiallyCollapsed(true);

		TAttribute<bool> IsOverrideLocationEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return (bool)MaterialEditorInstance->PostProcessOverrides.bOverrideBlendableLocation; }));
		TAttribute<bool> IsOverridePriorityEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return (bool)MaterialEditorInstance->PostProcessOverrides.bOverrideBlendablePriority; }));

		TSharedRef<IPropertyHandle> PostProcessOverridesProperty = DetailLayout.GetProperty("PostProcessOverrides");
		TSharedPtr<IPropertyHandle> BlendableLocationProperty = PostProcessOverridesProperty->GetChildHandle("BlendableLocationOverride");
		TSharedPtr<IPropertyHandle> BlendablePriorityProperty = PostProcessOverridesProperty->GetChildHandle("BlendablePriorityOverride");
		TSharedPtr<IPropertyHandle> UserSceneTextureOutputProperty = PostProcessOverridesProperty->GetChildHandle("UserSceneTextureOutput");

		FIsResetToDefaultVisible IsBlendableLocationPropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
			return MaterialEditorInstance->Parent && MaterialEditorInstance->Parent->GetMaterial() ?
				MaterialEditorInstance->PostProcessOverrides.BlendableLocationOverride != MaterialEditorInstance->Parent->GetMaterial()->BlendableLocation : false;
			});
		FResetToDefaultHandler ResetBlendableLocationPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
			if (MaterialEditorInstance->Parent && MaterialEditorInstance->Parent->GetMaterial())
			{
				MaterialEditorInstance->PostProcessOverrides.BlendableLocationOverride = MaterialEditorInstance->Parent->GetMaterial()->BlendableLocation;
			}
			});
		FResetToDefaultOverride ResetBlendableLocationPropertyOverride = FResetToDefaultOverride::Create(IsBlendableLocationPropertyResetVisible, ResetBlendableLocationPropertyHandler);

		IDetailPropertyRow& BlendableLocationPropertyRow = PostProcessCategory.AddProperty(BlendableLocationProperty);
		BlendableLocationPropertyRow
			.DisplayName(BlendableLocationProperty->GetPropertyDisplayName())
			.ToolTip(BlendableLocationProperty->GetToolTipText())
			.EditCondition(IsOverrideLocationEnabled, FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
				MaterialEditorInstance->PostProcessOverrides.bOverrideBlendableLocation = NewValue;
				MaterialEditorInstance->PostEditChange();
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			}))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible, IsOverrideLocationEnabled)))
			.OverrideResetToDefault(ResetBlendableLocationPropertyOverride);

		FIsResetToDefaultVisible IsBlendablePriorityPropertyResetVisible = FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
			return MaterialEditorInstance->Parent && MaterialEditorInstance->Parent->GetMaterial() ?
				MaterialEditorInstance->PostProcessOverrides.BlendablePriorityOverride != MaterialEditorInstance->Parent->GetMaterial()->BlendablePriority : false;
			});
		FResetToDefaultHandler ResetBlendablePriorityPropertyHandler = FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InHandle) {
			if (MaterialEditorInstance->Parent && MaterialEditorInstance->Parent->GetMaterial())
			{
				MaterialEditorInstance->PostProcessOverrides.BlendablePriorityOverride = MaterialEditorInstance->Parent->GetMaterial()->BlendablePriority;
			}
			});
		FResetToDefaultOverride ResetBlendablePriorityPropertyOverride = FResetToDefaultOverride::Create(IsBlendablePriorityPropertyResetVisible, ResetBlendablePriorityPropertyHandler);

		IDetailPropertyRow& BlendablePriorityPropertyRow = PostProcessCategory.AddProperty(BlendablePriorityProperty);
		BlendablePriorityPropertyRow
			.DisplayName(BlendablePriorityProperty->GetPropertyDisplayName())
			.ToolTip(BlendablePriorityProperty->GetToolTipText())
			.EditCondition(IsOverridePriorityEnabled, FOnBooleanValueChanged::CreateLambda([this](bool NewValue) {
				MaterialEditorInstance->PostProcessOverrides.bOverrideBlendablePriority = NewValue;
				MaterialEditorInstance->PostEditChange();
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			}))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible, IsOverridePriorityEnabled)))
			.OverrideResetToDefault(ResetBlendablePriorityPropertyOverride);

		if (MaterialEditorInstance->PostProcessOverrides.UserSceneTextureInputs.Num())
		{
			static FName GroupName(TEXT("UserSceneTextures"));
			IDetailGroup& UserSceneTexturesGroup = PostProcessCategory.AddGroup(GroupName, LOCTEXT("UserSceneTextureInputsGroup", "User Scene Texture Inputs"), false, true);

			TSharedPtr<IPropertyHandle> UserSceneTexturesArrayProperty = PostProcessOverridesProperty->GetChildHandle("UserSceneTextureInputs");

			for (int32 UserSceneTextureIndex = 0; UserSceneTextureIndex < MaterialEditorInstance->PostProcessOverrides.UserSceneTextureInputs.Num(); ++UserSceneTextureIndex)
			{
				TSharedPtr<IPropertyHandle> UserSceneTextureItemProperty = UserSceneTexturesArrayProperty->GetChildHandle(UserSceneTextureIndex);
				TSharedPtr<IPropertyHandle> UserSceneTextureValueProperty = UserSceneTextureItemProperty->GetChildHandle("Value");

				IDetailPropertyRow& PropertyRow = UserSceneTexturesGroup.AddPropertyRow(UserSceneTextureValueProperty.ToSharedRef());

				PropertyRow.CustomWidget()
					.NameContent()
					[
						SNew(STextBlock)
						.Text(FText::FromName(MaterialEditorInstance->PostProcessOverrides.UserSceneTextureInputs[UserSceneTextureIndex].Key))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						UserSceneTextureValueProperty->CreatePropertyValueWidget()
					];
			}
		}

		PostProcessCategory.AddProperty(UserSceneTextureOutputProperty);
	}
}

void FMaterialInstanceParameterDetails::CreateBasePropertyOverrideWidgets(IDetailLayoutBuilder& DetailLayout, IDetailGroup& MaterialPropertyOverrideGroup)
{
	IDetailGroup& BasePropertyOverrideGroup = MaterialPropertyOverrideGroup;
	TSharedRef<IPropertyHandle> BasePropertyOverrideProperty = DetailLayout.GetProperty("BasePropertyOverrides");
	FMaterialInstanceBasePropertyOverrides& BaseOverrides = MaterialEditorInstance->BasePropertyOverrides;
	TObjectPtr<UMaterialInterface> ParentMat = MaterialEditorInstance->Parent;
	const FText ParameterDisabledToolTipString = LOCTEXT("DisableParameterOverride", "This material instance parent restricts the creation of new shader permutations. Overriding this parameter would result in the generation of additional shader permutations.");
	const bool bStaticParametersOverrideDisabled = MaterialEditorInstance->SourceInstance->bDisallowStaticParameterPermutations;

	auto CreateBaseOverrideRow = [&] (
		const char* PropertyName,
		auto&& OverrideBoolEnabledMemberFn,
		auto&& OverrideBoolChangedMemberFn,
		auto&& IsResetPropertyVisibleLambda,
		auto&& ResetPropertyHandlerLambda,
		auto&& OverrideAndVisibleMemberFn
	)
	{
		TSharedPtr<IPropertyHandle> ValueProperty = BasePropertyOverrideProperty->GetChildHandle(PropertyName);
		check(ValueProperty.IsValid());
		TAttribute<bool> OverrideBoolAttr = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, OverrideBoolEnabledMemberFn));
		FIsResetToDefaultVisible IsResetPropertyVisible = FIsResetToDefaultVisible::CreateLambda(IsResetPropertyVisibleLambda);
		FResetToDefaultHandler ResetPropertyHandler = FResetToDefaultHandler::CreateLambda(ResetPropertyHandlerLambda);
		FResetToDefaultOverride ResetPropertyOverride = FResetToDefaultOverride::Create(IsResetPropertyVisible, ResetPropertyHandler);
		IDetailPropertyRow& PropertyRow = BasePropertyOverrideGroup.AddPropertyRow(ValueProperty.ToSharedRef())
			.DisplayName(ValueProperty->GetPropertyDisplayName())
			.ToolTip(bStaticParametersOverrideDisabled ? ParameterDisabledToolTipString : ValueProperty->GetToolTipText())
			.EditCondition(OverrideBoolAttr, FOnBooleanValueChanged::CreateSP(this, OverrideBoolChangedMemberFn))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, OverrideAndVisibleMemberFn, OverrideBoolAttr)))
			.OverrideResetToDefault(ResetPropertyOverride);
	};

#define CREATE_BASE_OVERRIDE_ROW_CUSTOM(PropertyName, PropertyVariableName, IsResetPropertyVisibleLambda, ResetPropertyHandlerLambda, IsOverriddenAndVisibleFn) \
	CreateBaseOverrideRow ( \
		#PropertyVariableName, \
		&FMaterialInstanceParameterDetails::Override ## PropertyName ## Enabled, \
		&FMaterialInstanceParameterDetails::OnOverride ## PropertyName ## Changed, \
		IsResetPropertyVisibleLambda, \
		ResetPropertyHandlerLambda, \
		IsOverriddenAndVisibleFn \
		)

#define CREATE_BASE_OVERRIDE_ROW_OVERRIDEFN(PropertyName, PropertyVariableName, ValueGetterName, IsOverriddenAndVisibleFn) \
	CREATE_BASE_OVERRIDE_ROW_CUSTOM( \
		PropertyName, \
		PropertyVariableName, \
		[this] (TSharedPtr<IPropertyHandle> InHandle) { \
			if (TObjectPtr<UMaterialInterface> ParentMat = MaterialEditorInstance->Parent) \
			{ \
				return MaterialEditorInstance->BasePropertyOverrides.PropertyVariableName != ParentMat->ValueGetterName(); \
			} \
			else \
			{ \
				return false; \
			} \
		}, \
		[this] (TSharedPtr<IPropertyHandle> InHandle) { \
			if (TObjectPtr<UMaterialInterface> ParentMat = MaterialEditorInstance->Parent) \
			{ \
				MaterialEditorInstance->BasePropertyOverrides.PropertyVariableName = ParentMat->ValueGetterName(); \
			} \
		}, \
		IsOverriddenAndVisibleFn \
	)

#define CREATE_BASE_OVERRIDE_ROW(PropertyName, PropertyVariableName, ValueGetterName) \
		CREATE_BASE_OVERRIDE_ROW_OVERRIDEFN(PropertyName, PropertyVariableName, ValueGetterName, &FMaterialInstanceParameterDetails::IsOverriddenAndVisible)

#define CREATE_BASE_OVERRIDE_ROW_BASIC(PropertyName) \
	CREATE_BASE_OVERRIDE_ROW(PropertyName, PropertyName, Get ## PropertyName)
#define CREATE_BASE_OVERRIDE_ROW_BOOL(PropertyName, GetterName) \
	CREATE_BASE_OVERRIDE_ROW(PropertyName, b ## PropertyName, GetterName)
#define CREATE_BASE_OVERRIDE_ROW_BASIC_BOOL(PropertyName) \
	CREATE_BASE_OVERRIDE_ROW(PropertyName, b ## PropertyName, PropertyName)

	CREATE_BASE_OVERRIDE_ROW_BASIC(OpacityMaskClipValue);
	CREATE_BASE_OVERRIDE_ROW_BASIC(BlendMode);
	CREATE_BASE_OVERRIDE_ROW_CUSTOM(ShadingModel, ShadingModel,
		[this](TSharedPtr<IPropertyHandle> InHandle) {
			if (TObjectPtr<UMaterialInterface> ParentMat = MaterialEditorInstance->Parent)
			{
				if (ParentMat->IsShadingModelFromMaterialExpression())
				{
					return MaterialEditorInstance->BasePropertyOverrides.ShadingModel != MSM_FromMaterialExpression;
				}
				else
				{
					return MaterialEditorInstance->BasePropertyOverrides.ShadingModel != ParentMat->GetShadingModels().GetFirstShadingModel();
				}
			}
			else
			{
				return false;
			}
		},
		[this](TSharedPtr<IPropertyHandle> InHandle) {
			if (TObjectPtr<UMaterialInterface> ParentMat = MaterialEditorInstance->Parent)
			{
				if (ParentMat->IsShadingModelFromMaterialExpression())
				{
					MaterialEditorInstance->BasePropertyOverrides.ShadingModel = MSM_FromMaterialExpression;
				}
				else
				{
					MaterialEditorInstance->BasePropertyOverrides.ShadingModel = ParentMat->GetShadingModels().GetFirstShadingModel();
				}
			}
		},
		&FMaterialInstanceParameterDetails::IsOverriddenAndVisibleShadingModels
	);
	CREATE_BASE_OVERRIDE_ROW(TwoSided, TwoSided, IsTwoSided);
	CREATE_BASE_OVERRIDE_ROW_OVERRIDEFN(IsThinSurface, bIsThinSurface, IsThinSurface, &FMaterialInstanceParameterDetails::IsOverriddenAndVisibleSubstrateOnly);
	CREATE_BASE_OVERRIDE_ROW(DitheredLODTransition, DitheredLODTransition, IsDitheredLODTransition);
	CREATE_BASE_OVERRIDE_ROW_BOOL(OutputTranslucentVelocity, IsTranslucencyWritingVelocity);
	CREATE_BASE_OVERRIDE_ROW_BASIC_BOOL(HasPixelAnimation);
	CREATE_BASE_OVERRIDE_ROW_BOOL(EnableTessellation, IsTessellationEnabled);
	CREATE_BASE_OVERRIDE_ROW_BASIC(DisplacementScaling);
	CREATE_BASE_OVERRIDE_ROW_BOOL(EnableDisplacementFade, IsDisplacementFadeEnabled);
	CREATE_BASE_OVERRIDE_ROW_BASIC(DisplacementFadeRange);
	CREATE_BASE_OVERRIDE_ROW_BASIC(MaxWorldPositionOffsetDisplacement);
	CREATE_BASE_OVERRIDE_ROW_BOOL(CastDynamicShadowAsMasked, GetCastDynamicShadowAsMasked);
	CREATE_BASE_OVERRIDE_ROW_BOOL(CompatibleWithLumenCardSharing, IsCompatibleWithLumenCardSharing);

#undef CREATE_BASE_OVERRIDE_ROW_BASIC_BOOL
#undef CREATE_BASE_OVERRIDE_ROW_BOOL
#undef CREATE_BASE_OVERRIDE_ROW_BASIC
#undef CREATE_BASE_OVERRIDE_ROW
#undef CREATE_BASE_OVERRIDE_ROW_CUSTOM
}

EVisibility FMaterialInstanceParameterDetails::IsOverriddenAndVisible(TAttribute<bool> IsOverridden) const
{
	bool bShouldBeVisible = true;
	if (MaterialEditorInstance->bShowOnlyOverrides)
	{
		bShouldBeVisible = IsOverridden.Get();
	}
	return bShouldBeVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMaterialInstanceParameterDetails::IsOverriddenAndVisibleShadingModels(TAttribute<bool> IsOverridden) const
{
	bool bShouldBeVisible = true;
	if (MaterialEditorInstance->bShowOnlyOverrides)
	{
		bShouldBeVisible = IsOverridden.Get();
	}
	// If Substrate is enabled, only allows ShadingModel to be visible if the parent allows it
	if (Substrate::IsSubstrateEnabled())
	{
		const UMaterial* ParentMaterial = MaterialEditorInstance->Parent ? MaterialEditorInstance->Parent->GetMaterial() : nullptr;
		bShouldBeVisible = ParentMaterial ? ParentMaterial->SupportsShadingModelOverride() : false;
	}
	return bShouldBeVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FMaterialInstanceParameterDetails::IsOverriddenAndVisibleSubstrateOnly(TAttribute<bool> IsOverridden) const
{
	return Substrate::IsSubstrateEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

/** Helper function used by some parameters to verify that they are allowed to be overridden. This
  * must be prevented if the source material instance disallows the creation of new static parameter
  * permutations as that would trigger a new shader creation. */
static bool DoesSourceMaterialInstanceDisallowStaticParameterPermutation(const UMaterialEditorInstanceConstant* mi, bool NewValue)
{
	if (NewValue && mi->SourceInstance->bDisallowStaticParameterPermutations)
	{
		return true;
	}
	return false;
}

#define IMPLEMENT_OVERRIDE_MEMBER_FUNCS_COMMON(PropertyName, PropertyVariableName, bRequiresPermutation) \
	bool FMaterialInstanceParameterDetails::Override ## PropertyName ## Enabled() const \
	{ \
		return MaterialEditorInstance->BasePropertyOverrides.bOverride_ ## PropertyVariableName ; \
	} \
	void FMaterialInstanceParameterDetails::OnOverride ## PropertyName ## Changed(bool NewValue) \
	{ \
		if (DoesSourceMaterialInstanceDisallowStaticParameterPermutation(MaterialEditorInstance, NewValue) && \
			bRequiresPermutation) \
		{ \
			return; \
		} \
		MaterialEditorInstance->BasePropertyOverrides.bOverride_ ## PropertyVariableName = NewValue; \
		MaterialEditorInstance->PostEditChange(); \
		FEditorSupportDelegates::RedrawAllViewports.Broadcast(); \
	}
#define IMPLEMENT_OVERRIDE_MEMBER_FUNCS(PropertyName, bRequiresPermutation) \
	IMPLEMENT_OVERRIDE_MEMBER_FUNCS_COMMON(PropertyName, PropertyName, bRequiresPermutation)
#define IMPLEMENT_OVERRIDE_MEMBER_FUNCS_BOOL(PropertyName, bRequiresPermutation) \
	IMPLEMENT_OVERRIDE_MEMBER_FUNCS_COMMON(PropertyName, b ## PropertyName, bRequiresPermutation)

IMPLEMENT_OVERRIDE_MEMBER_FUNCS(OpacityMaskClipValue, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(BlendMode, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(ShadingModel, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(TwoSided, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS_BOOL(IsThinSurface, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(DitheredLODTransition, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(OutputTranslucentVelocity, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS_BOOL(HasPixelAnimation, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS_BOOL(EnableTessellation, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(DisplacementScaling, false)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS_BOOL(EnableDisplacementFade, false)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(DisplacementFadeRange, false)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(MaxWorldPositionOffsetDisplacement, false)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(CastDynamicShadowAsMasked, true)
IMPLEMENT_OVERRIDE_MEMBER_FUNCS(CompatibleWithLumenCardSharing, false)

#undef IMPLEMENT_OVERRIDE_MEMBER_FUNCS_BOOL
#undef IMPLEMENT_OVERRIDE_MEMBER_FUNCS
#undef IMPLEMENT_OVERRIDE_MEMBER_FUNCS_COMMON

void FMaterialInstanceParameterDetails::CreateUsageFlagOverrideWidgets(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& CategoryBuilder)
{
	static FName GroupName(TEXT("UsageFlagOverrideGroup"));
	IDetailGroup& Group = CategoryBuilder.AddGroup(GroupName, LOCTEXT("UsageFlagOverrideGroup", "Usage Flag Overrides"), false, false);
	TSharedRef<IPropertyHandle> ParentProperty = DetailLayout.GetProperty("BasePropertyOverrides");
	TSharedPtr<IPropertyHandle> OverrideProperty = ParentProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMaterialInstanceBasePropertyOverrides, bOverride_UsageFlags));
	TSharedPtr<IPropertyHandle> UsageProperty = ParentProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMaterialInstanceBasePropertyOverrides, UsageFlags));
	UEnum* UsageEnum = StaticEnum<EMaterialUsage>();
	
	// Disable overrides for case where parent material instance is locked.
	const bool bIsWidgetDisabled = MaterialEditorInstance->SourceInstance->bDisallowStaticParameterPermutations;
	const FText DisabledToolTipText = LOCTEXT("DisableUsageOverride", "This material instance parent restricts the creation of new shader permutations. Overriding this usage flag would result in the generation of additional shader permutations.");

	for (int32 UsageIndex = 0; UsageIndex < MATUSAGE_MAX; UsageIndex++)
	{
		const FText UsageName = FText::FromString(UMaterialInterface::GetUsageName((EMaterialUsage)UsageIndex));
		const FText UsageToolTip = bIsWidgetDisabled ? DisabledToolTipText : FText::FromString(UsageEnum->GetMetaData(TEXT("ToolTip"), UsageIndex));

		Group.AddWidgetRow()
		.PropertyHandleList({UsageProperty})
		.IsEnabled(!bIsWidgetDisabled)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(UsageName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTip(SNew(SToolTip).Text(UsageToolTip))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([UsageProperty, UsageIndex]() -> ECheckBoxState
			{
				uint32 Value = 0;
				UsageProperty->GetValue(Value);
				return (Value & (1u << UsageIndex)) != 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([UsageProperty, UsageIndex](ECheckBoxState InState)
			{
				uint32 Value = 0;
				UsageProperty->GetValue(Value);

				if (InState == ECheckBoxState::Checked)
				{
					Value |= (1u << UsageIndex);
				}
				else
				{
					Value &= ~(1u << UsageIndex);
				}
				UsageProperty->SetValue(Value);
			})
		]
		.EditCondition(
			TAttribute<bool>::CreateLambda([OverrideProperty, UsageIndex]
			{
				uint32 Value = 0;
				OverrideProperty->GetValue(Value);
				return (Value & (1u << UsageIndex)) != 0;
			}),
			FOnBooleanValueChanged::CreateLambda([OverrideProperty, UsageIndex, bIsWidgetDisabled](bool bNewValue)
			{
				// Edit condition doesn't have a disabled flag and doesn't respect the row setting. So ignore changes when we are disabled.
				if (bIsWidgetDisabled)
				{
					return;
				}

				uint32 Value = 0;
				OverrideProperty->GetValue(Value);
				
				if (bNewValue)
				{
					Value |= (1u << UsageIndex);
				}
				else
				{
					Value &= ~(1u << UsageIndex);
				}
				OverrideProperty->SetValue(Value);
			})
		)
		.OverrideResetToDefault(FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateLambda([this, UsageIndex](TSharedPtr<IPropertyHandle>)
			{
				TObjectPtr<UMaterialInterface> Material = MaterialEditorInstance->GetMaterialInterface();
				TObjectPtr<UMaterialInterface> ParentMaterial = MaterialEditorInstance->GetParentMaterialInterface();
				if (!Material || !ParentMaterial)
				{
					return false;
				}
				return Material->GetUsageByFlag((EMaterialUsage)UsageIndex) != ParentMaterial->GetUsageByFlag((EMaterialUsage)UsageIndex);
			}), 
			FResetToDefaultHandler::CreateLambda([UsageProperty, UsageIndex](TSharedPtr<IPropertyHandle>)
			{
				// Simply toggling the bit is enough here since we can call this after the
				// IsResetToDefaultVisible above returns that we are different to the parent
				uint32 Value = 0;
				UsageProperty->GetValue(Value);
				Value ^= (1u << UsageIndex);
				UsageProperty->SetValue(Value);
			}))
		);
	}
}

#undef LOCTEXT_NAMESPACE
