// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Layout/Visibility.h"
#include "IDetailCustomization.h"
#include "SMaterialLayersFunctionsTree.h"
#include "Input/Reply.h"
#include "Customizations/ColorStructCustomization.h"


struct FAssetData;
class IDetailGroup;
class IDetailPropertyRow;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UMaterialEditorInstanceConstant;

DECLARE_DELEGATE_OneParam(FGetShowHiddenParameters, bool&);

/*-----------------------------------------------------------------------------
   FMaterialInstanceParameterDetails
-----------------------------------------------------------------------------*/

class FMaterialInstanceParameterDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(UMaterialEditorInstanceConstant* MaterialInstance, SMaterialLayersFunctionsInstanceWrapper* MaterialLayersFunctionsInstance, FGetShowHiddenParameters InShowHiddenDelegate);
	
	/** Constructor */
	FMaterialInstanceParameterDetails(UMaterialEditorInstanceConstant* MaterialInstance, SMaterialLayersFunctionsInstanceWrapper* MaterialLayersFunctionsInstance, FGetShowHiddenParameters InShowHiddenDelegate);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	/** Returns the function parent path */
	FString GetFunctionParentPath() const;

	/** Builds the custom parameter groups category */
	void CreateGroupsWidget(TSharedRef<IPropertyHandle> ParameterGroupsProperty, class IDetailCategoryBuilder& GroupsCategory);
	void CreateParameterCollectionOverrideCategory(TSharedRef<IPropertyHandle> ParameterGroupsProperty, class IDetailLayoutBuilder& DetailLayout);

	/** Builds the widget for an individual parameter group */
	void CreateSingleGroupWidget(struct FEditorParameterGroup& ParameterGroup, TSharedPtr<IPropertyHandle> ParameterGroupProperty, class IDetailGroup& DetailGroup, int32 GroupIndex = -1, bool bForceShowParam = false);

	/** Enable/Disable all parameter properties in a group */
	void EnableGroupParameters(TArray<int32> ParameterGroupIndices, bool bShouldEnable);

	/** Called to check if an asset can be set as a parent */
	bool OnShouldSetAsset(const FAssetData& InAssetData) const;

	/** Called when an asset is set as a parent */
	void OnAssetChanged(const FAssetData& InAssetData, TSharedRef<IPropertyHandle> InHandle);

	/** Returns true if the refraction options should be displayed */
	EVisibility ShouldShowMaterialRefractionSettings() const;

	/** Returns true if the Subsurface Profile options should be displayed */
	EVisibility ShouldShowSubsurfaceProfile() const;

	/** Returns true if the Specular Profile options should be displayed */
	EVisibility ShouldShowSpecularProfile() const;

	/** Returns true if the Toon Profile options should be displayed */
	EVisibility ShouldShowToonProfile() const;

	//Functions supporting copy/paste of entire parameter groups.

	/**
	 * Copy all parameter values in a parameter group to the clipboard
	 * in the format "Param1=\"Value1\"\nParams2=\"Value2\"..."
	 */
	void OnCopyParameterValues(TArray<int32> ParameterGroupIndices);

	/** Whether it is possible to copy parameter values for the given parent group index */
	bool CanCopyParameterValues(TArray<int32> ParameterGroupIndices);

	/**
	 * Paste parameter values from the clipboard, assumed to be
	 * in the format copied by CopyParameterValues.
	 */
	void OnPasteParameterValues(TArray<int32> ParameterGroupIndices);

	/**
	 * Whether it is possible to paste parameter values onto the given group index,
	 * and if there is anything on the clipboard to paste from.
	 */
	bool CanPasteParameterValues(TArray<int32> ParameterGroupIndices);

	/** Creates all the lightmass property override widgets. */
	void CreateLightmassOverrideWidgets(IDetailLayoutBuilder& DetailLayout);

	/** Creates Blendable Location / Priority and UserSceneTexture input / output override widgets. */
	void CreatePostProcessOverrideWidgets(IDetailLayoutBuilder& DetailLayout);

	//Functions supporting BasePropertyOverrides

	/** Creates all the base property override widgets. */
	void CreateBasePropertyOverrideWidgets(IDetailLayoutBuilder& DetailLayout, IDetailGroup& MaterialPropertyOverrideGroup);
	/** Creates all the usage flag override widgets. */
	void CreateUsageFlagOverrideWidgets(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& CategoryBuilder);

	EVisibility IsOverriddenAndVisible(TAttribute<bool> IsOverridden) const;
	EVisibility IsOverriddenAndVisibleShadingModels(TAttribute<bool> IsOverridden) const;
	EVisibility IsOverriddenAndVisibleSubstrateOnly(TAttribute<bool> IsOverridden) const;

#define DECLARE_OVERRIDE_MEMBER_FUNCS(PropertyName) \
	bool Override ## PropertyName ## Enabled() const; \
	void OnOverride ## PropertyName ## Changed(bool NewValue);

	DECLARE_OVERRIDE_MEMBER_FUNCS(OpacityMaskClipValue)
	DECLARE_OVERRIDE_MEMBER_FUNCS(BlendMode)
	DECLARE_OVERRIDE_MEMBER_FUNCS(ShadingModel)
	DECLARE_OVERRIDE_MEMBER_FUNCS(TwoSided)
	DECLARE_OVERRIDE_MEMBER_FUNCS(IsThinSurface)
	DECLARE_OVERRIDE_MEMBER_FUNCS(DitheredLODTransition)
	DECLARE_OVERRIDE_MEMBER_FUNCS(OutputTranslucentVelocity)
	DECLARE_OVERRIDE_MEMBER_FUNCS(HasPixelAnimation)
	DECLARE_OVERRIDE_MEMBER_FUNCS(EnableTessellation)
	DECLARE_OVERRIDE_MEMBER_FUNCS(DisplacementScaling)
	DECLARE_OVERRIDE_MEMBER_FUNCS(EnableDisplacementFade)
	DECLARE_OVERRIDE_MEMBER_FUNCS(DisplacementFadeRange)
	DECLARE_OVERRIDE_MEMBER_FUNCS(MaxWorldPositionOffsetDisplacement)
	DECLARE_OVERRIDE_MEMBER_FUNCS(CastDynamicShadowAsMasked)
	DECLARE_OVERRIDE_MEMBER_FUNCS(CompatibleWithLumenCardSharing)
	
#undef DECLARE_OVERRIDE_MEMBER_FUNCS

private:
	/** Object that stores all of the possible parameters we can edit */
	UMaterialEditorInstanceConstant* MaterialEditorInstance;

	SMaterialLayersFunctionsInstanceWrapper* MaterialLayersFunctionsInstance;
	
	/** Delegate to call to determine if hidden parameters should be shown */
	FGetShowHiddenParameters ShowHiddenDelegate;

	/** Associated FMaterialInstance utilities */
	TWeakPtr<class IPropertyUtilities> PropertyUtilities;
};

