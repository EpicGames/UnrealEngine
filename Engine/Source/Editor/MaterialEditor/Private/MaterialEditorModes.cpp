// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorModes.h"
#include "Settings/EditorExperimentalSettings.h"
#include "ToolMenus.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"

#include "Framework/Docking/LayoutExtender.h"
#include "MaterialStats.h"
#include "MaterialEditor.h"
#include "MaterialEditorTabs.h"
#include "Tabs/MaterialEditorTabFactories.h"

#define LOCTEXT_NAMESPACE "MaterialEditor"

const FName FMaterialEditorApplicationModes::StandardMaterialEditorMode(TEXT("GraphName"));

TSharedPtr<FTabManager::FLayout> FMaterialEditorApplicationModes::GetDefaultEditorLayout(TSharedPtr<FMaterialEditor> InMaterialEditor)
{
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_MaterialEditor_Layout_v17")
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal) ->SetSizeCoefficient(0.9f)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical) ->SetSizeCoefficient(0.2f)
				->Split
				(
					FTabManager::NewStack()
					->SetHideTabWell( true )
					->AddTab( FMaterialEditorTabs::PreviewTabId, ETabState::OpenedTab )
				)
				->Split
				(
					FTabManager::NewStack()
					->AddTab( FMaterialEditorTabs::PropertiesTabId, ETabState::OpenedTab )
					->AddTab( FMaterialEditorTabs::PreviewSettingsTabId, ETabState::ClosedTab )
					->AddTab( FMaterialEditorTabs::ParameterDefaultsTabId, ETabState::OpenedTab )
					->AddTab( FMaterialEditorTabs::CustomPrimitiveTabId, ETabState::ClosedTab )
					->AddTab( FMaterialEditorTabs::LayerPropertiesTabId, ETabState::ClosedTab )
					->AddTab( FMaterialEditorTabs::MaterialOverviewTabId, ETabState::OpenedTab )
					->SetForegroundTab( FMaterialEditorTabs::PropertiesTabId )
				)
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
				->SetSizeCoefficient(0.80f)
				->Split
				(
					FTabManager::NewStack() 
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell( true )
					->AddTab(FMaterialEditorTabs::GraphEditor, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient( 0.20f )
					->AddTab( FMaterialStats::GetGridStatsTabName(), ETabState::ClosedTab )
					->AddTab( FMaterialStats::GetGridOldStatsTabName(), ETabState::ClosedTab )
					->AddTab( FMaterialEditorTabs::FindTabId, ETabState::ClosedTab )
					->AddTab( FMaterialEditorTabs::SubstrateTabId, ETabState::OpenedTab )
				)
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal) ->SetSizeCoefficient(0.2f)
				->Split
				(
					FTabManager::NewStack()
					->AddTab( FMaterialEditorTabs::PaletteTabId, ETabState::SidebarTab, ESidebarLocation::Right, 0.14f )
					->AddTab( FMaterialEditorTabs::NamedReroutesTabId, ETabState::SidebarTab, ESidebarLocation::Right, 0.14f )
				)
			)
		)
	);

	return StandaloneDefaultLayout;
}

FMaterialEditorApplicationMode::FMaterialEditorApplicationMode(TSharedPtr<FMaterialEditor> InMaterialEditor)
	: FApplicationMode(FMaterialEditorApplicationModes::StandardMaterialEditorMode, FMaterialEditorApplicationModes::GetLocalizedMode)
{
	MyMaterialEditor = InMaterialEditor;
	MaterialEditorTabFactories.RegisterFactory(InMaterialEditor->GraphEditorTabFactoryPtr.Pin().ToSharedRef());
	TabLayout = FMaterialEditorApplicationModes::GetDefaultEditorLayout(InMaterialEditor);
}

void FMaterialEditorApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FMaterialEditor> MaterialEditor = MyMaterialEditor.Pin();

	MaterialEditor->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	MaterialEditor->PushTabFactories(MaterialEditorTabFactories);
}

#undef LOCTEXT_NAMESPACE