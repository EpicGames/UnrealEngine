// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorModule.h"
#include "Modules/ModuleManager.h"
#include "IMaterialEditor.h"
#include "MaterialEditor.h"
#include "MaterialEditorUtilities.h"
#include "MaterialInstanceEditor.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Settings/EditorExperimentalSettings.h"
#include "ContentBrowserModule.h"
#include "Styling/AppStyle.h"
#include "ContentBrowserDelegates.h"
#include "Interfaces/IPluginManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Misc/EngineBuildSettings.h"
#include "MaterialEditorSettings.h"
#include "EdGraphUtilities.h"
#include "ISettingsModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "MaterialEditorGraphPanelPinFactory.h"
#include "SubstrateMaterialEditorStyle.h"
#include "MaterialEditorDetailCustomization.h"
#include "MaterialGraphConnectionDrawingPolicy.h"
#include "MaterialEditingLibrary.h"

const FName MaterialEditorAppIdentifier = FName(TEXT("MaterialEditorApp"));
const FName MaterialInstanceEditorAppIdentifier = FName(TEXT("MaterialInstanceEditorApp"));

namespace MaterialEditorModuleConstants
{
	const static FName SettingsContainerName("Editor");
	const static FName SettingsCategoryName("ContentEditors");
	const static FName SettingsSectionName("Material Editor");
}

/**
 * Material editor module
 */
class FMaterialEditorModule : public IMaterialEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FMaterialEditorModule()
	{
		// Trigger a redraw of post process preview materials when this debug setting changes
		IConsoleVariable* CVarPostProcessUserSceneTextureDebug = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.UserSceneTextureDebug"));
		if (CVarPostProcessUserSceneTextureDebug)
		{
			CVarPostProcessUserSceneTextureDebug->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Variable)
			{
				FMaterialEditorUtilities::RefreshPostProcessPreviewMaterials(nullptr, /* bRedrawOnly= */ true);
			}));
		}
	}

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		ISettingsModule* SettingsModule = FModuleManager::LoadModulePtr<ISettingsModule>("Settings");
		if (SettingsModule)
		{
			SettingsModule->RegisterSettings(
				MaterialEditorModuleConstants::SettingsContainerName,
				MaterialEditorModuleConstants::SettingsCategoryName,
				MaterialEditorModuleConstants::SettingsSectionName,
				NSLOCTEXT("MaterialEditorModule", "SettingsName", "Material Editor"),
				NSLOCTEXT("MaterialEditorModule", "SettingsDesc", "Settings related to the material editor."),
				GetMutableDefault<UMaterialEditorSettings>());
		}

		// Register customizaed detail panel layouts
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(
			"MaterialExpressionColorRamp",
			FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialExpressionColorRampCustomization::MakeInstance)
		);
		PropertyModule.RegisterCustomClassLayout(
			"MaterialExpressionOperator",
			FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialExpressionOperatorCustomization::MakeInstance)
		);

		// Register a callback to bulk update materials when many materials are edited and saved via the property matrix.
		OnPropertyMatrixAssetsSavedDelegateHandle = PropertyModule.OnPropertyMatrixAssetsSaved().AddLambda([](const TArray<UObject*>& SavedObjects)
		{
			TArray<UMaterial*> Materials;
			Materials.Reserve(SavedObjects.Num());
			for (UObject* Object : SavedObjects)
			{
				if (UMaterial* Material = Cast<UMaterial>(Object))
				{
					Materials.Add(Material);
				}
			}
			UMaterialEditingLibrary::RecompileMaterials(Materials, UMaterialEditingLibrary::FOnItemComplete::CreateLambda([]() {}));
		});

		GraphPanelPinFactory = MakeShared<FMaterialEditorGraphPanelPinFactory>();
		FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

		GraphConnectionFactory = MakeShared<FMaterialEditorGraphPinConnectionFactory>();
		FEdGraphUtilities::RegisterVisualPinConnectionFactory(GraphConnectionFactory);

		FSubstrateMaterialEditorStyle::Initialize();
	}

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override
	{
		FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);

		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout("MaterialExpressionColorRamp");
			PropertyModule.UnregisterCustomClassLayout("MaterialExpressionOperator");
			PropertyModule.OnPropertyMatrixAssetsSaved().Remove(OnPropertyMatrixAssetsSavedDelegateHandle);
		}
		
		FSubstrateMaterialEditorStyle::Shutdown();
	}

	/**
	 * Creates a new material editor, either for a material or a material function
	 */
	virtual TSharedRef<IMaterialEditor> CreateMaterialEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterial* Material) override
	{
		TSharedRef<FMaterialEditor> NewMaterialEditor(new FMaterialEditor());
		Material->SetMarkTextureAsEditorStreamingPool(true);
		NewMaterialEditor->InitEditorForMaterial(Material);
		OnMaterialEditorOpened().Broadcast(NewMaterialEditor);
		NewMaterialEditor->InitMaterialEditor(Mode, InitToolkitHost, Material);
		return NewMaterialEditor;
	}

	virtual TSharedRef<IMaterialEditor> CreateMaterialEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterialFunction* MaterialFunction) override
	{
		TSharedRef<FMaterialEditor> NewMaterialEditor(new FMaterialEditor());
		NewMaterialEditor->InitEditorForMaterialFunction(MaterialFunction);
		OnMaterialFunctionEditorOpened().Broadcast(NewMaterialEditor);
		NewMaterialEditor->InitMaterialEditor(Mode, InitToolkitHost, MaterialFunction);
		return NewMaterialEditor;
	}

	virtual TSharedRef<IMaterialEditor> CreateMaterialInstanceEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterialInstance* MaterialInstance) override
	{
		TSharedRef<FMaterialInstanceEditor> NewMaterialInstanceEditor(new FMaterialInstanceEditor());
		MaterialInstance->SetMarkTextureAsEditorStreamingPool(true);
		NewMaterialInstanceEditor->InitEditorForMaterial(MaterialInstance);
		OnMaterialInstanceEditorOpened().Broadcast(NewMaterialInstanceEditor);
		NewMaterialInstanceEditor->InitMaterialInstanceEditor(Mode, InitToolkitHost, MaterialInstance);
		return NewMaterialInstanceEditor;
	}

	virtual TSharedRef<IMaterialEditor> CreateMaterialInstanceEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterialFunctionInstance* MaterialFunction) override
	{
		TSharedRef<FMaterialInstanceEditor> NewMaterialInstanceEditor(new FMaterialInstanceEditor());
		NewMaterialInstanceEditor->InitEditorForMaterialFunction(MaterialFunction);
		OnMaterialInstanceEditorOpened().Broadcast(NewMaterialInstanceEditor);
		NewMaterialInstanceEditor->InitMaterialInstanceEditor(Mode, InitToolkitHost, MaterialFunction);
		return NewMaterialInstanceEditor;
	}
	
	virtual void GetVisibleMaterialParameters(const class UMaterial* Material, class UMaterialInstance* MaterialInstance, TArray<FMaterialParameterInfo>& VisibleExpressions) override
	{
		FMaterialEditorUtilities::GetVisibleMaterialParameters(Material, MaterialInstance, VisibleExpressions);
	}

	/** Gets the extensibility managers for outside entities to extend material editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FMaterialEditorGraphPanelPinFactory> GraphPanelPinFactory;
	TSharedPtr<FGraphPanelPinConnectionFactory> GraphConnectionFactory;
	FDelegateHandle OnPropertyMatrixAssetsSavedDelegateHandle;
};

IMPLEMENT_MODULE( FMaterialEditorModule, MaterialEditor );
