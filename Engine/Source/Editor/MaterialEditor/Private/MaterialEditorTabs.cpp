// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorTabs.h"

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

const FName FMaterialEditorTabs::PreviewTabId(TEXT("MaterialEditor_Preview"));
const FName FMaterialEditorTabs::PropertiesTabId(TEXT("MaterialEditor_MaterialProperties"));
const FName FMaterialEditorTabs::PaletteTabId(TEXT("MaterialEditor_Palette"));
const FName FMaterialEditorTabs::FindTabId(TEXT("MaterialEditor_Find"));
const FName FMaterialEditorTabs::GraphEditor(TEXT("Document"));
const FName FMaterialEditorTabs::PreviewSettingsTabId(TEXT("MaterialEditor_PreviewSettings"));
const FName FMaterialEditorTabs::ParameterDefaultsTabId(TEXT("MaterialEditor_ParameterDefaults"));
const FName FMaterialEditorTabs::CustomPrimitiveTabId(TEXT("MaterialEditor_CustomPrimitiveData"));
const FName FMaterialEditorTabs::LayerPropertiesTabId(TEXT("MaterialInstanceEditor_MaterialLayerProperties"));
const FName FMaterialEditorTabs::SubstrateTabId(TEXT("MaterialEditor_Substrate"));
const FName FMaterialEditorTabs::MaterialOverviewTabId(TEXT("MaterialEditor_MaterialExpressionOverview"));
const FName FMaterialEditorTabs::NamedReroutesTabId(TEXT("MaterialEditor_NamedReroutes"));


const FName FMaterialInstanceEditorTabs::PreviewTabId( TEXT( "MaterialInstanceEditor_Preview" ) );
const FName FMaterialInstanceEditorTabs::PropertiesTabId( TEXT( "MaterialInstanceEditor_MaterialProperties" ) );
const FName FMaterialInstanceEditorTabs::LayerPropertiesTabId(TEXT("MaterialInstanceEditor_MaterialLayerProperties"));
const FName FMaterialInstanceEditorTabs::PreviewSettingsTabId(TEXT("MaterialInstanceEditor_PreviewSettings"));
