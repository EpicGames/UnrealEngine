// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

/**  Material editor tabs identifiers */
struct FMaterialEditorTabs
{
	/**	The tab id for the Material fixture types tab */
	static const FName PreviewTabId;
	static const FName PropertiesTabId;
	static const FName PaletteTabId;
	static const FName FindTabId;
	static const FName GraphEditor;
	static const FName PreviewSettingsTabId;
	static const FName ParameterDefaultsTabId;
	static const FName CustomPrimitiveTabId;
	static const FName LayerPropertiesTabId;
	static const FName SubstrateTabId;
	static const FName MaterialOverviewTabId;
	static const FName NamedReroutesTabId;

	// Disable default constructor
	FMaterialEditorTabs() = delete;
};

struct FMaterialInstanceEditorTabs
{
	/**	The ids for the tabs spawned by this toolkit */
	static const FName PreviewTabId;
	static const FName PropertiesTabId;
	static const FName LayerPropertiesTabId;
	static const FName PreviewSettingsTabId;

	// Disable default constructor
	FMaterialInstanceEditorTabs() = delete;
};

