// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorUtilities.h"
#include "UObject/UObjectHash.h"
#include "EdGraph/EdGraph.h"
#include "Materials/Material.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "IMaterialEditor.h"

#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionTextureCollectionParameter.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionRerouteBase.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"

#include "DebugViewModeHelpers.h"
#include "Toolkits/ToolkitManager.h"
#include "MaterialEditor.h"
#include "MaterialExpressionClasses.h"
#include "MaterialInstanceEditor.h"
#include "Materials/MaterialInstance.h"
#include "MaterialUtilities.h"
#include "Misc/ScopedSlowTask.h"
#include "Templates/UniquePtr.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "MaterialEditor/PreviewMaterial.h"

#define LOCTEXT_NAMESPACE "MaterialEditorUtilities"

DEFINE_LOG_CATEGORY_STATIC(LogMaterialEditorUtilities, Log, All);

UMaterialExpression* FMaterialEditorUtilities::CreateNewMaterialExpression(const class UEdGraph* Graph, UClass* NewExpressionClass, const UE::Slate::FDeprecateVector2DParameter& NodePos, bool bAutoSelect, bool bAutoAssignResource)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		return MaterialEditor->CreateNewMaterialExpression(NewExpressionClass, FDeprecateSlateVector2D(NodePos), bAutoSelect, bAutoAssignResource, Graph);
	}
	return nullptr;
}

UMaterialExpressionComposite* FMaterialEditorUtilities::CreateNewMaterialExpressionComposite(const class UEdGraph* Graph, const FVector2D& NodePos)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		return MaterialEditor->CreateNewMaterialExpressionComposite(NodePos, Graph);
	}
	return nullptr;
}

UMaterialExpressionComment* FMaterialEditorUtilities::CreateNewMaterialExpressionComment(const class UEdGraph* Graph, const FVector2D& NodePos)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		return MaterialEditor->CreateNewMaterialExpressionComment(NodePos, Graph);
	}
	return nullptr;
}

void FMaterialEditorUtilities::ForceRefreshExpressionPreviews(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->ForceRefreshExpressionPreviews();
	}
}

void FMaterialEditorUtilities::AddToSelection(const class UEdGraph* Graph, UMaterialExpression* Expression)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->AddToSelection(Expression);
	}
}

void FMaterialEditorUtilities::DeleteSelectedNodes(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->DeleteSelectedNodes();
	}
}


void FMaterialEditorUtilities::DeleteNodes(const class UEdGraph* Graph, const TArray<UEdGraphNode*>& NodesToDelete)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->DeleteNodes(NodesToDelete);
	}
}

FText FMaterialEditorUtilities::GetOriginalObjectName(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		return MaterialEditor->GetOriginalObjectName();
	}

	// When diffing Material, there might not be any MaterialEditor opened for the Material
	// So we have to get the MaterialGraph's name another way
	if (UMaterial* GraphMaterial = Cast<UMaterial>(Graph->GetOuter()))
	{
		return FText::FromString(GraphMaterial->GetName());
	}
	
	return FText::GetEmpty();
}

TSharedPtr<IAssetReferenceFilter> FMaterialEditorUtilities::MakeMaterialAssetReferenceFilter(const UEdGraph* Graph)
{
	if (TSharedPtr<IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph))
	{
		return MaterialEditor->MakeAssetReferenceFilter();
	}

	return {};
}

void FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->UpdateMaterialAfterGraphChange();
	}
}

void FMaterialEditorUtilities::MarkMaterialDirty(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->MarkMaterialDirty();
	}
}

void FMaterialEditorUtilities::UpdateDetailView(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->UpdateDetailView();
	}
}

bool FMaterialEditorUtilities::CanPasteNodes(const class UEdGraph* Graph)
{
	bool bCanPaste = false;
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if(MaterialEditor.IsValid())
	{
		bCanPaste = MaterialEditor->CanPasteNodes();
	}
	return bCanPaste;
}

void FMaterialEditorUtilities::PasteNodesHere(class UEdGraph* Graph, const FVector2D& Location)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if(MaterialEditor.IsValid())
	{
		MaterialEditor->PasteNodesHere(Location);
	}
}

int32 FMaterialEditorUtilities::GetNumberOfSelectedNodes(const class UEdGraph* Graph)
{
	int32 SelectedNodes = 0;
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if(MaterialEditor.IsValid())
	{
		SelectedNodes = MaterialEditor->GetNumberOfSelectedNodes();
	}
	return SelectedNodes;
}

void FMaterialEditorUtilities::GetMaterialExpressionActions(FGraphActionMenuBuilder& ActionMenuBuilder, bool bMaterialFunction)
{
	UObject* MaterialOrFunction = bMaterialFunction ? UMaterialFunction::StaticClass()->GetDefaultObject() : UMaterial::StaticClass()->GetDefaultObject();
	GetMaterialExpressionActions(ActionMenuBuilder, MaterialOrFunction);
}

void FMaterialEditorUtilities::GetMaterialExpressionActions(FGraphActionMenuBuilder& ActionMenuBuilder, const UObject* MaterialOrFunction)
{
	bool bUseUnsortedMenus = false;
	MaterialExpressionClasses* ExpressionClasses = MaterialExpressionClasses::Get();

	if (bUseUnsortedMenus)
	{
		AddMaterialExpressionCategory(ActionMenuBuilder, FText::GetEmpty(), &ExpressionClasses->AllExpressionClasses, MaterialOrFunction);
	}
	else
	{
		// Add Favourite expressions as a category
		const FText FavouritesCategory = LOCTEXT("FavoritesMenu", "Favorites");
		AddMaterialExpressionCategory(ActionMenuBuilder, FavouritesCategory, &ExpressionClasses->FavoriteExpressionClasses, MaterialOrFunction);

		// Add each category to the menu
		for (int32 CategoryIndex = 0; CategoryIndex < ExpressionClasses->CategorizedExpressionClasses.Num(); ++CategoryIndex)
		{
			FCategorizedMaterialExpressionNode* CategoryNode = &(ExpressionClasses->CategorizedExpressionClasses[CategoryIndex]);
			AddMaterialExpressionCategory(ActionMenuBuilder, CategoryNode->CategoryName, &CategoryNode->MaterialExpressions, MaterialOrFunction);
		}

		if (ExpressionClasses->UnassignedExpressionClasses.Num() > 0)
		{
			AddMaterialExpressionCategory(ActionMenuBuilder, FText::GetEmpty(), &ExpressionClasses->UnassignedExpressionClasses, MaterialOrFunction);
		}
	}
}

bool FMaterialEditorUtilities::IsMaterialExpressionInFavorites(UMaterialExpression* InExpression)
{
	return MaterialExpressionClasses::Get()->IsMaterialExpressionInFavorites(InExpression);
}

FMaterialRenderProxy* FMaterialEditorUtilities::GetExpressionPreview(const class UEdGraph* Graph, UMaterialExpression* InExpression)
{
	FMaterialRenderProxy* ExpressionPreview = NULL;
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if(MaterialEditor.IsValid())
	{
		ExpressionPreview = MaterialEditor->GetExpressionPreview(InExpression);
	}
	return ExpressionPreview;
}

void FMaterialEditorUtilities::UpdateSearchResults(const class UEdGraph* Graph)
{
	TSharedPtr<class IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(Graph);
	if(MaterialEditor.IsValid())
	{
		MaterialEditor->UpdateSearch(false);
	}
}

/////////////////////////////////////////////////////
// Static functions moved from SMaterialEditorCanvas

void FMaterialEditorUtilities::GetVisibleMaterialParameters(const UMaterial* Material, UMaterialInstance* MaterialInstance, TArray<FMaterialParameterInfo>& VisibleExpressions)
{
	check(Material);
	check(MaterialInstance);

	VisibleExpressions.Empty();
	if (Material->IsUsingNewHLSLGenerator())
	{
		// When using the new HLSL generator, MI parameter list will already have unused parameters culled
		// We can assume that any remaining parameters are visible
		TArray<FMaterialParameterInfo> ParameterInfo;
		TArray<FGuid> ParamterGuid;
		for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
		{
			MaterialInstance->GetAllParameterInfoOfType((EMaterialParameterType)TypeIndex, ParameterInfo, ParamterGuid);
			VisibleExpressions.Append(ParameterInfo);
		}
		return;
	}

	TUniquePtr<FGetVisibleMaterialParametersFunctionState> FunctionState = MakeUnique<FGetVisibleMaterialParametersFunctionState>(nullptr);
	TArray<FGetVisibleMaterialParametersFunctionState*> FunctionStack;
	FunctionStack.Push(FunctionState.Get());

	for (uint32 i = 0; i < MP_MAX; ++i)
	{
		FExpressionInput* ExpressionInput = ((UMaterial*)Material)->GetExpressionInputForProperty((EMaterialProperty)i);

		if (ExpressionInput)
		{
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(ExpressionInput->Expression, ExpressionInput->OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}
	}

	TArray<UMaterialExpressionCustomOutput*> CustomOutputExpressions;
	Material->GetAllCustomOutputExpressions(CustomOutputExpressions);
	for (UMaterialExpressionCustomOutput* Expression : CustomOutputExpressions)
	{
		GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(Expression, 0), MaterialInstance, VisibleExpressions, FunctionStack);
	}

	TArray<UMaterialExpressionFunctionOutput*> FunctionOutputExpressions;
	Material->GetAllFunctionOutputExpressions(FunctionOutputExpressions);
	for (UMaterialExpressionFunctionOutput* Expression : FunctionOutputExpressions)
	{
		GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(Expression, 0), MaterialInstance, VisibleExpressions, FunctionStack);
	}
}

bool FMaterialEditorUtilities::GetStaticSwitchExpressionValue(UMaterialInstance* MaterialInstance, UMaterialExpression* SwitchValueExpression, bool& bOutValue, FGuid& OutExpressionID, TArray<FGetVisibleMaterialParametersFunctionState*>& FunctionStack)
{
	// Trace any re-route nodes between the input pin and the actual expression
	UMaterialExpression* TracedExpression = SwitchValueExpression;
	if (UMaterialExpressionRerouteBase* Reroute = Cast<UMaterialExpressionRerouteBase>(TracedExpression))
	{
		TracedExpression = Reroute->TraceInputsToRealInput().Expression;
	}

	// If switch value is a function input expression then we must recursively find the associated input expressions from the parent function/material to evaluate the value.
	UMaterialExpressionFunctionInput* FunctionInputExpression =  Cast<UMaterialExpressionFunctionInput>(TracedExpression);
	if(FunctionInputExpression && FunctionInputExpression->InputType == FunctionInput_StaticBool)
	{
		FGetVisibleMaterialParametersFunctionState* TopmostFunctionState = FunctionStack.Pop();
		if (TopmostFunctionState->FunctionCall)
		{
			const TArray<FFunctionExpressionInput>* FunctionInputs = &TopmostFunctionState->FunctionCall->FunctionInputs;

			// Get the FFunctionExpressionInput which stores information about the input node from the parent that this is linked to.
			const FFunctionExpressionInput* MatchingInput = FindInputById(FunctionInputExpression, *FunctionInputs);
			if (MatchingInput && (MatchingInput->Input.Expression || !FunctionInputExpression->bUsePreviewValueAsDefault))
			{
				GetStaticSwitchExpressionValue(MaterialInstance, MatchingInput->Input.Expression, bOutValue, OutExpressionID, FunctionStack);
			}
			else
			{
				GetStaticSwitchExpressionValue(MaterialInstance, FunctionInputExpression->Preview.Expression, bOutValue, OutExpressionID, FunctionStack);
			}
		}
		FunctionStack.Push(TopmostFunctionState);
	}

	if (TracedExpression)
	{
		UMaterialExpressionStaticBoolParameter* SwitchParamValue = Cast<UMaterialExpressionStaticBoolParameter>(TracedExpression);
		UMaterialExpressionStaticBool* StaticBoolValue = Cast<UMaterialExpressionStaticBool>(TracedExpression);
		UMaterialExpressionStaticSwitch* StaticSwitchValue = Cast<UMaterialExpressionStaticSwitch>(TracedExpression);

		if (SwitchParamValue)
		{
			// Use the current stack state's parameter association
			FMaterialParameterInfo ParamInfo = FunctionStack.Top()->StackParameterInfo;
			ParamInfo.Name = SwitchParamValue->ParameterName;
			MaterialInstance->GetStaticSwitchParameterValue(ParamInfo, bOutValue, OutExpressionID);
			return true;
		}
		else if (StaticBoolValue)
		{
			bOutValue = StaticBoolValue->Value;
			return true;
		}
		else if (StaticSwitchValue)
		{
			bool bSwitchValue = StaticSwitchValue->DefaultValue;
			GetStaticSwitchExpressionValue(MaterialInstance, StaticSwitchValue->Value.Expression, bSwitchValue, OutExpressionID, FunctionStack);

			if (bSwitchValue)
			{
				GetStaticSwitchExpressionValue(MaterialInstance, StaticSwitchValue->A.Expression, bOutValue, OutExpressionID, FunctionStack);
			}
			else
			{
				GetStaticSwitchExpressionValue(MaterialInstance, StaticSwitchValue->B.Expression, bOutValue, OutExpressionID, FunctionStack);
			}

			return true;
		}
	}
	
	return false;
}

const FFunctionExpressionInput* FMaterialEditorUtilities::FindInputById(const UMaterialExpressionFunctionInput* InputExpression, const TArray<FFunctionExpressionInput>& Inputs)
{
	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); InputIndex++)
	{
		const FFunctionExpressionInput& CurrentInput = Inputs[InputIndex];
		if (CurrentInput.ExpressionInputId == InputExpression->Id && CurrentInput.ExpressionInput->GetOuter() == InputExpression->GetOuter())
		{
			return &CurrentInput;
		}
	}
	return NULL;
}

void FMaterialEditorUtilities::InitExpressions(UMaterial* Material)
{
	FString ParmName;

	Material->GetExpressionCollection().Empty();

	TArray<UObject*> ChildObjects;
	GetObjectsWithOuter(Material, ChildObjects, EGetObjectsFlags::None);

	for ( int32 ChildIdx = 0; ChildIdx < ChildObjects.Num(); ++ChildIdx )
	{
		UMaterialExpression* MaterialExpression = Cast<UMaterialExpression>(ChildObjects[ChildIdx]);
		if( IsValid(MaterialExpression) )
		{
			// Comment expressions are stored in a separate list.
			if ( MaterialExpression->IsA( UMaterialExpressionComment::StaticClass() ) )
			{
				Material->GetExpressionCollection().AddComment( static_cast<UMaterialExpressionComment*>(MaterialExpression) );
			}
			else
			{
				Material->GetExpressionCollection().AddExpression( MaterialExpression );
			}
		}
	}

	Material->BuildEditorParameterList();

	// Propagate RF_Transactional to all referenced material expressions.
	Material->SetFlags( RF_Transactional );
	for(UMaterialExpression* MaterialExpression : Material->GetExpressions())
	{
		if(MaterialExpression)
		{
			MaterialExpression->SetFlags( RF_Transactional );
		}
	}
	for(UMaterialExpressionComment* Comment : Material->GetEditorComments())
	{
		Comment->SetFlags( RF_Transactional );
	}
}

///////////
// private

void FMaterialEditorUtilities::GetVisibleMaterialParametersFromExpression(
	FMaterialExpressionKey MaterialExpressionKey, 
	UMaterialInstance* MaterialInstance, 
	TArray<FMaterialParameterInfo>& VisibleExpressions, 
	TArray<FGetVisibleMaterialParametersFunctionState*>& FunctionStack)
{
	if (!MaterialExpressionKey.Expression)
	{
		return;
	}

	check(MaterialInstance);

	// Bail if we already parsed this expression
	if (FunctionStack.Top()->VisitedExpressions.Contains(MaterialExpressionKey))
	{
		return;
	}

	FunctionStack.Top()->VisitedExpressions.Add(MaterialExpressionKey);
	FunctionStack.Top()->ExpressionStack.Push(MaterialExpressionKey);
	const int32 FunctionDepth = FunctionStack.Num();
	FMaterialParameterInfo ParameterInfo = FunctionStack.Top()->StackParameterInfo;

	UMaterial* BaseMaterial = MaterialInstance->GetBaseMaterial();
	bool bCompilingFunctionPreview = BaseMaterial && BaseMaterial->bIsFunctionPreviewMaterial;

	// If it's a material parameter it must be visible so add it to the list
	UMaterialExpressionParameter* Param = Cast<UMaterialExpressionParameter>( MaterialExpressionKey.Expression );
	UMaterialExpressionTextureSampleParameter* TexParam = Cast<UMaterialExpressionTextureSampleParameter>( MaterialExpressionKey.Expression );
	UMaterialExpressionTextureCollectionParameter* TextureCollectionParam = Cast<UMaterialExpressionTextureCollectionParameter>( MaterialExpressionKey.Expression );
	UMaterialExpressionRuntimeVirtualTextureSampleParameter* RuntimeVirtualTexParam = Cast<UMaterialExpressionRuntimeVirtualTextureSampleParameter>(MaterialExpressionKey.Expression);
	UMaterialExpressionSparseVolumeTextureSampleParameter* SparseVolumeTexParam = Cast<UMaterialExpressionSparseVolumeTextureSampleParameter>(MaterialExpressionKey.Expression);
	UMaterialExpressionFontSampleParameter* FontParam = Cast<UMaterialExpressionFontSampleParameter>( MaterialExpressionKey.Expression );
	UMaterialExpressionCollectionParameter* ParameterCollectionParam = Cast<UMaterialExpressionCollectionParameter>(MaterialExpressionKey.Expression);

	if (Param)
	{
		ParameterInfo.Name = Param->ParameterName;
	}
	else if (TexParam)
	{
		ParameterInfo.Name = TexParam->ParameterName;
	}
	else if (TextureCollectionParam)
	{
		ParameterInfo.Name = TextureCollectionParam->ParameterName;
	}
	else if (RuntimeVirtualTexParam)
	{
		ParameterInfo.Name = RuntimeVirtualTexParam->ParameterName;
	}
	else if (SparseVolumeTexParam)
	{
		ParameterInfo.Name = SparseVolumeTexParam->ParameterName;
	}
	else if (FontParam)
	{
		ParameterInfo.Name = FontParam->ParameterName;
	}
	else if (ParameterCollectionParam)
	{
		ParameterInfo.Name = ParameterCollectionParam->ParameterName;
	}
		
	if (Param || TexParam || TextureCollectionParam || FontParam || RuntimeVirtualTexParam || SparseVolumeTexParam || ParameterCollectionParam)
	{
		VisibleExpressions.AddUnique(ParameterInfo);
	}

	// Check if it's a switch expression and branch according to its value
	UMaterialExpressionStaticSwitchParameter* StaticSwitchParamExpression = Cast<UMaterialExpressionStaticSwitchParameter>(MaterialExpressionKey.Expression);
	UMaterialExpressionStaticSwitch* StaticSwitchExpression = Cast<UMaterialExpressionStaticSwitch>(MaterialExpressionKey.Expression);
	UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpressionKey.Expression);
	UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(MaterialExpressionKey.Expression);
	UMaterialExpressionFunctionInput* FunctionInputExpression = Cast<UMaterialExpressionFunctionInput>(MaterialExpressionKey.Expression);

	if (StaticSwitchParamExpression)
	{
		bool Value = false;
		FGuid ExpressionID;

		MaterialInstance->GetStaticSwitchParameterValue(ParameterInfo, Value, ExpressionID);

		if (Value)
		{
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(StaticSwitchParamExpression->A.Expression, StaticSwitchParamExpression->A.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}
		else
		{
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(StaticSwitchParamExpression->B.Expression, StaticSwitchParamExpression->B.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}
	}
	else if (StaticSwitchExpression)
	{
		bool bValue = StaticSwitchExpression->DefaultValue;
		FGuid ExpressionID;

		if (StaticSwitchExpression->Value.Expression)
		{
			GetStaticSwitchExpressionValue(MaterialInstance, StaticSwitchExpression->Value.Expression, bValue, ExpressionID, FunctionStack);
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(StaticSwitchExpression->Value.Expression, StaticSwitchExpression->Value.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}

		if(bValue)
		{
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(StaticSwitchExpression->A.Expression, StaticSwitchExpression->A.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}
		else
		{
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(StaticSwitchExpression->B.Expression, StaticSwitchExpression->B.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}
	}
	else if (FunctionCallExpression)
	{
		if (FunctionCallExpression->MaterialFunction && FunctionCallExpression->FunctionOutputs.IsValidIndex(MaterialExpressionKey.OutputIndex))
		{			
			for (int32 FunctionCallIndex = 0; FunctionCallIndex < FunctionStack.Num(); FunctionCallIndex++)
			{
				checkSlow(FunctionStack[FunctionCallIndex]->FunctionCall != FunctionCallExpression);
			}

			TUniquePtr<FGetVisibleMaterialParametersFunctionState> NewFunctionState = MakeUnique<FGetVisibleMaterialParametersFunctionState>(FunctionCallExpression);
			NewFunctionState->StackParameterInfo = ParameterInfo; // Don't change back to Global parameter association when stepping into a function called from a blend/layer function
			FunctionStack.Push(NewFunctionState.Get());
		
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(FunctionCallExpression->FunctionOutputs[MaterialExpressionKey.OutputIndex].ExpressionOutput, 0), MaterialInstance, VisibleExpressions, FunctionStack);
		
			check(FunctionStack.Top()->ExpressionStack.Num() == 0);
			FunctionStack.Pop();
		}
	}
	else if (LayersExpression)
	{
		//ParameterInfo.Name = LayersExpression->ParameterName;
		//VisibleExpressions.AddUnique(ParameterInfo);

		// TODO: We only need to traverse a solo Layer[0] or the final Blend[N-1] here it will recurse anyway
		FMaterialLayersFunctions LayersValue;
		if (MaterialInstance->GetMaterialLayers(LayersValue))
		{
			LayersExpression->OverrideLayerGraph(&LayersValue);
			if (LayersExpression->bIsLayerGraphBuilt)
			{
				TArray<TObjectPtr<UMaterialExpressionMaterialFunctionCall>>& LayerCallers = LayersExpression->LayerCallers;
				TArray<TObjectPtr<UMaterialExpressionMaterialFunctionCall>>& BlendCallers = LayersExpression->BlendCallers;
				
				for (auto& Layer : LayerCallers)
				{
					// Possible that Layer->FunctionOutputs will be empty if this is a newly create layer
					if (Layer && Layer->MaterialFunction && Layer->FunctionOutputs.IsValidIndex(MaterialExpressionKey.OutputIndex))
					{
						TUniquePtr<FGetVisibleMaterialParametersFunctionState> NewFunctionState = MakeUnique<FGetVisibleMaterialParametersFunctionState>(Layer);
						FunctionStack.Push(NewFunctionState.Get());
						GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(Layer->FunctionOutputs[MaterialExpressionKey.OutputIndex].ExpressionOutput, 0), MaterialInstance, VisibleExpressions, FunctionStack);

						check(FunctionStack.Top()->ExpressionStack.Num() == 0);
						FunctionStack.Pop();
					}
				}

				for (auto& Blend : BlendCallers)
				{
					if (Blend && Blend->MaterialFunction && Blend->FunctionOutputs.IsValidIndex(MaterialExpressionKey.OutputIndex))
					{
						TUniquePtr<FGetVisibleMaterialParametersFunctionState> NewFunctionState = MakeUnique<FGetVisibleMaterialParametersFunctionState>(Blend);
						FunctionStack.Push(NewFunctionState.Get());

						GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(Blend->FunctionOutputs[MaterialExpressionKey.OutputIndex].ExpressionOutput, 0), MaterialInstance, VisibleExpressions, FunctionStack);

						check(FunctionStack.Top()->ExpressionStack.Num() == 0);
						FunctionStack.Pop();
					}
				}
			}

			LayersExpression->OverrideLayerGraph(nullptr);
		}
	}
	else if (FunctionInputExpression && FunctionStack.Num() > 1)
	{
		GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(FunctionInputExpression->Preview.Expression, FunctionInputExpression->Preview.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		
		FGetVisibleMaterialParametersFunctionState* FunctionState = FunctionStack.Pop();

		const FFunctionExpressionInput* MatchingInput = FindInputById(FunctionInputExpression, FunctionState->FunctionCall->FunctionInputs);
		check(MatchingInput);
		
		GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(MatchingInput->Input.Expression, MatchingInput->Input.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);

		FunctionStack.Push(FunctionState);
	}
	else
	{
		// If this is a reroute node of any type, we trace to the first available 'real' input and traverse that single input 
		if (const UMaterialExpressionRerouteBase* Reroute = Cast<UMaterialExpressionRerouteBase>(MaterialExpressionKey.Expression))
		{
			FExpressionInput Input = Reroute->TraceInputsToRealInput();
			GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(Input.Expression, Input.OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
		}
		else
		{
			// Retrieve the expression input and then start parsing its children
			for (FExpressionInputIterator It{ MaterialExpressionKey.Expression }; It; ++It)
			{
				GetVisibleMaterialParametersFromExpression(FMaterialExpressionKey(It->Expression, It->OutputIndex), MaterialInstance, VisibleExpressions, FunctionStack);
			}
		}
	}

	FMaterialExpressionKey TopExpressionKey = FunctionStack.Top()->ExpressionStack.Pop();
	check(FunctionDepth == FunctionStack.Num());
	//ensure that the top of the stack matches what we expect (the same as MaterialExpressionKey)
	check(MaterialExpressionKey == TopExpressionKey);
}

TSharedPtr<class IMaterialEditor> FMaterialEditorUtilities::GetIMaterialEditorForObject(const UObject* ObjectToFocusOn)
{
	check(ObjectToFocusOn);

	// Find the associated Material
	UMaterial* Material = Cast<UMaterial>(ObjectToFocusOn->GetOuter());

	// May be inspecting a subgraph, in which case, get the material from the composite node 
	if (!Material)
	{
		if (UMaterialGraphNode_Composite* Composite = Cast<UMaterialGraphNode_Composite>(ObjectToFocusOn->GetOuter()))
		{
			Material = Composite->MaterialExpression->Material;
		}
	}

	TSharedPtr<IMaterialEditor> MaterialEditor;
	if (Material != NULL)
	{
		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Material);
		if (FoundAssetEditor.IsValid())
		{
			MaterialEditor = StaticCastSharedPtr<IMaterialEditor>(FoundAssetEditor);
		}
	}
	return MaterialEditor;
}

void FMaterialEditorUtilities::BringFocusAttentionOnObject(const UObject* ObjectToFocusOn)
{
	TSharedPtr<IMaterialEditor> MaterialEditor = GetIMaterialEditorForObject(ObjectToFocusOn);
	if (MaterialEditor.IsValid())
	{
		MaterialEditor->FocusWindow();
		MaterialEditor->JumpToHyperlink(ObjectToFocusOn);
	}
}

void FMaterialEditorUtilities::AddMaterialExpressionCategory(FGraphActionMenuBuilder& ActionMenuBuilder, FText CategoryName, TArray<struct FMaterialExpression>* MaterialExpressions, bool bMaterialFunction)
{
	UObject* MaterialOrFunction = bMaterialFunction ? UMaterialFunction::StaticClass()->GetDefaultObject() : UMaterial::StaticClass()->GetDefaultObject();
	AddMaterialExpressionCategory(ActionMenuBuilder, CategoryName, MaterialExpressions, MaterialOrFunction);
}

void FMaterialEditorUtilities::AddMaterialExpressionCategory(FGraphActionMenuBuilder& ActionMenuBuilder, FText CategoryName, TArray<struct FMaterialExpression>* MaterialExpressions, const UObject* MaterialOrFunction)
{
	// Get type of dragged pin
	EMaterialValueType FromPinType{};
	if (ActionMenuBuilder.FromPin)
	{
		FromPinType = UMaterialGraphSchema::GetMaterialIOValueType(ActionMenuBuilder.FromPin);
	}

	bool bMaterialFunction = Cast<UMaterialFunction>(MaterialOrFunction) != nullptr;

	for (int32 Index = 0; Index < MaterialExpressions->Num(); ++Index)
	{
		const FMaterialExpression& MaterialExpression = (*MaterialExpressions)[Index];
		UMaterialExpression* DefaultExpression = CastChecked<UMaterialExpression>(MaterialExpression.MaterialClass->GetDefaultObject());
		if (DefaultExpression->IsAllowedIn(MaterialOrFunction) && MaterialExpression.MaterialClass != UMaterialExpressionComposite::StaticClass())
		{
			if (!ActionMenuBuilder.FromPin || HasCompatibleConnection(MaterialExpression.MaterialClass, FromPinType, ActionMenuBuilder.FromPin->Direction, bMaterialFunction))
			{
				FText CreationName = FText::FromString(MaterialExpression.Name);
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), CreationName);
				FText ToolTip = FText::Format(LOCTEXT("NewMaterialExpressionTooltip", "Adds a {Name} node here"), Arguments);
				if (!MaterialExpression.CreationDescription.IsEmpty())
				{
					ToolTip = MaterialExpression.CreationDescription;
				}
				if (!MaterialExpression.CreationName.IsEmpty())
				{
					CreationName = MaterialExpression.CreationName;
				}

				UMaterialExpression* MaterialExpressionDefaultObject = CastChecked<UMaterialExpression>(MaterialExpression.MaterialClass->GetDefaultObject());

				TSharedPtr<FMaterialGraphSchemaAction_NewNode> NewNodeAction(new FMaterialGraphSchemaAction_NewNode(
					CategoryName,
					CreationName,
					ToolTip,
					0, 
					MaterialExpressionDefaultObject->GetKeywords()
				));
				NewNodeAction->MaterialExpressionClass = MaterialExpression.MaterialClass;
				ActionMenuBuilder.AddAction(NewNodeAction);

				// Register Any Additional Menu Actions for this Expression
				MaterialExpressionDefaultObject->RegisterAdditionalMenuActions(ActionMenuBuilder, CategoryName);
			}
		}
	}
}

bool FMaterialEditorUtilities::HasCompatibleConnection(UClass* ExpressionClass, uint32 TestType, EEdGraphPinDirection TestDirection, bool bMaterialFunction)
{
	return HasCompatibleConnection(ExpressionClass, static_cast<EMaterialValueType>(TestType), TestDirection, bMaterialFunction);
}

bool FMaterialEditorUtilities::HasCompatibleConnection(UClass* ExpressionClass, EMaterialValueType TestType, EEdGraphPinDirection TestDirection, bool bMaterialFunction)
{
	if (TestType != 0)
	{
		UMaterialExpression* DefaultExpression = CastChecked<UMaterialExpression>(ExpressionClass->GetDefaultObject());
		if (TestDirection == EGPD_Output)
		{
			for (FExpressionInputIterator It{ DefaultExpression }; It; ++It)
			{
				EMaterialValueType InputType = DefaultExpression->GetInputValueType(It.Index);
				if (CanConnectMaterialValueTypes(InputType, TestType))
				{
					return true;
				}
			}
		}
		else
		{
			int32 NumOutputs = DefaultExpression->GetOutputs().Num();
			for (int32 Index = 0; Index < NumOutputs; ++Index)
			{
				EMaterialValueType OutputType = DefaultExpression->GetOutputValueType(Index);
				if (CanConnectMaterialValueTypes(TestType, OutputType))
				{
					return true;
				}
			}
		}
		
		if (bMaterialFunction)
		{
			// Specific test as Default object won't have texture input
			if (ExpressionClass == UMaterialExpressionTextureSample::StaticClass() && TestType & MCT_Texture && TestDirection == EGPD_Output)
			{
				return true;
			}
			// Always allow creation of new inputs as they can take any type
			else if (ExpressionClass == UMaterialExpressionFunctionInput::StaticClass())
			{
				return true;
			}
			// Allow creation of outputs for floats and material attributes
			else if (ExpressionClass == UMaterialExpressionFunctionOutput::StaticClass() && TestType & (MCT_Float|MCT_MaterialAttributes))
			{
				return true;
			}
		}
	}

	return false;
}

void FMaterialEditorUtilities::BuildTextureStreamingData(UMaterialInterface* UpdatedMaterial)
{
	const EMaterialQualityLevel::Type QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	const EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;

	if (UpdatedMaterial)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		FScopedSlowTask SlowTask(2.f, (LOCTEXT("MaterialEditorUtilities_UpdatingTextureStreamingData", "Updating Texture Streaming Data")));
		SlowTask.MakeDialog(true);

		// Clear the build data.
		const TArray<FMaterialTextureInfo> EmptyTextureStreamingData;
		UpdatedMaterial->SetTextureStreamingData(EmptyTextureStreamingData);

		// Skip compilation for cooked materials
		UMaterial* RootMaterial = UpdatedMaterial->GetMaterial();
		if (!RootMaterial || !RootMaterial->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
		{
			TSet<UMaterialInterface*> Materials;
			Materials.Add(UpdatedMaterial);

			if (CompileDebugViewModeShaders(DVSM_OutputMaterialTextureScales, QualityLevel, FeatureLevel, Materials, &SlowTask))
			{
				FMaterialUtilities::FExportErrorManager ExportErrors(ShaderPlatform);
				for (UMaterialInterface* MaterialInterface : Materials)
				{
					FMaterialUtilities::ExportMaterialUVDensities(MaterialInterface, QualityLevel, ShaderPlatform, ExportErrors);
				}
				ExportErrors.OutputToLog();

				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			}
		}
	}
}


void FMaterialEditorUtilities::OnOpenMaterial(const FAssetData InMaterial)
{
	UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(InMaterial.GetAsset());
	OpenSelectedParentEditor(MaterialInterface);
}

void FMaterialEditorUtilities::OnOpenFunction(const FAssetData InFunction)
{
	UMaterialFunctionInterface* MaterialFunctionInterface = Cast<UMaterialFunctionInterface>(InFunction.GetAsset());
	OpenSelectedParentEditor(MaterialFunctionInterface);
}

void FMaterialEditorUtilities::OnShowMaterialInContentBrowser(const FAssetData InMaterial)
{
	TArray<UObject*> SyncedObject;
	SyncedObject.Add(InMaterial.GetAsset());
	GEditor->SyncBrowserToObjects(SyncedObject);
}

void FMaterialEditorUtilities::OnShowFunctionInContentBrowser(const FAssetData InFunction)
{
	TArray<UObject*> SyncedObject;
	SyncedObject.Add(InFunction.GetAsset());
	GEditor->SyncBrowserToObjects(SyncedObject);
}

void FMaterialEditorUtilities::OpenSelectedParentEditor(UMaterialInterface* InMaterialInterface)
{
	// See if its a material or material instance constant. 
	if (ensure(InMaterialInterface))
	{
		if (InMaterialInterface->IsA(UMaterial::StaticClass()))
		{
			// Show material editor
			UMaterial* Material = Cast<UMaterial>(InMaterialInterface);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Material);
		}
		else if (InMaterialInterface->IsA(UMaterialInstance::StaticClass()))
		{
			// Show material instance editor
			UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(InMaterialInterface);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(MaterialInstance);
		}
	}
}

void FMaterialEditorUtilities::OpenSelectedParentEditor(UMaterialFunctionInterface* InMaterialFunction)
{
	// See if its a material or material instance constant.  
	if (ensure(InMaterialFunction) )
	{
		if (InMaterialFunction->IsA(UMaterialFunctionInstance::StaticClass()))
		{
			// Show function instance editor
			UMaterialFunctionInstance* FunctionInstance = Cast<UMaterialFunctionInstance>(InMaterialFunction);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(FunctionInstance);
		}
		else
		{
			// Show function editor
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InMaterialFunction);
		}
	}
}

void FMaterialEditorUtilities::RefreshPostProcessPreviewMaterials(UMaterialInterface* ExcludeMaterialInterface, bool bRedrawOnly)
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();
	for (UObject* EditedAsset : EditedAssets)
	{
		UPreviewMaterial* EditedPreviewMaterial = Cast<UPreviewMaterial>(EditedAsset);
		if (EditedPreviewMaterial && EditedPreviewMaterial != ExcludeMaterialInterface)
		{
			UMaterial* EditedMaterial = EditedPreviewMaterial->GetMaterial();
			if (EditedMaterial->IsPostProcessMaterial())
			{
				TArray<IAssetEditorInstance*> Editors = AssetEditorSubsystem->FindEditorsForAsset(EditedAsset);
				for (IAssetEditorInstance* Editor : Editors)
				{
					if (Editor->GetEditorName() == FName("MaterialEditor"))
					{
						FMaterialEditor* MaterialEditor = (FMaterialEditor*)Editor;
						if (bRedrawOnly)
						{
							MaterialEditor->RefreshPreviewViewport();
						}
						else
						{
							// Calling "SetPreviewMaterial" will refresh the other editor
							MaterialEditor->SetPreviewMaterial(EditedPreviewMaterial);
						}
					}
				}
			}
		}

		UMaterialInstanceConstant* EditedMaterialInstance = Cast<UMaterialInstanceConstant>(EditedAsset);
		if (EditedMaterialInstance && EditedMaterialInstance != ExcludeMaterialInterface)
		{
			UMaterial* BaseMaterial = EditedMaterialInstance->GetBaseMaterial();
			if (BaseMaterial && BaseMaterial->IsPostProcessMaterial())
			{
				TArray<IAssetEditorInstance*> Editors = AssetEditorSubsystem->FindEditorsForAsset(EditedAsset);
				for (IAssetEditorInstance* Editor : Editors)
				{
					if (Editor->GetEditorName() == FName("MaterialInstanceEditor"))
					{
						FMaterialInstanceEditor* MaterialEditor = (FMaterialInstanceEditor*)Editor;
						if (bRedrawOnly)
						{
							MaterialEditor->RefreshPreviewViewport();
						}
						else
						{
							// Calling "SetPreviewMaterial" will refresh the other editor
							MaterialEditor->SetPreviewMaterial(EditedMaterialInstance);
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
