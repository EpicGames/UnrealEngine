// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorValidation.h"

#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialEditorValidation)

void UMaterialEditorValidationSubsystem::AddValidator(UMaterialEditorValidatorBase* InValidator)
{
	Validators.AddUnique(InValidator);
}

void UMaterialEditorValidationSubsystem::RemoveValidator(UMaterialEditorValidatorBase* InValidator)
{
	Validators.Remove(InValidator);
}

bool UMaterialEditorValidationSubsystem::Validate(UMaterialInterface* InMaterial, FMaterialEditorValidatorContext& InContext) const
{
	for (UMaterialEditorValidatorBase* Validator : Validators)
	{
		if (Validator != nullptr && !Validator->Validate(InMaterial, InContext))
		{
			// Return false on first validation failure.
			return false;
		}
	}
	return true;
}
