// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditorViewportToolbarSections.h"
#include "EditorViewportCommands.h"
#include "MaterialEditorActions.h"
#include "Materials/Material.h"
#include "SEditorViewport.h"
#include "SMaterialEditorViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Styling/SlateIconFinder.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

#define LOCTEXT_NAMESPACE "MaterialEditorViewportToolbarSections"

namespace UE::MaterialEditor::Private
{

void FillShowSubmenu(UToolMenu* InMenu, bool bInShowViewportStatsToggle)
{
	if (UUnrealEdViewportToolbarContext* const EditorViewportContext =
			InMenu->FindContext<UUnrealEdViewportToolbarContext>())
	{
		if (TSharedPtr<SMaterialEditor3DPreviewViewport> StaticMeshEditorViewport =
				StaticCastSharedPtr<SMaterialEditor3DPreviewViewport>(EditorViewportContext->Viewport.Pin()))
		{
			FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);

			if (bInShowViewportStatsToggle)
			{
				UnnamedSection.AddMenuEntry(
					FEditorViewportCommands::Get().ToggleStats, LOCTEXT("ViewportStatsLabel", "Viewport Stats")
				);

				UnnamedSection.AddSeparator(NAME_None);
			}

			UnnamedSection.AddMenuEntry(FMaterialEditorCommands::Get().ToggleMaterialStats);
		}
	}
}
	
}

TSharedRef<SWidget> UE::MaterialEditor::CreateShowMenuWidget(const TSharedRef<SMaterialEditor3DPreviewViewport>& InMaterialEditorViewport, bool bInShowViewportStatsToggle
)
{
	InMaterialEditorViewport->OnFloatingButtonClicked();

	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldShowMenuName = "MaterialEditor.OldViewportToolbar.Show";

	if (!UToolMenus::Get()->IsMenuRegistered(OldShowMenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldShowMenuName, NAME_None, EMultiBoxType::Menu, false);
		Menu->AddDynamicSection(
			"BaseSection",
			FNewToolMenuDelegate::CreateLambda(
				[ViewportWeak = InMaterialEditorViewport.ToWeakPtr(), bInShowViewportStatsToggle](UToolMenu* InMenu)
				{
					if (TSharedPtr<SMaterialEditor3DPreviewViewport> Viewport = ViewportWeak.Pin())
					{
						UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
						ContextObject->Viewport = ViewportWeak;
						InMenu->Context.AddObject(ContextObject);

						Private::FillShowSubmenu(InMenu, bInShowViewportStatsToggle);
					}
				}
			)
		);
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InMaterialEditorViewport->GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(InMaterialEditorViewport);

			MenuContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(OldShowMenuName, MenuContext);
}

void UE::MaterialEditor::ExtendPreviewSceneSettingsSubmenu(FName InSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InSubmenuName);
	if (!Submenu)
	{
		return;
	}

	FToolMenuSection& PreviewMeshSection = Submenu->FindOrAddSection(
		"AssetViewerPreviewMeshSection", LOCTEXT("AssetViewerPreviewMeshSectionLabel", "Preview Mesh Options")
	);
	
	FToolMenuEntryToolBarData ToolbarData;
	ToolbarData.BlockGroupName = "PreviewMeshOptions";
	ToolbarData.ResizeParams.ClippingPriority = 2000;
	
	const FMaterialEditorCommands& MaterialEditorCommands = FMaterialEditorCommands::Get();
	const TArray<TSharedPtr<FUICommandInfo>> PreviewMeshCommands
	{
		MaterialEditorCommands.SetSpherePreview,
		MaterialEditorCommands.SetCylinderPreview,
		MaterialEditorCommands.SetPlanePreview,
		MaterialEditorCommands.SetCubePreview,
		MaterialEditorCommands.SetShaderBallPreview
	};

	for (const TSharedPtr<FUICommandInfo>& Command : PreviewMeshCommands)
	{
		FToolMenuEntry& Entry = PreviewMeshSection.AddMenuEntry(Command);
		Entry.SetShowInToolbarTopLevel(true);
		Entry.ToolBarData = ToolbarData;
	}

	FToolMenuEntry& MeshFromSelectionEntry =
		PreviewMeshSection.AddMenuEntry(MaterialEditorCommands.SetPreviewMeshFromSelection);
	MeshFromSelectionEntry.Label = LOCTEXT("SetPreviewMeshFromSelectionLabel", "Static Mesh in Content Browser");
	MeshFromSelectionEntry.Icon = FSlateIconFinder::FindIconForClass(UStaticMesh::StaticClass());
	MeshFromSelectionEntry.SetShowInToolbarTopLevel(true);
	MeshFromSelectionEntry.ToolBarData = ToolbarData;

	FToolMenuEntry& MaterialSlotsEntry =
		PreviewMeshSection.AddMenuEntry(FMaterialEditorCommands::Get().SelectPreviewMeshMaterialSlots);
	MaterialSlotsEntry.Label = LOCTEXT("SelectPreviewMeshMaterialSlotsLabel", "Mesh material slots to use when previewing material");
	MaterialSlotsEntry.Icon = FSlateIconFinder::FindIconForClass(UMaterial::StaticClass());
	MaterialSlotsEntry.SetShowInToolbarTopLevel(true);
	MaterialSlotsEntry.ToolBarData = ToolbarData;
}

#undef LOCTEXT_NAMESPACE
