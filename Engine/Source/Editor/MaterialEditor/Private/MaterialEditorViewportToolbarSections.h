// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "ToolMenuEntry.h"

class SMaterialEditor3DPreviewViewport;

namespace UE::MaterialEditor
{
TSharedRef<SWidget> CreateShowMenuWidget(const TSharedRef<SMaterialEditor3DPreviewViewport>& InMaterialEditorViewport, bool bInShowViewportStatsToggle = true);
void ExtendPreviewSceneSettingsSubmenu(FName InSubmenuName);
}
