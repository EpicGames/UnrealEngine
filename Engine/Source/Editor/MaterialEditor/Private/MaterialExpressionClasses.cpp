// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionClasses.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Materials/MaterialExpression.h"
#include "UObject/Package.h"
#include "MaterialEditor.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Modules/ModuleManager.h"
#include "Preferences/MaterialEditorOptions.h"
#include "MaterialEditorModule.h"
#include "MaterialShared.h"

#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionMaterialLayerOutput.h"
#include "Materials/MaterialExpressionNamedReroute.h"

#include "Materials/MaterialExpressionIfThenElse.h"
#include "Materials/MaterialExpressionSubstrate.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// MaterialExpressionClasses
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

MaterialExpressionClasses::MaterialExpressionClasses()
	: bInitialized( false )
{

}

MaterialExpressionClasses::~MaterialExpressionClasses()
{

}

MaterialExpressionClasses* MaterialExpressionClasses::Get()
{
	static MaterialExpressionClasses Inst;
	Inst.InitMaterialExpressionClasses();
	return &Inst;
}

const UStruct* MaterialExpressionClasses::GetExpressionInputStruct()
{
	static const UStruct* ExpressionInputStruct =
		CastChecked<UStruct>( StaticFindObject( UStruct::StaticClass(), nullptr, TEXT("/Script/Engine.ExpressionInput")) );
	check( ExpressionInputStruct );
	return ExpressionInputStruct;
}

FCategorizedMaterialExpressionNode* MaterialExpressionClasses::GetCategoryNode(const FText& InCategoryName, bool bCreate)
{
	for (int32 CheckIndex = 0; CheckIndex < CategorizedExpressionClasses.Num(); CheckIndex++)
	{
		FCategorizedMaterialExpressionNode& CheckNode = CategorizedExpressionClasses[CheckIndex];
		if (CheckNode.CategoryName.EqualTo(InCategoryName))
		{
			return &CheckNode;
		}
	}

	if (bCreate == true)
	{
		FCategorizedMaterialExpressionNode* NewNode = new(CategorizedExpressionClasses)FCategorizedMaterialExpressionNode;
		check(NewNode);

		NewNode->CategoryName = InCategoryName;
		return NewNode;
	}

	return NULL;
}

void MaterialExpressionClasses::InitMaterialExpressionClasses()
{
	if(!bInitialized)
	{
		static const auto CVarMaterialTranslatorEnableNew = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.Material.Translator.EnableNew"));
		const bool bMaterialTranslatorEnableNew = CVarMaterialTranslatorEnableNew->GetValueOnAnyThread();

		UMaterialEditorOptions* EditorOptions = UMaterialEditorOptions::Get();
		UClass* BaseType = UMaterialExpression::StaticClass();
		if( BaseType )
		{
			TArray<FStructProperty*>	ExpressionInputs;
			const UStruct*				ExpressionInputStruct = GetExpressionInputStruct();

			for( TObjectIterator<UClass> It ; It ; ++It )
			{
				UClass* Class = *It;
				if( !Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated) )
				{
					if( Class->IsChildOf(UMaterialExpression::StaticClass()) )
					{
						ExpressionInputs.Empty();

						if (Class == UMaterialExpressionMaterialLayerOutput::StaticClass())
						{
							continue;
						}

						if (!bMaterialTranslatorEnableNew && Class->HasMetaData("NewMaterialTranslator"))
						{
							continue;
						}

						// Hide node types that are tagged private
						if(Class->HasMetaData(TEXT("Private")))
						{
							continue;
						}

						// Exclude comments from the expression list, as well as the base parameter expression and local variable usage, as they should not be used directly
						if ( Class != UMaterialExpressionComment::StaticClass() 
							&& Class != UMaterialExpressionParameter::StaticClass()
							&& Class != UMaterialExpressionNamedRerouteUsage::StaticClass() )
						{
							FMaterialExpression MaterialExpression;
							// Trim the material expression name and add it to the list used for filtering.
							static const FString ExpressionPrefix = TEXT("MaterialExpression");
							FString ClassName = *Class->GetName();

							if (Class->HasMetaData("DisplayName"))
							{
								ClassName = Class->GetDisplayNameText().ToString();
							}

							if (ClassName.StartsWith(ExpressionPrefix, ESearchCase::CaseSensitive))
							{
								ClassName.MidInline(ExpressionPrefix.Len(), MAX_int32, EAllowShrinking::No);
							}
							MaterialExpression.Name = ClassName;
							MaterialExpression.MaterialClass = Class;
							UMaterialExpression* TempObject = Cast<UMaterialExpression>(Class->GetDefaultObject());
							if (TempObject)
							{
								MaterialExpression.CreationDescription = TempObject->GetCreationDescription();
								MaterialExpression.CreationName = TempObject->GetCreationName();
							}

							AllExpressionClasses.Add(MaterialExpression);

							// Initialize the expression class input map.							
							for( TFieldIterator<FStructProperty> InputIt(Class) ; InputIt ; ++InputIt )
							{
								FStructProperty* StructProp = *InputIt;
								if( StructProp->Struct == ExpressionInputStruct )
								{
									ExpressionInputs.Add( StructProp );
								}
							}

							// See if it is in the favorites array...
							for (int32 FavoriteIndex = 0; FavoriteIndex < EditorOptions->FavoriteExpressions.Num(); FavoriteIndex++)
							{
								if (Class->GetName() == EditorOptions->FavoriteExpressions[FavoriteIndex])
								{
									FavoriteExpressionClasses.AddUnique(MaterialExpression);
								}
							}

							// Skip adding Substrate node to the context menu if Substrate is disabled
							// SUBSTRATE_TODO: remove this when Substrate becomes the only shading path
							if ((Class->IsChildOf(UMaterialExpressionSubstrateBSDF::StaticClass()) || Class->IsChildOf(UMaterialExpressionSubstrateUtilityBase::StaticClass())) && !Substrate::IsSubstrateEnabled())
							{
								continue;
							}

							// Category fill...
							if (TempObject)
							{
								if (TempObject->MenuCategories.Num() == 0)
								{
									UnassignedExpressionClasses.Add(MaterialExpression);
								}
								else
								{
									for (int32 CategoryIndex = 0; CategoryIndex < TempObject->MenuCategories.Num(); CategoryIndex++)
									{
										FCategorizedMaterialExpressionNode* CategoryNode = GetCategoryNode(TempObject->MenuCategories[CategoryIndex], true);
										check(CategoryNode);

										CategoryNode->MaterialExpressions.AddUnique(MaterialExpression);
									}
								}
							}
						}
					}
				}
			}
		}

		struct FCompareFMaterialExpression
		{
			FORCEINLINE bool operator()( const FMaterialExpression& A, const FMaterialExpression& B ) const
			{
				return A.Name < B.Name;
			}
		};
		AllExpressionClasses.Sort(FCompareFMaterialExpression());
		struct FCompareFCategorizedMaterialExpressionNode
		{
			FORCEINLINE bool operator()( const FCategorizedMaterialExpressionNode& A, const FCategorizedMaterialExpressionNode& B ) const
			{
				return A.CategoryName.CompareTo(B.CategoryName) < 0;
			}
		};
		CategorizedExpressionClasses.Sort( FCompareFCategorizedMaterialExpressionNode() );

		bInitialized = true;
	}
}

bool MaterialExpressionClasses::IsMaterialExpressionInFavorites(UMaterialExpression* InExpression)
{
	for (int32 CheckIndex = 0; CheckIndex < FavoriteExpressionClasses.Num(); CheckIndex++)
	{
		if (FavoriteExpressionClasses[CheckIndex].MaterialClass == InExpression->GetClass())
		{
			return true;
		}
	}

	return false;
}

void MaterialExpressionClasses::RemoveMaterialExpressionFromFavorites(UClass* InExpression)
{
	for (int32 i = 0; i < FavoriteExpressionClasses.Num(); ++i)
	{
		if (FavoriteExpressionClasses[i].MaterialClass == InExpression)
		{
			FavoriteExpressionClasses.RemoveAt(i);
		}
	}
}

void MaterialExpressionClasses::AddMaterialExpressionToFavorites(UClass* InExpression)
{
	bool bIsUnique = true;
	for (int32 i = 0; i < FavoriteExpressionClasses.Num(); ++i)
	{
		if (FavoriteExpressionClasses[i].MaterialClass == InExpression)
		{
			bIsUnique = false;
			break;
		}
	}
	if (bIsUnique)
	{
		FMaterialExpression MaterialExpression;
		// Trim the material expression name and add it to the list used for filtering.
		MaterialExpression.Name = FString(*InExpression->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
		MaterialExpression.MaterialClass = InExpression;

		FavoriteExpressionClasses.Add(MaterialExpression);
	}
}
