// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMaterialExpression;

class MaterialExpressionClasses
{
public:
	/** Gets singleton instance */
	static MaterialExpressionClasses* Get();

	/**
	 *	Checks if the given expression is in the favorites list...
	 *
	 *	@param	InExpression	The expression to check for.
	 *
	 *	@return	bool			true if it's in the list, false if not.
	 */
	bool IsMaterialExpressionInFavorites(UMaterialExpression* InExpression);
	
	/**
	 * Remove the expression from the favorites menu list.
	 */
	void RemoveMaterialExpressionFromFavorites(UClass* InExpression);

	/**
	 * Add the expression to the favorites menu list.
	 */
	void AddMaterialExpressionToFavorites(UClass* InExpression);

	/** array of UMaterialExpression-derived classes, shared between all material editor instances. */
	TArray<struct FMaterialExpression> AllExpressionClasses;
	TArray<struct FMaterialExpression> FavoriteExpressionClasses;

	/** array of UMaterialExpression-derived classes, shared between all material editor instances. */
	TArray<struct FCategorizedMaterialExpressionNode> CategorizedExpressionClasses;
	TArray<struct FMaterialExpression> UnassignedExpressionClasses;

private:
	MaterialExpressionClasses();

	virtual ~MaterialExpressionClasses();

	/**
	 * 
	 */
	const UStruct* GetExpressionInputStruct();

	/** 
	 *	Grab the expression array for the given category.
	 *	
	 *	@param	InCategoryName	The category to retrieve
	 *	@param	bCreate			If true, create the entry if not found.
	 *
	 *	@return	The category node.
	 */
	struct FCategorizedMaterialExpressionNode* GetCategoryNode(const FText& InCategoryName, bool bCreate);

	/**
	 * Initializes the list of UMaterialExpression-derived classes shared between all material editor instances.
	 */
	void InitMaterialExpressionClasses();

	/** 
	 * true if the list of UMaterialExpression-derived classes shared between material
	 * editors has already been created.
	 */
	bool bInitialized;
};
