// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialGraphConnectionDrawingPolicy.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditorSettings.h"
#include "Layout/ArrangedWidget.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialGraph/MaterialShaderValueTypeObject.h"
#include "MaterialGraphNode_Knot.h"
#include "Math/Color.h"
#include "SGraphPin.h"
#include "Shader/ShaderTypes.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "MaterialEditor/SGraphSubstrateMaterial.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class FSlateRect;

/////////////////////////////////////////////////////
// FMaterialGraphConnectionDrawingPolicy

FMaterialGraphConnectionDrawingPolicy::FMaterialGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, MaterialGraph(CastChecked<UMaterialGraph>(InGraphObj))
	, MaterialGraphSchema(CastChecked<UMaterialGraphSchema>(InGraphObj->GetSchema()))
{
	// Don't want to draw ending arrowheads
	ArrowImage = nullptr;
	ArrowRadius = FVector2f::ZeroVector;

	// Still need to be able to perceive the graph while dragging connectors, esp over comment boxes
	HoverDeemphasisDarkFraction = 0.4f;
}

bool FMaterialGraphConnectionDrawingPolicy::FindPinCenter(UEdGraphPin* Pin, FVector2f& OutCenter) const
{
	if (const TSharedPtr<SGraphPin>* pPinWidget = PinToPinWidgetMap.Find(Pin))
	{
		if (FArrangedWidget* pPinEntry = PinGeometries->Find((*pPinWidget).ToSharedRef()))
		{
			OutCenter = FGeometryHelper::CenterOf(pPinEntry->Geometry);
			return true;
		}
	}

	return false;
}

bool FMaterialGraphConnectionDrawingPolicy::GetAverageConnectedPosition(class UMaterialGraphNode_Knot* Knot, EEdGraphPinDirection Direction, FVector2f& OutPos) const
{
	FVector2f Result = FVector2f::ZeroVector;
	int32 ResultCount = 0;

	UEdGraphPin* Pin = (Direction == EGPD_Input) ? Knot->GetInputPin() : Knot->GetOutputPin();
	for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
	{
		FVector2f CenterPoint;
		if (FindPinCenter(LinkedPin, /*out*/ CenterPoint))
		{
			Result += CenterPoint;
			ResultCount++;
		}
	}

	if (ResultCount > 0)
	{
		OutPos = Result * (1.0f / static_cast<float>(ResultCount));
		return true;
	}
	else
	{
		return false;
	}
}

bool FMaterialGraphConnectionDrawingPolicy::ShouldChangeTangentForKnot(UMaterialGraphNode_Knot* Knot)
{
	if (bool* pResult = KnotToReversedDirectionMap.Find(Knot))
	{
		return *pResult;
	}
	else
	{
		bool bPinReversed = false;

		FVector2f AverageLeftPin;
		FVector2f AverageRightPin;
		FVector2f CenterPin;
		bool bCenterValid = FindPinCenter(Knot->GetOutputPin(), /*out*/ CenterPin);
		bool bLeftValid = GetAverageConnectedPosition(Knot, EGPD_Input, /*out*/ AverageLeftPin);
		bool bRightValid = GetAverageConnectedPosition(Knot, EGPD_Output, /*out*/ AverageRightPin);

		if (bLeftValid && bRightValid)
		{
			bPinReversed = AverageRightPin.X < AverageLeftPin.X;
		}
		else if (bCenterValid)
		{
			if (bLeftValid)
			{
				bPinReversed = CenterPin.X < AverageLeftPin.X;
			}
			else if (bRightValid)
			{
				bPinReversed = AverageRightPin.X < CenterPin.X;
			}
		}

		KnotToReversedDirectionMap.Add(Knot, bPinReversed);

		return bPinReversed;
	}
}

void FMaterialGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	Params.WireColor = MaterialGraphSchema->ActivePinColor;

	if (Substrate::IsSubstrateEnabled() && (FSubstrateWidget::HasOutputSubstrateType(OutputPin) || FSubstrateWidget::HasInputSubstrateType(InputPin) || FSubstrateWidget::HasInputSubstrateType(OutputPin)))
	{
		Params.WireColor = FSubstrateWidget::GetConnectionColor();
	}

	bool bInactivePin = false;
	bool bExecPin = false;

	// Have to consider both pins as the input will be an 'output' when previewing a connection
	if (OutputPin)
	{
		if (!MaterialGraph->IsInputActive(OutputPin))
		{
			bInactivePin = true;
		}

		if (OutputPin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
		{
			bExecPin = true;
		}

		UEdGraphNode* OutputNode = OutputPin->GetOwningNode();
		if (UMaterialGraphNode_Knot* OutputKnotNode = Cast<UMaterialGraphNode_Knot>(OutputNode))
		{
			if (ShouldChangeTangentForKnot(OutputKnotNode))
			{
				Params.StartDirection = EGPD_Input;
			}
		}
		else if (!OutputNode->IsNodeEnabled() || OutputNode->IsDisplayAsDisabledForced() || OutputNode->IsNodeUnrelated())
		{
			bInactivePin = true;
		}
	}
	if (InputPin)
	{
		if (!MaterialGraph->IsInputActive(InputPin))
		{
			bInactivePin = true;
		}

		if (InputPin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
		{
			bExecPin = true;
		}

		UEdGraphNode* InputNode = InputPin->GetOwningNode();
		if (UMaterialGraphNode_Knot* InputKnotNode = Cast<UMaterialGraphNode_Knot>(InputNode))
		{
			if (ShouldChangeTangentForKnot(InputKnotNode))
			{
				Params.EndDirection = EGPD_Output;
			}
		}
		else if (!InputNode->IsNodeEnabled() || InputNode->IsDisplayAsDisabledForced() || InputNode->IsNodeUnrelated())
		{
			bInactivePin = true;
		}
	}

	if (bInactivePin)
	{
		Params.WireColor = MaterialGraphSchema->InactivePinColor;
	}
	else if (bExecPin)
	{
		Params.WireColor = Settings->ExecutionPinTypeColor;
		Params.WireThickness = Settings->DefaultExecutionWireThickness;
	}
	else if (InputPin)
	{
		if (const UMaterialShaderValueTypeObject* ValueTypeObj = Cast<UMaterialShaderValueTypeObject>(InputPin->PinType.PinSubCategoryObject.Get()))
		{
			Params.WireColor = UMaterialGraphSchema::GetColorForConnectionType(ValueTypeObj->ValueType);
		}
	}

	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
	if (bDeemphasizeUnhoveredPins)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}
}

TSharedPtr<IToolTip> FMaterialGraphConnectionDrawingPolicy::GetConnectionToolTip(const SGraphPanel& GraphPanel,	const FGraphSplineOverlapResult& OverlapData) const
{
	TSharedPtr<SGraphPin> Pin1Widget;
	TSharedPtr<SGraphPin> Pin2Widget;
	OverlapData.GetPinWidgets(GraphPanel, Pin1Widget, Pin2Widget);

	if (!Pin1Widget || !Pin2Widget)
	{
		return FConnectionDrawingPolicy::GetConnectionToolTip(GraphPanel, OverlapData);
	}

	const FText LeftText = FText::Format(NSLOCTEXT("Unreal", "PinConnectionTooltipLeft", "<< {0}"), GetNodePinInfo(Pin1Widget));
	const FText RightText = FText::Format(NSLOCTEXT("Unreal", "PinConnectionTooltipRight", "{0} >>"), GetNodePinInfo(Pin2Widget));

	return SNew(SToolTip)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				SNew(STextBlock)
					.Margin(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
					.Justification(ETextJustify::Left)
					.Text(LeftText)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
					.Margin(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
					.Justification(ETextJustify::Right)
					.Text(RightText)
			]
		];
}

FText FMaterialGraphConnectionDrawingPolicy::GetNodePinInfo(const TSharedPtr<SGraphPin>& PinWidget) const
{
	const UEdGraphPin* PinObj = PinWidget->GetPinObj();
	const UEdGraphNode* EdNode = PinObj->GetOwningNode();
	 
	FString NodeTitle = EdNode->GetNodeTitle(ENodeTitleType::ListView).ToString();
	NodeTitle.RemoveFromStart(TEXT("Material Expression "));
	
	if (EdNode->GetCanRenameNode())
	{
		FText NodeEditableName = EdNode->GetNodeTitle(ENodeTitleType::EditableTitle);
		NodeTitle = NodeTitle + TEXT(" (") + NodeEditableName.ToString() + TEXT(")");
	}
	const FText PinName = PinObj->GetDisplayName().IsEmptyOrWhitespace() ? FText::FromName(PinObj->PinName) : PinObj->GetDisplayName();
	
	return FText::Format(NSLOCTEXT("Unreal", "PinConnectionTooltipPartial", "{0}\r\n{1}"), FText::FromString(NodeTitle), /*NodeType,*/ PinName);
}

FConnectionDrawingPolicy* FMaterialEditorGraphPinConnectionFactory::CreateConnectionPolicy(const UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj) const
{
	if (Schema->IsA(UMaterialGraphSchema::StaticClass()))
	{
		return new FMaterialGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
	}
	return nullptr;
}
