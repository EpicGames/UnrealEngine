// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"

class FSlateRect;
class FSlateWindowElementList;
class UEdGraph;
class UEdGraphPin;
class UMaterialGraph;
class UMaterialGraphSchema;

/////////////////////////////////////////////////////
// FMaterialGraphConnectionDrawingPolicy

// This class draws the connections for an UEdGraph using a MaterialGraph schema
class FMaterialGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	UMaterialGraph* MaterialGraph;
	const UMaterialGraphSchema* MaterialGraphSchema;

	// Each time a knot is encountered, input geometry is compared to output geometry to see if the pins on the knot need to be reversed
	TMap<class UMaterialGraphNode_Knot*, bool> KnotToReversedDirectionMap;

	bool ShouldChangeTangentForKnot(class UMaterialGraphNode_Knot* Knot);
	bool GetAverageConnectedPosition(class UMaterialGraphNode_Knot* Knot, EEdGraphPinDirection Direction, FVector2f& OutPos) const;
	bool FindPinCenter(UEdGraphPin* Pin, FVector2f& OutCenter) const;

public:
	FMaterialGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	virtual TSharedPtr<IToolTip> GetConnectionToolTip(const SGraphPanel& GraphPanel, const FGraphSplineOverlapResult& OverlapData) const override;
	
	// FConnectionDrawingPolicy interface
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	// End of FConnectionDrawingPolicy interface

private:
	FText GetNodePinInfo(const TSharedPtr<SGraphPin>& PinWidget) const;
};

struct FMaterialEditorGraphPinConnectionFactory : public FGraphPanelPinConnectionFactory
{
public:
	virtual class FConnectionDrawingPolicy* CreateConnectionPolicy(const class UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const class FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
};
