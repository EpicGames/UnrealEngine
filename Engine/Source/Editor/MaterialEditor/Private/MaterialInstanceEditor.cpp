// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialInstanceEditor.h"

#include "Widgets/Text/STextBlock.h"
#include "EngineGlobals.h"
#include "Engine/Texture.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Views/SListView.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Materials/Material.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "MaterialEditorModule.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "MaterialEditorContext.h"

#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"

#include "MaterialEditor.h"
#include "MaterialEditorActions.h"
#include "MaterialEditorUtilities.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorRuntimeVirtualTextureParameterValue.h"
#include "MaterialEditor/DEditorSparseVolumeTextureParameterValue.h"
#include "MaterialEditor/DEditorStaticSwitchParameterValue.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"

#include "PropertyEditorModule.h"
#include "MaterialEditorInstanceDetailCustomization.h"
#include "SMaterialLayersFunctionsTree.h"

#include "EditorViewportCommands.h"
#include "Widgets/Docking/SDockTab.h"
#include "CanvasTypes.h"
#include "UnrealEdGlobals.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "AdvancedPreviewSceneModule.h"
#include "ContentBrowserModule.h"
#include "Misc/MessageDialog.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styling/AppStyle.h"
#include "MaterialStats.h"
#include "MaterialEditingLibrary.h"
#include "Widgets/Layout/SScrollBox.h"
#include "DebugViewModeHelpers.h"
#include "IContentBrowserSingleton.h"
#include "MaterialEditorTabs.h"
#include "MaterialEditorViewportToolbarSections.h"
#include "Materials/MaterialFunctionMaterialLayer.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "Materials/MaterialInstanceBasePropertyOverrides.h"
#include "Materials/MaterialExpressionUtils.h"
#include "VT/RuntimeVirtualTexture.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationBackground.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "SMaterialEditorSubstrateWidget.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#include "DiffTool/Widgets/SMaterialRevisionMenu.h"
#include "Preferences/MaterialEditorOptions.h"

#define LOCTEXT_NAMESPACE "MaterialInstanceEditor"

DEFINE_LOG_CATEGORY_STATIC(LogMaterialInstanceEditor, Log, All);

static TAutoConsoleVariable<bool> CVarEnableTextureValidationInMaterialInstanceEditor(
	TEXT("r.MaterialEditor.EnableTextureValidation"),
	true,
	TEXT("Enables expensive texture validation in the material instance editor.  On by default.  Disable this to improve performance when changing texture parameters on large materials."),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////
// SMaterialTreeWidgetItem
class SMaterialTreeWidgetItem : public SMultiColumnTableRow< TWeakObjectPtr<UMaterialInterface> >
{
public:
	SLATE_BEGIN_ARGS(SMaterialTreeWidgetItem)
		: _ParentIndex( -1 )
		, _WidgetInfoToVisualize()
		{}
		SLATE_ARGUMENT( int32, ParentIndex )
		SLATE_ARGUMENT( TWeakObjectPtr<UMaterialInterface>, WidgetInfoToVisualize )
	SLATE_END_ARGS()

	/**
	 * Construct child widgets that comprise this widget.
	 *
	 * @param InArgs  Declaration from which to construct this widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		this->WidgetInfo = InArgs._WidgetInfoToVisualize;
		this->ParentIndex = InArgs._ParentIndex;

		SMultiColumnTableRow< TWeakObjectPtr<UMaterialInterface> >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
	}

	/** @return Widget based on the column name */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		FText Entry;
		FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", 9);
		if ( ColumnName == "Parent" )
		{
			if ( ParentIndex == 0 )
			{
				Entry = NSLOCTEXT("UnrealEd", "Material", "Material");
			}
			else if ( ParentIndex != -1 )
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("Index"), ParentIndex );
				Entry = FText::Format( FText::FromString("Parent {Index}"), Args );
			}
			else
			{
				Entry = NSLOCTEXT("UnrealEd", "Current", "Current");
				FontInfo = FCoreStyle::GetDefaultFontStyle("Bold", 9);
			}
		}
		else
		{
			Entry = FText::FromString( WidgetInfo.Get()->GetName() );
			if ( ParentIndex == -1 )
			{
				FontInfo = FCoreStyle::GetDefaultFontStyle("Bold", 9);
			}
		}
		
		return
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew( STextBlock )
				.Text( Entry )
				.Font( FontInfo )
			];
	}

protected:
	/** The info about the widget that we are visualizing */
	TAttribute< TWeakObjectPtr<UMaterialInterface> > WidgetInfo;

	/** The index this material has in our parents array */
	int32 ParentIndex;
};

//////////////////////////////////////////////////////////////////////////
// SFunctionTreeWidgetItem
class SFunctionTreeWidgetItem : public SMultiColumnTableRow< TWeakObjectPtr<UMaterialFunctionInterface> >
{
public:
	SLATE_BEGIN_ARGS(SFunctionTreeWidgetItem)
		: _ParentIndex( -1 )
		, _WidgetInfoToVisualize()
		{}
		SLATE_ARGUMENT( int32, ParentIndex )
		SLATE_ARGUMENT( TWeakObjectPtr<UMaterialFunctionInterface>, WidgetInfoToVisualize )
	SLATE_END_ARGS()

	/**
	 * Construct child widgets that comprise this widget.
	 *
	 * @param InArgs  Declaration from which to construct this widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		this->WidgetInfo = InArgs._WidgetInfoToVisualize;
		this->ParentIndex = InArgs._ParentIndex;

		SMultiColumnTableRow< TWeakObjectPtr<UMaterialFunctionInterface> >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
	}

	/** @return Widget based on the column name */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		FText Entry;
		FSlateFontInfo FontInfo = FSlateFontInfo(FCoreStyle::GetDefaultFont(), 9, "Regular");
		if ( ColumnName == "Parent" )
		{
			if ( ParentIndex == 0 )
			{
				Entry = NSLOCTEXT("UnrealEd", "Function", "Function");
			}
			else if ( ParentIndex != -1 )
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("Index"), ParentIndex );
				Entry = FText::Format( FText::FromString("Parent {Index}"), Args );
			}
			else
			{
				Entry = NSLOCTEXT("UnrealEd", "Current", "Current");
				FontInfo = FSlateFontInfo(FCoreStyle::GetDefaultFont(), 9, "Bold");
			}
		}
		else
		{
			Entry = FText::FromString( WidgetInfo.Get()->GetName() );
			if ( ParentIndex == -1 )
			{
				FontInfo = FSlateFontInfo(FCoreStyle::GetDefaultFont(), 9, "Bold");
			}
		}
		
		return
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew( STextBlock )
				.Text( Entry )
				.Font( FontInfo )
			];
	}

protected:
	/** The info about the widget that we are visualizing */
	TAttribute< TWeakObjectPtr<UMaterialFunctionInterface> > WidgetInfo;

	/** The index this material has in our parents array */
	int32 ParentIndex;
};

//////////////////////////////////////////////////////////////////////////
// FMaterialInstanceEditor

static constexpr int32 SpacingBetweenLines = 13;

void FMaterialInstanceEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MaterialInstanceEditor", "Material Instance Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	
	InTabManager->RegisterTabSpawner( FMaterialInstanceEditorTabs::PreviewTabId, FOnSpawnTab::CreateSP( this, &FMaterialInstanceEditor::SpawnTab_Preview ) )
		.SetDisplayName( LOCTEXT( "ViewportTab", "Viewport" ) )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon( FSlateIcon( FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports" ) );
	
	InTabManager->RegisterTabSpawner( FMaterialInstanceEditorTabs::PropertiesTabId, FOnSpawnTab::CreateSP( this, &FMaterialInstanceEditor::SpawnTab_Properties ) )
		.SetDisplayName( LOCTEXT( "PropertiesTab", "Details" ) )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon( FSlateIcon( FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details" ) );

	if (!bIsFunctionPreviewMaterial)
	{
		InTabManager->RegisterTabSpawner(FMaterialInstanceEditorTabs::LayerPropertiesTabId, FOnSpawnTab::CreateSP(this, &FMaterialInstanceEditor::SpawnTab_LayerProperties))
			.SetDisplayName(LOCTEXT("LayerPropertiesTab", "Layer Parameters"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Layers"));
	}
	
	InTabManager->RegisterTabSpawner(FMaterialInstanceEditorTabs::PreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FMaterialInstanceEditor::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	MaterialStatsManager->RegisterTabs();

	OnRegisterTabSpawners().Broadcast(InTabManager);
}

void FMaterialInstanceEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( FMaterialInstanceEditorTabs::PreviewTabId );
	InTabManager->UnregisterTabSpawner(FMaterialInstanceEditorTabs:: PropertiesTabId );
	if (!bIsFunctionPreviewMaterial)
	{
		InTabManager->UnregisterTabSpawner(FMaterialInstanceEditorTabs::LayerPropertiesTabId);
	}
	InTabManager->UnregisterTabSpawner( FMaterialInstanceEditorTabs::PreviewSettingsTabId );

	MaterialStatsManager->UnregisterTabs();

	OnUnregisterTabSpawners().Broadcast(InTabManager);
}

void FMaterialInstanceEditor::InitEditorForMaterial(UMaterialInstance* InMaterial)
{
	check(InMaterial);
	MaterialFunctionOriginal = nullptr;
	MaterialFunctionInstance = nullptr;
	FunctionMaterialProxy = nullptr;
	FunctionInstanceProxy = nullptr;
}

void FMaterialInstanceEditor::InitEditorForMaterialFunction(UMaterialFunctionInstance* InMaterialFunction)
{
	check(InMaterialFunction);
	MaterialFunctionOriginal = InMaterialFunction;

	// Working version of the function instance
	MaterialFunctionInstance = (UMaterialFunctionInstance*)StaticDuplicateObject(InMaterialFunction, GetTransientPackage(), NAME_None, ~RF_Standalone, UMaterialFunctionInstance::StaticClass()); 
	MaterialFunctionInstance->Parent = InMaterialFunction;
	
	// Preview material for function expressions
	FunctionMaterialProxy = NewObject<UMaterial>(); 
	{
		FArchiveUObject DummyArchive;
		FunctionMaterialProxy->Serialize(DummyArchive);
	}

	FunctionMaterialProxy->SetShadingModel(MSM_Unlit);
	FunctionMaterialProxy->SetFlags(RF_Transactional);
	FunctionMaterialProxy->bIsFunctionPreviewMaterial = true;

	UMaterialFunctionInterface* BaseFunctionInterface = MaterialFunctionInstance;
	while (UMaterialFunctionInstance* Instance = Cast<UMaterialFunctionInstance>(BaseFunctionInterface))
	{
		BaseFunctionInterface = Instance->GetBaseFunction();
	}
	if (UMaterialFunction* BaseFunction = Cast<UMaterialFunction>(BaseFunctionInterface))
	{
		FunctionMaterialProxy->AssignExpressionCollection(BaseFunction->GetExpressionCollection());
	}

	// Set expressions to be used with preview material
	bool bSetPreviewExpression = false;
	UMaterialExpressionFunctionOutput* FirstOutput = NULL;
	for (int32 ExpressionIndex = FunctionMaterialProxy->GetExpressions().Num() - 1; ExpressionIndex >= 0; --ExpressionIndex)
	{
		UMaterialExpression* Expression = FunctionMaterialProxy->GetExpressions()[ExpressionIndex];
		check(Expression);

		Expression->Function = NULL;
		Expression->Material = FunctionMaterialProxy;

		if (UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression))
		{
			FirstOutput = FunctionOutput;
			if (FunctionOutput->bLastPreviewed)
			{
				bSetPreviewExpression = true;
				FunctionOutput->ConnectToPreviewMaterial(FunctionMaterialProxy, 0);
			}
		}
	}

	if (!bSetPreviewExpression && FirstOutput)
	{
		FirstOutput->ConnectToPreviewMaterial(FunctionMaterialProxy, 0);
	}

	{
		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::SyncWithRenderingThread);
		UpdateContext.AddMaterial(FunctionMaterialProxy);
		FunctionMaterialProxy->PreEditChange(NULL);
		FunctionMaterialProxy->PostEditChange();
	}

	UMaterialFunctionInstance* ParentFunctionInstance = Cast<UMaterialFunctionInstance>(InMaterialFunction->Parent);
	UMaterialInterface* FunctionInstanceParent = FunctionMaterialProxy;
	if (ParentFunctionInstance)
	{
		// If our FunctionInstance inherits from *another* FunctionInstance, the parent may have overriden certain parameters
		// These overriden values should be the default value as far as our FunctionInstance is concerned
		// To model this, we create a MI that will hold these overriden values.  So our editor material hierarchy will look like this
		// * UMaterial - includes the UMaterialExpressions, copied from our base UMaterialFunction
		// * UMaterialInstance - we're creating this here, holds all parameter values overriden by our parent UMaterialFunctionInstance(s)
		// * UMaterialInstance - will be created below, this is the object/proxy we'll be editing and potentially setting parameters on
		UMaterialInstanceConstant* FunctionMaterialInstanceProxy = NewObject<UMaterialInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);
		FunctionMaterialInstanceProxy->SetParentEditorOnly(FunctionMaterialProxy);
		{
			FMaterialInstanceParameterUpdateContext UpdateContext(FunctionMaterialInstanceProxy);
			TArray<FMaterialParameterInfo> ParameterInfos;
			TArray<FGuid> ParameterIds;
			for (int32 ParameterTypeIndex = 0; ParameterTypeIndex < NumMaterialParameterTypes; ++ParameterTypeIndex)
			{
				const EMaterialParameterType ParameterType = (EMaterialParameterType)ParameterTypeIndex;
				FunctionMaterialProxy->GetAllParameterInfoOfType(ParameterType, ParameterInfos, ParameterIds);
				for (const FMaterialParameterInfo& ParameterInfo : ParameterInfos)
				{
					FMaterialParameterMetadata ParameterMeta;
					if (ParentFunctionInstance->GetParameterOverrideValue(ParameterType, ParameterInfo.Name, ParameterMeta))
					{
						UpdateContext.SetParameterValueEditorOnly(ParameterInfo, ParameterMeta, EMaterialSetParameterValueFlags::SetCurveAtlas);
					}
				}
			}
		}
		FunctionInstanceParent = FunctionMaterialInstanceProxy;
	}

	// Preview instance for function expressions
	FunctionInstanceProxy = NewObject<UMaterialInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);
	FunctionInstanceProxy->SetParentEditorOnly(FunctionInstanceParent);

	MaterialFunctionInstance->OverrideMaterialInstanceParameterValues(FunctionInstanceProxy);
	FunctionInstanceProxy->PreEditChange(NULL);
	FunctionInstanceProxy->PostEditChange();
}

void FMaterialInstanceEditor::InitMaterialInstanceEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit )
{
	GEditor->RegisterForUndo( this );

	check( ObjectToEdit );
	bIsFunctionPreviewMaterial = !!(MaterialFunctionInstance);
	UMaterialInstanceConstant* InstanceConstant = bIsFunctionPreviewMaterial ? FunctionInstanceProxy : Cast<UMaterialInstanceConstant>(ObjectToEdit);

	bShowAllMaterialParameters = false;

	// Construct a temp holder for our instance parameters.
	MaterialEditorInstance = NewObject<UMaterialEditorInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);

	bool bTempUseOldStyleMICEditorGroups = true;
	GConfig->GetBool(TEXT("/Script/UnrealEd.EditorEngine"), TEXT("UseOldStyleMICEditorGroups"), bTempUseOldStyleMICEditorGroups, GEngineIni);
	MaterialEditorInstance->bUseOldStyleMICEditorGroups = bTempUseOldStyleMICEditorGroups;
	MaterialEditorInstance->SetSourceInstance(InstanceConstant);
	MaterialEditorInstance->SetSourceFunction(MaterialFunctionOriginal);

	MaterialStatsManager = FMaterialStatsUtils::CreateMaterialStats(this, false, GetDefault<UMaterialEditorSettings>()->bAllowIgnoringCompilationErrors);
	MaterialStatsManager->SetMaterialsDisplayNames({MaterialEditorInstance->SourceInstance->GetName()});

	// Register our commands. This will only register them if not previously registered
	FMaterialEditorCommands::Register();

	CreateInternalWidgets();

	BindCommands();
	RegisterToolBar();

	UpdatePreviewViewportsVisibility();
	IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>("MaterialEditor");

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_MaterialInstanceEditor_Layout_v8")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()->SetSizeCoefficient(0.70f)->SetHideTabWell(true)
					->AddTab(FMaterialInstanceEditorTabs::PreviewTabId, ETabState::OpenedTab)
					->AddTab(FMaterialInstanceEditorTabs::PreviewSettingsTabId, ETabState::ClosedTab)
				)
				->Split
				(
					FTabManager::NewStack()->SetSizeCoefficient(0.30f)
					->AddTab(FMaterialInstanceEditorTabs::PropertiesTabId, ETabState::OpenedTab)
				)
			)
		);

	if (!bIsFunctionPreviewMaterial)
	{
		StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_MaterialInstanceEditor_Layout_v8")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
					->Split
					(
						FTabManager::NewStack()->SetSizeCoefficient(0.70f)->SetHideTabWell(true)
						->AddTab(FMaterialInstanceEditorTabs::PreviewTabId, ETabState::OpenedTab)
						->AddTab(FMaterialInstanceEditorTabs::PreviewSettingsTabId, ETabState::ClosedTab)
					)
					->Split
					(
						FTabManager::NewStack()->SetSizeCoefficient(0.30f)
						->AddTab(FMaterialInstanceEditorTabs::PropertiesTabId, ETabState::OpenedTab)
						->AddTab(FMaterialInstanceEditorTabs::LayerPropertiesTabId, ETabState::OpenedTab)
						->SetForegroundTab(FMaterialInstanceEditorTabs::PropertiesTabId)
					)
				)
			);
	}
	
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	TArray<UObject*> ObjectsToEdit;
	ObjectsToEdit.Add(ObjectToEdit);
	ObjectsToEdit.Add(MaterialEditorInstance);
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, MaterialInstanceEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit );

	AddMenuExtender(MaterialEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// @todo toolkit world centric editing
	/*if( IsWorldCentricAssetEditor() )
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		SpawnToolkitTab(PreviewTabId, FString(), EToolkitTabSpot::Viewport);		
		SpawnToolkitTab(PropertiesTabId, FString(), EToolkitTabSpot::Details);
	}*/

	// Load editor settings.
	LoadSettings();
	
	// Set the preview mesh for the material.  This call must occur after the toolbar is initialized.
	
	if ( !SetPreviewAssetByName( *InstanceConstant->PreviewMesh.ToString() ) )
	{
		// If the preview mesh could not be found for this instance, attempt to use the preview mesh for the parent material if one exists,
		//	or use a default instead if the parent's preview mesh cannot be used

		if ( InstanceConstant->Parent == nullptr || !SetPreviewAssetByName( *InstanceConstant->Parent->PreviewMesh.ToString() ) )
		{
			// Fallback on the default material editor preview mesh
			if (const UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
			{
				SetPreviewAsset(MaterialEditorOptions->GetPreviewMesh());
			}
		}
	}

	PreviewVC->RefreshViewport();
	RebuildInheritanceList();
	MaterialInstanceDetails->SetObject(MaterialEditorInstance, true);
	RefreshOnScreenTextureWarnings();

	// Notify other editors if this material editor has a post process named output, which may affect their preview
	NotifyUserSceneTextureLoadOrUnload();
}

void FMaterialInstanceEditor::ReInitMaterialFunctionProxies()
{
	if (bIsFunctionPreviewMaterial)
	{
		// Temporarily store unsaved parameters
		TArray<FScalarParameterValue> ScalarParameterValues = FunctionInstanceProxy->ScalarParameterValues;
		TArray<FVectorParameterValue> VectorParameterValues = FunctionInstanceProxy->VectorParameterValues;
		TArray<FDoubleVectorParameterValue> DoubleVectorParameterValues = FunctionInstanceProxy->DoubleVectorParameterValues;
		TArray<FTextureParameterValue> TextureParameterValues = FunctionInstanceProxy->TextureParameterValues;
		TArray<FTextureCollectionParameterValue> TextureCollectionParameterValues = FunctionInstanceProxy->TextureCollectionParameterValues;
		TArray<FRuntimeVirtualTextureParameterValue> RuntimeVirtualTextureParameterValues = FunctionInstanceProxy->RuntimeVirtualTextureParameterValues;
		TArray<FSparseVolumeTextureParameterValue> SparseVolumeTextureParameterValues = FunctionInstanceProxy->SparseVolumeTextureParameterValues;
		TArray<FFontParameterValue> FontParameterValues = FunctionInstanceProxy->FontParameterValues;

		const FStaticParameterSet& OldStaticParameters = FunctionInstanceProxy->GetStaticParameters();
		TArray<FStaticSwitchParameter> StaticSwitchParameters = OldStaticParameters.StaticSwitchParameters;
		TArray<FStaticComponentMaskParameter> StaticComponentMaskParameters = OldStaticParameters.EditorOnly.StaticComponentMaskParameters;

		// Regenerate proxies
		InitEditorForMaterialFunction(MaterialFunctionOriginal);
		MaterialEditorInstance->SetSourceInstance(FunctionInstanceProxy);
		MaterialEditorInstance->SetSourceFunction(MaterialFunctionOriginal);
		
		// Restore dynamic parameters, filtering those that no-longer exist
		TArray<FMaterialParameterInfo> OutParameterInfo;
		TArray<FGuid> Guids;

		FunctionInstanceProxy->GetAllScalarParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->ScalarParameterValues.Empty();
		for (FScalarParameterValue& ScalarParameter : ScalarParameterValues)
		{
			int32 Index = Guids.Find(ScalarParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->ScalarParameterValues.Add(ScalarParameter);
				FunctionInstanceProxy->ScalarParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllVectorParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->VectorParameterValues.Empty();
		for (FVectorParameterValue& VectorParameter : VectorParameterValues)
		{
			int32 Index = Guids.Find(VectorParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->VectorParameterValues.Add(VectorParameter);
				FunctionInstanceProxy->VectorParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllDoubleVectorParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->DoubleVectorParameterValues.Empty();
		for (FDoubleVectorParameterValue& DoubleVectorParameter : DoubleVectorParameterValues)
		{
			int32 Index = Guids.Find(DoubleVectorParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->DoubleVectorParameterValues.Add(DoubleVectorParameter);
				FunctionInstanceProxy->DoubleVectorParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllTextureParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->TextureParameterValues.Empty();
		for (FTextureParameterValue& TextureParameter : TextureParameterValues)
		{
			int32 Index = Guids.Find(TextureParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->TextureParameterValues.Add(TextureParameter);
				FunctionInstanceProxy->TextureParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllTextureCollectionParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->TextureCollectionParameterValues.Empty();
		for (FTextureCollectionParameterValue& TextureCollectionParameter : TextureCollectionParameterValues)
		{
			int32 Index = Guids.Find(TextureCollectionParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->TextureCollectionParameterValues.Add(TextureCollectionParameter);
				FunctionInstanceProxy->TextureCollectionParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllRuntimeVirtualTextureParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->RuntimeVirtualTextureParameterValues.Empty();
		for (FRuntimeVirtualTextureParameterValue& RuntimeVirtualTextureParameter : RuntimeVirtualTextureParameterValues)
		{
			int32 Index = Guids.Find(RuntimeVirtualTextureParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->RuntimeVirtualTextureParameterValues.Add(RuntimeVirtualTextureParameter);
				FunctionInstanceProxy->RuntimeVirtualTextureParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllSparseVolumeTextureParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->SparseVolumeTextureParameterValues.Empty();
		for (FSparseVolumeTextureParameterValue& SparseVolumeTextureParameter : SparseVolumeTextureParameterValues)
		{
			int32 Index = Guids.Find(SparseVolumeTextureParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->SparseVolumeTextureParameterValues.Add(SparseVolumeTextureParameter);
				FunctionInstanceProxy->SparseVolumeTextureParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllFontParameterInfo(OutParameterInfo, Guids);
		FunctionInstanceProxy->FontParameterValues.Empty();
		for (FFontParameterValue& FontParameter : FontParameterValues)
		{
			int32 Index = Guids.Find(FontParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				FunctionInstanceProxy->FontParameterValues.Add(FontParameter);
				FunctionInstanceProxy->FontParameterValues.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		// Restore static parameters, filtering those that no-longer exist
		FStaticParameterSet StaticParametersOverride = FunctionInstanceProxy->GetStaticParameters();

		FunctionInstanceProxy->GetAllStaticSwitchParameterInfo(OutParameterInfo, Guids);
		StaticParametersOverride.StaticSwitchParameters.Empty();
		for (FStaticSwitchParameter& StaticSwitchParameter : StaticSwitchParameters)
		{
			int32 Index = Guids.Find(StaticSwitchParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				StaticParametersOverride.StaticSwitchParameters.Add(StaticSwitchParameter);
				StaticParametersOverride.StaticSwitchParameters.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->GetAllStaticComponentMaskParameterInfo(OutParameterInfo, Guids);
		StaticParametersOverride.EditorOnly.StaticComponentMaskParameters.Empty();
		for (FStaticComponentMaskParameter& StaticComponentMaskParameter : StaticComponentMaskParameters)
		{
			int32 Index = Guids.Find(StaticComponentMaskParameter.ExpressionGUID);
			if (Index != INDEX_NONE)
			{
				StaticParametersOverride.EditorOnly.StaticComponentMaskParameters.Add(StaticComponentMaskParameter);
				StaticParametersOverride.EditorOnly.StaticComponentMaskParameters.Last().ParameterInfo = OutParameterInfo[Index];
			}
		}

		FunctionInstanceProxy->UpdateStaticPermutation(StaticParametersOverride);

		// Refresh and apply to preview
		FunctionInstanceProxy->PreEditChange(NULL);
		FunctionInstanceProxy->PostEditChange();
		SetPreviewMaterial(FunctionInstanceProxy);
	}
}

FMaterialInstanceEditor::FMaterialInstanceEditor()
: MaterialEditorInstance(nullptr)
, bIsFunctionPreviewMaterial(false)
, MenuExtensibilityManager(new FExtensibilityManager)
, ToolBarExtensibilityManager(new FExtensibilityManager)
, MaterialFunctionOriginal(nullptr)
, MaterialFunctionInstance(nullptr)
, FunctionMaterialProxy(nullptr)
, FunctionInstanceProxy(nullptr)
{
	UPackage::PreSavePackageWithContextEvent.AddRaw(this, &FMaterialInstanceEditor::PreSavePackage);
}

FMaterialInstanceEditor::~FMaterialInstanceEditor()
{
	bDestructing = true;

	// Notify other editors if this material editor has a post process named output, which may affect their preview
	NotifyUserSceneTextureLoadOrUnload();

	// Broadcast that this editor is going down to all listeners
	OnMaterialEditorClosed().Broadcast();

	GEditor->UnregisterForUndo( this );

	UPackage::PreSavePackageWithContextEvent.RemoveAll(this);

	// The streaming data will be null if there were any edits
	if (MaterialEditorInstance && MaterialEditorInstance->SourceInstance && !MaterialEditorInstance->SourceInstance->HasTextureStreamingData())
	{
		MaterialEditorInstance->SourceInstance->SetMarkTextureAsEditorStreamingPool(false);

		UPackage* Package = MaterialEditorInstance->SourceInstance->GetOutermost();
		if (Package && Package->IsDirty() && Package != GetTransientPackage())
		{
			FMaterialEditorUtilities::BuildTextureStreamingData(MaterialEditorInstance->SourceInstance);
		}
	}

	if (MaterialEditorInstance)
	{
		MaterialEditorInstance->SourceInstance = nullptr;
		MaterialEditorInstance->SourceFunction = nullptr;
		MaterialEditorInstance->MarkAsGarbage();
		MaterialEditorInstance = nullptr;
	}

	MaterialParentList.Empty();
	FunctionParentList.Empty();

	SaveSettings();

	MaterialInstanceDetails.Reset();
}

void FMaterialInstanceEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Serialize our custom object instance
	Collector.AddReferencedObject(MaterialEditorInstance);
}

void FMaterialInstanceEditor::BindCommands()
{
	const FMaterialEditorCommands& Commands = FMaterialEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Apply,
		FExecuteAction::CreateSP( this, &FMaterialInstanceEditor::OnApply ),
		FCanExecuteAction::CreateSP( this, &FMaterialInstanceEditor::OnApplyEnabled ),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FMaterialInstanceEditor::OnApplyVisible));

	ToolkitCommands->MapAction(
		Commands.ShowAllMaterialParameters,
		FExecuteAction::CreateSP( this, &FMaterialInstanceEditor::ToggleShowAllMaterialParameters ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &FMaterialInstanceEditor::IsShowAllMaterialParametersChecked ) );

	ToolkitCommands->MapAction(
		FEditorViewportCommands::Get().ToggleRealTime,
		FExecuteAction::CreateSP( PreviewVC.ToSharedRef(), &SMaterialEditor3DPreviewViewport::OnToggleRealtime ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( PreviewVC.ToSharedRef(), &SMaterialEditor3DPreviewViewport::IsRealtime ) );
}

void FMaterialInstanceEditor::OnApply()
{
	if (bIsFunctionPreviewMaterial && MaterialEditorInstance)
	{
		UE_LOGF(LogMaterialInstanceEditor, Log, "Applying instance %ls", *GetEditingObjects()[0]->GetName());
		MaterialEditorInstance->bIsFunctionInstanceDirty = true;
		MaterialEditorInstance->ApplySourceFunctionChanges();
	}
}

bool FMaterialInstanceEditor::OnApplyEnabled() const
{
	return MaterialEditorInstance && MaterialEditorInstance->bIsFunctionInstanceDirty == true;
}

bool FMaterialInstanceEditor::OnApplyVisible() const
{
	return MaterialEditorInstance && MaterialEditorInstance->bIsFunctionPreviewMaterial == true;
}

bool FMaterialInstanceEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	if (MaterialEditorInstance->bIsFunctionInstanceDirty && InCloseReason != EAssetEditorCloseReason::AssetForceDeleted)
	{
		// Find out the user wants to do with this dirty function instance
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
			FText::Format(
				NSLOCTEXT("UnrealEd", "Prompt_MaterialInstanceEditorClose", "Would you like to apply changes to this instance to the original asset?\n{0}\n(No will lose all changes!)"),
				FText::FromString(MaterialFunctionOriginal->GetPathName())));

		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			// Update instance and exit
			MaterialEditorInstance->ApplySourceFunctionChanges();
			break;
				
		case EAppReturnType::No:
			// Exit
			break;
				
		case EAppReturnType::Cancel:
			// Don't exit
			return false;
		}
	}

	return true;
}

void FMaterialInstanceEditor::ToggleShowAllMaterialParameters()
{
	bShowAllMaterialParameters = !bShowAllMaterialParameters;
	UpdatePropertyWindow();
}

bool FMaterialInstanceEditor::IsShowAllMaterialParametersChecked() const
{
	return bShowAllMaterialParameters;
}

void FMaterialInstanceEditor::CreateInternalWidgets()
{
	PreviewVC = SNew(SMaterialEditor3DPreviewViewport)
		.MaterialEditor(SharedThis(this));

	PreviewUIViewport = SNew(SMaterialEditorUIPreviewViewport, GetMaterialInterface());

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bShowCustomFilterOption = true;
	MaterialInstanceDetails = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	// the sizes of the parameter lists are only based on the parent material and not changed out from under the details panel 
	// When a parameter is added open MI editors are refreshed
	// the tree should also refresh if one of the layer or blend assets is swapped

	auto ValidationLambda = ([](const FRootPropertyNodeList& PropertyNodeList) { return true; });
	MaterialInstanceDetails->SetCustomValidatePropertyNodesFunction(FOnValidateDetailsViewPropertyNodes::CreateLambda(MoveTemp(ValidationLambda)));
	if (!bIsFunctionPreviewMaterial)
	{
		MaterialLayersFunctionsInstance = SNew(SMaterialLayersFunctionsInstanceWrapper)
			.InMaterialEditorInstance(MaterialEditorInstance)
			.InShowHiddenDelegate(FGetShowHiddenParameters::CreateSP(this, &FMaterialInstanceEditor::GetShowHiddenParameters))
			.InNotifyHook(this);
		MaterialLayersFunctionsInstance->OnLayerPropertyChanged.BindSP(this, &FMaterialInstanceEditor::Refresh);
	}
	
	FOnGetDetailCustomizationInstance LayoutMICDetails = FOnGetDetailCustomizationInstance::CreateStatic( 
		&FMaterialInstanceParameterDetails::MakeInstance, MaterialEditorInstance.Get(), MaterialLayersFunctionsInstance.Get(), FGetShowHiddenParameters::CreateSP(this, &FMaterialInstanceEditor::GetShowHiddenParameters) );
	MaterialInstanceDetails->RegisterInstancedCustomPropertyLayout( UMaterialEditorInstanceConstant::StaticClass(), LayoutMICDetails );
	MaterialInstanceDetails->SetCustomFilterLabel(LOCTEXT("ShowOverriddenOnly", "Show Only Overridden Parameters"));
	MaterialInstanceDetails->SetCustomFilterDelegate(FSimpleDelegate::CreateSP(this, &FMaterialInstanceEditor::FilterOverriddenProperties));
	MaterialEditorInstance->DetailsView = MaterialInstanceDetails;

	SubstrateWidget = SNew(SMaterialEditorSubstrateWidget, SharedThis(this));
}

void FMaterialInstanceEditor::FilterOverriddenProperties()
{
	MaterialEditorInstance->bShowOnlyOverrides = !MaterialEditorInstance->bShowOnlyOverrides;
	MaterialInstanceDetails->ForceRefresh();
}

void FMaterialInstanceEditor::UpdatePreviewViewportsVisibility()
{
	UMaterial* PreviewMaterial = GetMaterialInterface()->GetBaseMaterial();
	if( PreviewMaterial->IsUIMaterial() )
	{
		PreviewVC->SetVisibility(EVisibility::Collapsed);
		PreviewUIViewport->SetVisibility(EVisibility::Visible);
	}
	else
	{
		PreviewVC->SetVisibility(EVisibility::Visible);
		PreviewUIViewport->SetVisibility(EVisibility::Collapsed);
	}
}

void FMaterialInstanceEditor::RegisterToolBar()
{
	FName MenuName = FAssetEditorToolkit::GetToolMenuToolbarName();
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(MenuName, "AssetEditor.DefaultToolBar", EMultiBoxType::ToolBar);

		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
		{
			FToolMenuSection& MaterialInstanceSection = ToolBar->AddSection("MaterialInstanceTools", TAttribute<FText>(), InsertAfterAssetSection);

			MaterialInstanceSection.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().Apply));
			MaterialInstanceSection.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().ShowAllMaterialParameters));
			MaterialInstanceSection.AddEntry(FToolMenuEntry::InitComboButton(
				"Hierarchy",
				FToolUIActionChoice(),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
					{
						UMaterialEditorMenuContext* SubMenuContext = InSubMenu->FindContext<UMaterialEditorMenuContext>();
						if (SubMenuContext && SubMenuContext->MaterialEditor.IsValid())
						{
							SubMenuContext->MaterialEditor.Pin()->GenerateInheritanceMenu(InSubMenu);
						}
					}),
				LOCTEXT("Hierarchy", "Hierarchy"),
						FText::GetEmpty(),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MaterialEditor.Hierarchy"),
						false
						));
		}

		FToolMenuSection& UISection = ToolBar->AddSection("Stats", TAttribute<FText>(), InsertAfterAssetSection);
		UISection.AddEntry(FToolMenuEntry::InitToolBarButton(FMaterialEditorCommands::Get().TogglePlatformStats));

		SMaterialRevisionMenu::MakeDiffMenu(ToolBar);
	}
}

void FMaterialInstanceEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UMaterialEditorMenuContext* Context = NewObject<UMaterialEditorMenuContext>();
	Context->MaterialEditor = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FMaterialInstanceEditor::ExtendToolbar()
{
	AddToolbarExtender(GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
	AddToolbarExtender(MaterialEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

TSharedPtr<IAssetReferenceFilter> FMaterialInstanceEditor::MakeAssetReferenceFilter() const
{
	if (GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(GetEditingObjects());
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

void FMaterialInstanceEditor::GenerateInheritanceMenu(UToolMenu* Menu)
{
	RebuildInheritanceList();
	Menu->bShouldCloseWindowAfterMenuSelection = true;
	Menu->SetMaxHeight(500);

	{
		FToolMenuSection& Section = Menu->AddSection("ParentChain", LOCTEXT("ParentChain", "Parent Chain"));
		if (bIsFunctionPreviewMaterial)
		{
			if (FunctionParentList.Num() == 0)
			{
				const FText NoParentText = LOCTEXT("NoParentFound", "No Parent Found");
				TSharedRef<SWidget> NoParentWidget = SNew(STextBlock)
					.Text(NoParentText);
				Section.AddEntry(FToolMenuEntry::InitWidget("NoParentEntry", NoParentWidget, FText::GetEmpty()));
			}
			for (FAssetData FunctionParent : FunctionParentList)
			{
				FMaterialEditor::AddInheritanceMenuEntry(Section, FunctionParent, bIsFunctionPreviewMaterial);
			}
		}
		else
		{
			if (MaterialParentList.Num() == 0)
			{
				const FText NoParentText = LOCTEXT("NoParentFound", "No Parent Found");
				TSharedRef<SWidget> NoParentWidget = SNew(STextBlock)
					.Text(NoParentText);
				Section.AddEntry(FToolMenuEntry::InitWidget("NoParentEntry", NoParentWidget, FText::GetEmpty()));
			}
			for (FAssetData MaterialParent : MaterialParentList)
			{
				FMaterialEditor::AddInheritanceMenuEntry(Section, MaterialParent, bIsFunctionPreviewMaterial);
			}
		}
	}

	if (!bIsFunctionPreviewMaterial)
	{
		FToolMenuSection& Section = Menu->AddSection("MaterialInstances", LOCTEXT("MaterialInstances", "Material Instances"));
		for (const FAssetData& MaterialChild : MaterialChildList)
		{
			FMaterialEditor::AddInheritanceMenuEntry(Section, MaterialChild, bIsFunctionPreviewMaterial);
		}
	}
}

void FMaterialInstanceEditor::RefreshPreviewViewport()
{
	if (PreviewVC.IsValid())
	{
		PreviewVC->RefreshViewport();
	}
}

TSharedRef<SDockTab> FMaterialInstanceEditor::SpawnTab_Preview( const FSpawnTabArgs& Args )
{	
	check( Args.GetTabId().TabType == FMaterialInstanceEditorTabs::PreviewTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("ViewportTabTitle", "Viewport"))
		[
			SNew( SOverlay )
			+ SOverlay::Slot()
			[
				PreviewVC.ToSharedRef()
			]
			+ SOverlay::Slot()
			[
				PreviewUIViewport.ToSharedRef()
			]
			+ SOverlay::Slot()
			.Padding(8)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(SNotificationBackground)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(6.f)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SAssignNew(NumShadersPillText, STextBlock)
					]
				]
			]
		];

	PreviewVC->OnAddedToTab( SpawnedTab );

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );
	return SpawnedTab;
}


TSharedRef<SDockTab> FMaterialInstanceEditor::SpawnTab_Properties( const FSpawnTabArgs& Args )
{	
	check( Args.GetTabId().TabType == FMaterialInstanceEditorTabs::PropertiesTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("MaterialPropertiesTitle", "Details"))
		[
			SNew(SBorder)
			.Padding(4)
			[
				MaterialInstanceDetails.ToSharedRef()
			]
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );
	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialInstanceEditor::SpawnTab_LayerProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == FMaterialInstanceEditorTabs::LayerPropertiesTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
	.Label(LOCTEXT("MaterialLayerPropertiesTitle", "Layer Parameters"))
	[
		SNew(SBorder)
			.Padding(4)
			[
				MaterialLayersFunctionsInstance.ToSharedRef()
			]
	];

	AddToSpawnedToolPanels(Args.GetTabId().TabType, SpawnedTab);
	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialInstanceEditor::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FMaterialInstanceEditorTabs::PreviewSettingsTabId);

	TSharedRef<SWidget> InWidget = SNullWidget::NullWidget;
	if (PreviewVC.IsValid())
	{
		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		InWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewVC->GetPreviewScene());
	}

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		[
			SNew(SBox)
			[
				InWidget
			]
		];

	return SpawnedTab;
}

void FMaterialInstanceEditor::AddToSpawnedToolPanels(const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab)
{
	TWeakPtr<SDockTab>* TabSpot = SpawnedToolPanels.Find(TabIdentifier);
	if (!TabSpot)
	{
		SpawnedToolPanels.Add(TabIdentifier, SpawnedTab);
	}
	else
	{
		check(!TabSpot->IsValid());
		*TabSpot = SpawnedTab;
	}
}

FName FMaterialInstanceEditor::GetToolkitFName() const
{
	return FName("MaterialInstanceEditor");
}

FText FMaterialInstanceEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Material Instance Editor");
}

FText FMaterialInstanceEditor::GetToolkitName() const
{
	const UObject* EditingObject = GetEditingObjects()[0];
	check(EditingObject);

	return GetLabelForObject(EditingObject);
}

FText FMaterialInstanceEditor::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetEditingObjects()[0];

	// Overridden to accommodate editing of multiple objects (original and preview materials)
	return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
}

FString FMaterialInstanceEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Material Instance ").ToString();
}

FLinearColor FMaterialInstanceEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.3f, 0.2f, 0.5f, 0.5f );
}

UMaterialInterface* FMaterialInstanceEditor::GetMaterialInterface() const
{
	return MaterialEditorInstance->PreviewInstance ? MaterialEditorInstance->PreviewInstance : MaterialEditorInstance->SourceInstance;
}

void FMaterialInstanceEditor::NotifyPreChange(FProperty* PropertyThatChanged)
{
}

void FMaterialInstanceEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// If they changed the parent, regenerate the parent list.
	if(PropertyThatChanged->GetName()==TEXT("Parent"))
	{
		bool bSetEmptyParent = false;

		// Check to make sure they didnt set the parent to themselves.
		if(MaterialEditorInstance->Parent==MaterialEditorInstance->SourceInstance)
		{
			bSetEmptyParent = true;
		}

		if (bSetEmptyParent)
		{
			FMaterialUpdateContext UpdateContext;
			MaterialEditorInstance->Parent = NULL;

			if(MaterialEditorInstance->SourceInstance)
			{
				MaterialEditorInstance->SourceInstance->SetParentEditorOnly(NULL);
				MaterialEditorInstance->SourceInstance->PostEditChange();
			}
			UpdateContext.AddMaterialInstance(MaterialEditorInstance->SourceInstance);
		}

		RebuildInheritanceList();

		UpdatePropertyWindow();

		// If the parent of this instance changed we need to update the cached state on the stats manager to have the updated parent.
		MaterialStatsManager->SetMaterial(MaterialEditorInstance->SourceInstance);
		MaterialStatsManager->Update();
	}
	else if(PropertyThatChanged->GetName() == TEXT("PreviewMesh"))
	{
		RefreshPreviewAsset();
	}

	// Rebuild the property window to account for the possibility that  
	// the item changed was a static switch or function call parameter
	UObject* PropertyClass = PropertyThatChanged->GetOwner<UObject>();
	if(PropertyClass && (PropertyClass == UDEditorStaticSwitchParameterValue::StaticClass() || 
		PropertyClass == FEditorParameterGroup::StaticStruct() || 
		PropertyClass == FMaterialInstanceBasePropertyOverrides::StaticStruct())
		&& MaterialEditorInstance->Parent && MaterialEditorInstance->SourceInstance )
	{
		// TODO: We need to hit this on MaterialLayerParam updates but only get notifications for their array elements changing, hence the overly generic test above
		MaterialEditorInstance->VisibleExpressions.Empty();
		FMaterialEditorUtilities::GetVisibleMaterialParameters(MaterialEditorInstance->Parent->GetMaterial(), MaterialEditorInstance->SourceInstance, MaterialEditorInstance->VisibleExpressions);

		UpdatePropertyWindow();
	}

	if (PropertyClass && 
	   (PropertyClass == UDEditorTextureParameterValue::StaticClass() ||
		PropertyClass == UDEditorRuntimeVirtualTextureParameterValue::StaticClass()||
		PropertyClass == UDEditorSparseVolumeTextureParameterValue::StaticClass()))
	{
		RefreshOnScreenTextureWarnings();
	}
	
	const bool bRefreshStats = PropertyClass && (
		PropertyClass == UDEditorStaticSwitchParameterValue::StaticClass() ||
		PropertyClass == FMaterialInstanceBasePropertyOverrides::StaticStruct()
		);

	if (bRefreshStats)
	{
		// something was changed in the material so we need to reflect this in the stats
		MaterialStatsManager->SignalMaterialChanged();
	}

	// Update the preview window when the user changes a property.
	PreviewVC->RefreshViewport();
}

void FMaterialInstanceEditor::RefreshPreviewAsset()
{
	UObject* PreviewAsset = MaterialEditorInstance->SourceInstance->PreviewMesh.TryLoad();
	if (!PreviewAsset)
	{
		// Attempt to use the parent material's preview mesh if the instance's preview mesh is invalid, and use the default
		//	preview mesh instead if the parent's mesh is also invalid
		UMaterialInterface* ParentMaterial = MaterialEditorInstance->SourceInstance->Parent;

		PreviewAsset = ParentMaterial != nullptr ? ParentMaterial->PreviewMesh.TryLoad() : nullptr;

		if (!PreviewAsset)
		{
			// Fallback on the default material editor preview mesh
			if (const UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
			{
				PreviewAsset = MaterialEditorOptions->GetPreviewMesh();
			}
		}

		USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(MaterialEditorInstance->SourceInstance->ThumbnailInfo);
		if (ThumbnailInfo)
		{
			ThumbnailInfo->PreviewMesh.Reset();
		}

	}
	PreviewVC->SetPreviewAsset(PreviewAsset);
}

void FMaterialInstanceEditor::PreSavePackage(UPackage* Package, FObjectPreSaveContext ObjectSaveContext)
{
	// The streaming data will be null if there were any edits
	if (MaterialEditorInstance && 
		MaterialEditorInstance->SourceInstance && 
		MaterialEditorInstance->SourceInstance->GetOutermost() == Package &&
		!MaterialEditorInstance->SourceInstance->HasTextureStreamingData())
	{
		FMaterialEditorUtilities::BuildTextureStreamingData(MaterialEditorInstance->SourceInstance);
	}
}

void FMaterialInstanceEditor::RebuildInheritanceList()
{
	if (bIsFunctionPreviewMaterial)
	{
		FunctionParentList.Empty();

		// Append function instance parent chain
		UMaterialFunctionInstance* Current = MaterialFunctionOriginal;
		UMaterialFunctionInterface* Parent = Current->Parent;
		while (Parent)
		{
			FunctionParentList.Insert(Parent, 0);

			Current = Cast<UMaterialFunctionInstance>(Parent);
			Parent = Current ? ToRawPtr(Current->Parent) : nullptr;
		}
	}
	else
	{
		MaterialChildList.Empty();
		MaterialParentList.Empty();

		// Travel up the parent chain for this material instance until we reach the root material.
		UMaterialInstance* InstanceConstant = MaterialEditorInstance->SourceInstance;

		if(InstanceConstant)
		{
			UMaterialEditingLibrary::GetChildInstances(InstanceConstant, MaterialChildList);

			// Add all parents
			UMaterialInterface* Parent = InstanceConstant->Parent;
			while(Parent && Parent != InstanceConstant)
			{
				MaterialParentList.Insert(Parent,0);

				// If the parent is a material then break.
				InstanceConstant = Cast<UMaterialInstance>(Parent);

				if(InstanceConstant)
				{
					Parent = InstanceConstant->Parent;
				}
				else
				{
					break;
				}
			}
		}
	}
}

void FMaterialInstanceEditor::RebuildMaterialInstanceEditor()
{
	if( MaterialEditorInstance )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialEditingLibrary::RebuildMaterialInstanceEditors)

		ReInitMaterialFunctionProxies();
		MaterialEditorInstance->CopyBasePropertiesFromParent();
		MaterialEditorInstance->RegenerateArrays();
		RebuildInheritanceList(); // Required b/c recompiled parent materials result in invalid weak object pointers
		UpdatePropertyWindow();
	}
}

void FMaterialInstanceEditor::DrawMessages( FViewport* Viewport, FCanvas* Canvas )
{
	Canvas->PushAbsoluteTransform(FMatrix::Identity);
	if ( MaterialEditorInstance->Parent && MaterialEditorInstance->SourceInstance && GAreScreenMessagesEnabled )
	{
		const FMaterialResource* MaterialResource = MaterialEditorInstance->SourceInstance->GetMaterialResource(GMaxRHIShaderPlatform);
		UMaterial* BaseMaterial = MaterialEditorInstance->SourceInstance->GetMaterial();
		int32 DrawPositionY = 50;
		if ( BaseMaterial && MaterialResource )
		{
			constexpr int32 DrawPositionX = 5;

			ERHIFeatureLevel::Type FeatureLevel = MaterialResource->GetFeatureLevel();
			FString FeatureLevelName;
			GetFeatureLevelName(FeatureLevel,FeatureLevelName);

			const TArray<FString>& CompileErrors = MaterialResource->GetCompileErrors();
			for(int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
			{
				Canvas->DrawShadowedString(DrawPositionX, DrawPositionY, *FString::Printf(TEXT("[%s] %s"), *FeatureLevelName, *CompileErrors[ErrorIndex]), GEngine->GetTinyFont(), FLinearColor(1, 0, 0));
				DrawPositionY += SpacingBetweenLines;
			}
		}

		DrawOnScreenMessages( Canvas, DrawPositionY );
	}
	Canvas->PopTransform();
}

void FMaterialInstanceEditor::RefreshOnScreenTextureWarnings()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialInstanceEditor::RefreshOnScreenTextureWarnings)

	OnScreenMessages.Reset();

	const bool bValidateMaterials = CVarEnableTextureValidationInMaterialInstanceEditor.GetValueOnAnyThread();

	if (bValidateMaterials && MaterialEditorInstance->SourceInstance)
	{
		UMaterial* BaseMaterial = MaterialEditorInstance->SourceInstance->GetMaterial();
		if (BaseMaterial)
		{
			UEnum* SamplerTypeEnum = StaticEnum<EMaterialSamplerType>();
			check(SamplerTypeEnum);
			UEnum* MaterialTypeEnum = StaticEnum<ERuntimeVirtualTextureMaterialType>();
			check(MaterialTypeEnum);

			TMap<FGuid, TObjectPtr<UMaterialExpression>> MaterialGuids;
			BaseMaterial->BuildExpressionParametersGuidsMap(MaterialGuids);

			const int32 GroupCount = MaterialEditorInstance->ParameterGroups.Num();
			for (int32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
			{
				const FEditorParameterGroup& Group = MaterialEditorInstance->ParameterGroups[GroupIndex];
				const int32 ParameterCount = Group.Parameters.Num();
				for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
				{
					UDEditorTextureParameterValue* TextureParameterValue = Cast<UDEditorTextureParameterValue>(Group.Parameters[ParameterIndex]);
					if (TextureParameterValue && TextureParameterValue->ExpressionId.IsValid())
					{
						UTexture* Texture = NULL;
						MaterialEditorInstance->SourceInstance->GetTextureParameterValue(TextureParameterValue->ParameterInfo, Texture);
						if (Texture)
						{
							EMaterialSamplerType SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture(Texture);
							UMaterialExpressionTextureSampleParameter* Expression = nullptr;

							TObjectPtr<UMaterialExpression>* MapExpressions = MaterialGuids.Find(TextureParameterValue->ExpressionId);
							if (MapExpressions)
							{
								Expression = Cast<UMaterialExpressionTextureSampleParameter>(*MapExpressions);
							}

							FString ErrorMessage;
							if (Expression && !Expression->TextureIsValid(Texture, ErrorMessage))
							{
								OnScreenMessages.Emplace(FLinearColor(1, 0, 0),
									FString::Printf(TEXT("Error: %s has invalid texture %s: %s."),
									*TextureParameterValue->ParameterInfo.Name.ToString(),
									*Texture->GetPathName(),
									*ErrorMessage));
							}
							else
							{
								if (Expression && Expression->SamplerType != SamplerType)
								{
									FString SamplerTypeDisplayName = SamplerTypeEnum->GetDisplayNameTextByValue(Expression->SamplerType).ToString();
									OnScreenMessages.Emplace(FLinearColor(1, 1, 0),
										FString::Printf(TEXT("Warning: %s samples %s as %s."),
										*TextureParameterValue->ParameterInfo.Name.ToString(),
										*Texture->GetPathName(),
										*SamplerTypeDisplayName));
								}
								if (Expression && ((Expression->SamplerType == (EMaterialSamplerType)TC_Normalmap || Expression->SamplerType == (EMaterialSamplerType)TC_Masks) && Texture->SRGB))
								{
									FString SamplerTypeDisplayName = SamplerTypeEnum->GetDisplayNameTextByValue(Expression->SamplerType).ToString();
									OnScreenMessages.Emplace(FLinearColor(1, 1, 0),
										FString::Printf(TEXT("Warning: %s samples texture as '%s'. SRGB should be disabled for '%s'."),
											*TextureParameterValue->ParameterInfo.Name.ToString(),
											*SamplerTypeDisplayName,
											*Texture->GetPathName()));
								}
							}
						}
					}

					UDEditorRuntimeVirtualTextureParameterValue* RuntimeVirtualTextureParameterValue = Cast<UDEditorRuntimeVirtualTextureParameterValue>(Group.Parameters[ParameterIndex]);
					if (RuntimeVirtualTextureParameterValue && RuntimeVirtualTextureParameterValue->ExpressionId.IsValid())
					{
						URuntimeVirtualTexture* RuntimeVirtualTexture = NULL;
						MaterialEditorInstance->SourceInstance->GetRuntimeVirtualTextureParameterValue(RuntimeVirtualTextureParameterValue->ParameterInfo, RuntimeVirtualTexture);
						if (RuntimeVirtualTexture)
						{
							UMaterialExpressionRuntimeVirtualTextureSampleParameter* Expression = nullptr;

							TObjectPtr<UMaterialExpression>* MapExpression = MaterialGuids.Find(RuntimeVirtualTextureParameterValue->ExpressionId);
							if (MapExpression)
							{
								Expression = Cast<UMaterialExpressionRuntimeVirtualTextureSampleParameter>(*MapExpression);
							}

							if (!Expression)
							{
								const FText ExpressionNameText = FText::Format(LOCTEXT("MissingRVTExpression", "Warning: Runtime Virtual Texture Expression {0} not found."), FText::FromName(RuntimeVirtualTextureParameterValue->ParameterInfo.Name));
								OnScreenMessages.Emplace(FLinearColor(1, 1, 0), ExpressionNameText.ToString());
							}
							if (Expression && Expression->MaterialType != RuntimeVirtualTexture->GetMaterialType())
							{
								FString BaseMaterialTypeDisplayName = MaterialTypeEnum->GetDisplayNameTextByValue((int64)(Expression->MaterialType)).ToString();
								FString OverrideMaterialTypeDisplayName = MaterialTypeEnum->GetDisplayNameTextByValue((int64)(RuntimeVirtualTexture->GetMaterialType())).ToString();

								OnScreenMessages.Emplace(FLinearColor(1, 1, 0),
									FText::Format(LOCTEXT("MismatchedRVTType", "Warning: '{0}' interprets the virtual texture as '{1}' not '{2}', {3}"),
									FText::FromName(RuntimeVirtualTextureParameterValue->ParameterInfo.Name),
									FText::FromString(BaseMaterialTypeDisplayName),
									FText::FromString(OverrideMaterialTypeDisplayName),
									FText::FromString(RuntimeVirtualTexture->GetPathName())).ToString());
							}
							if (Expression && Expression->bSinglePhysicalSpace != RuntimeVirtualTexture->GetSinglePhysicalSpace())
							{
								OnScreenMessages.Emplace(FLinearColor(1, 1, 0),
									FText::Format(LOCTEXT("VirtualTexturePagePackingWarning", "Warning: '{0}' interprets the virtual texture page table packing as {1} not {2}, {3}"),
									FText::FromName(RuntimeVirtualTextureParameterValue->ParameterInfo.Name),
									FText::FromString(RuntimeVirtualTexture->GetSinglePhysicalSpace() ? TEXT("true") : TEXT("false")),
									FText::FromString(Expression->bSinglePhysicalSpace ? TEXT("true") : TEXT("false")),
									FText::FromString(RuntimeVirtualTexture->GetPathName())).ToString());
							}
							if (Expression && Expression->bAdaptive != RuntimeVirtualTexture->GetAdaptivePageTable())
							{
								OnScreenMessages.Emplace(FLinearColor(1, 1, 0),
									FText::Format(LOCTEXT("VirtualTextureAdaptiveWarning", "Warning: '{0}' interprets the adaptive page table setting as {1} not {2}, {3}"),
									FText::FromName(RuntimeVirtualTextureParameterValue->ParameterInfo.Name),
									FText::FromString(RuntimeVirtualTexture->GetAdaptivePageTable() ? TEXT("true") : TEXT("false")),
									FText::FromString(Expression->bAdaptive ? TEXT("true") : TEXT("false")),
									FText::FromString(RuntimeVirtualTexture->GetPathName())).ToString());
							}
						}
					}

					UDEditorSparseVolumeTextureParameterValue * SparseVolumeTextureParameterValue = Cast<UDEditorSparseVolumeTextureParameterValue>(Group.Parameters[ParameterIndex]);
					if (SparseVolumeTextureParameterValue && SparseVolumeTextureParameterValue->ExpressionId.IsValid())
					{
						USparseVolumeTexture* SparseVolumeTexture = NULL;
						MaterialEditorInstance->SourceInstance->GetSparseVolumeTextureParameterValue(SparseVolumeTextureParameterValue->ParameterInfo, SparseVolumeTexture);
						if (SparseVolumeTexture)
						{
							UMaterialExpressionSparseVolumeTextureSampleParameter* Expression = nullptr;

							TObjectPtr<UMaterialExpression>* MapExpression = MaterialGuids.Find(SparseVolumeTextureParameterValue->ExpressionId);
							if (MapExpression)
							{
								Expression = Cast<UMaterialExpressionSparseVolumeTextureSampleParameter>(*MapExpression);
							}

							if (!Expression)
							{
								const FText ExpressionNameText = FText::Format(LOCTEXT("MissingSVTExpression", "Warning: Sparse Volume Texture Expression {0} not found."), FText::FromName(SparseVolumeTextureParameterValue->ParameterInfo.Name));
								OnScreenMessages.Emplace(FLinearColor(1, 1, 0), ExpressionNameText.ToString());
							}
						}
					}
				}
			}
		}
	}
}

/**
 * Draws sampler/texture mismatch warning strings.
 * @param Canvas - The canvas on which to draw.
 * @param DrawPositionY - The Y position at which to draw. Upon return contains the Y value following the last line of text drawn.
 */
void FMaterialInstanceEditor::DrawOnScreenMessages(FCanvas* Canvas, int32& DrawPositionY)
{
	UFont* FontToUse = GEngine->GetTinyFont();
	for (const FOnScreenMessage& Message : OnScreenMessages)
	{
		Canvas->DrawShadowedString(
			5,
			DrawPositionY,
			*Message.Message,
			FontToUse,
			Message.Color);
		DrawPositionY += SpacingBetweenLines;
	}
}

bool FMaterialInstanceEditor::SetPreviewAsset(UObject* InAsset)
{
	if (PreviewVC.IsValid())
	{
		return PreviewVC->SetPreviewAsset(InAsset);
	}
	return false;
}

bool FMaterialInstanceEditor::SetPreviewAssetByName(const TCHAR* InAssetName)
{
	if (PreviewVC.IsValid())
	{
		return PreviewVC->SetPreviewAssetByName(InAssetName);
	}
	return false;
}

void FMaterialInstanceEditor::SetPreviewMaterial(UMaterialInterface* InMaterialInterface)
{
	if (PreviewVC.IsValid())
	{
		PreviewVC->SetPreviewMaterial(InMaterialInterface);
	}
}

void FMaterialInstanceEditor::GetShowHiddenParameters(bool& bShowHiddenParameters)
{
	bShowHiddenParameters = bShowAllMaterialParameters;
}

void FMaterialInstanceEditor::Tick(float DeltaTime)
{
	MaterialStatsManager->SetMaterial(MaterialEditorInstance->SourceInstance);
	MaterialStatsManager->Update();

	if (NumShadersPillText.IsValid())
	{
		// Update the Num Shaders text in the viewport.
		if (MaterialEditorInstance->SourceInstance->bHasStaticPermutationResource)
		{
			EShaderPlatform Platform = GEditor->GetActiveShaderPlatform();
			if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform))
			{
				Platform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(Platform);
			}
			NumShadersPillText->SetText(MaterialStatsManager->GetShaderCountText(Platform, GetCachedScalabilityCVars().MaterialQualityLevel, 0));
		}
		else
		{
			NumShadersPillText->SetText(NSLOCTEXT("MaterialInstanceEditor", "NumShadersText", "Shaders: 0"));
		}
	}
}

TStatId FMaterialInstanceEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FMaterialInstanceEditor, STATGROUP_Tickables);
}

void FMaterialInstanceEditor::SaveAsset_Execute()
{
	if (bIsFunctionPreviewMaterial && MaterialEditorInstance)
	{
		UE_LOGF(LogMaterialInstanceEditor, Log, "Saving and applying instance %ls", *GetEditingObjects()[0]->GetName());
		MaterialEditorInstance->ApplySourceFunctionChanges();
	}

	IMaterialEditor::SaveAsset_Execute();
}

void FMaterialInstanceEditor::SaveAssetAs_Execute()
{
	if (bIsFunctionPreviewMaterial && MaterialEditorInstance)
	{
		UE_LOGF(LogMaterialInstanceEditor, Log, "Saving and applying instance %ls", *GetEditingObjects()[0]->GetName());
		MaterialEditorInstance->ApplySourceFunctionChanges();
	}

	IMaterialEditor::SaveAssetAs_Execute();
}

void FMaterialInstanceEditor::SaveSettings()
{
	GConfig->SetBool(TEXT("MaterialInstanceEditor"), TEXT("bDrawGrid"), PreviewVC->IsRealtime(), GEditorPerProjectIni);

	if (UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
	{
		MaterialEditorOptions->SaveConfig();
	}
}

void FMaterialInstanceEditor::LoadSettings()
{
	bool bRealtime=false;
	GConfig->GetBool(TEXT("MaterialInstanceEditor"), TEXT("bDrawGrid"), bRealtime, GEditorPerProjectIni);

	if(PreviewVC.IsValid())
	{
		if ( bRealtime )
		{
			PreviewVC->OnToggleRealtime();
		}
	}
}

void FMaterialInstanceEditor::OpenSelectedParentEditor(UMaterialInterface* InMaterialInterface)
{
	ensure(InMaterialInterface);

	// See if its a material or material instance constant.  Don't do anything if the user chose the current material instance.
	if(InMaterialInterface && MaterialEditorInstance->SourceInstance!=InMaterialInterface)
	{
		if(InMaterialInterface->IsA(UMaterial::StaticClass()))
		{
			// Show material editor
			UMaterial* Material = Cast<UMaterial>(InMaterialInterface);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Material);
		}
		else if(InMaterialInterface->IsA(UMaterialInstance::StaticClass()))
		{
			// Show material instance editor
			UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(InMaterialInterface);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(MaterialInstance);
		}
	}
}

void FMaterialInstanceEditor::OpenSelectedParentEditor(UMaterialFunctionInterface* InMaterialFunction)
{
	ensure(InMaterialFunction);

	// See if its a material or material instance constant.  Don't do anything if the user chose the current material instance.
	if(InMaterialFunction && MaterialFunctionOriginal != InMaterialFunction)
	{
		if(InMaterialFunction->IsA(UMaterialFunctionInstance::StaticClass()))
		{
			// Show function instance editor
			UMaterialFunctionInstance* FunctionInstance = Cast<UMaterialFunctionInstance>(InMaterialFunction);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(FunctionInstance);
		}
		else
		{
			// Show function editor
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InMaterialFunction);
		}
	}
}

void FMaterialInstanceEditor::UpdatePropertyWindow()
{
	TArray<UObject*> SelectedObjects;
	SelectedObjects.Add( MaterialEditorInstance );
	MaterialInstanceDetails->SetObjects( SelectedObjects, true );
	if (MaterialLayersFunctionsInstance.IsValid())
	{
		MaterialLayersFunctionsInstance->SetEditorInstance(MaterialEditorInstance);
	}
}

UObject* FMaterialInstanceEditor::GetSyncObject()
{
	if (MaterialEditorInstance)
	{
		return MaterialEditorInstance->SourceInstance;
	}
	return NULL;
}

bool FMaterialInstanceEditor::ApproveSetPreviewAsset(UObject* InAsset)
{
	// Default impl is to always accept.
	return true;
}

void FMaterialInstanceEditor::Refresh()
{
	int32 TempIndex;
	const bool bParentChanged = !MaterialParentList.Find( ToRawPtr(MaterialEditorInstance->Parent), TempIndex );

	PreviewVC->RefreshViewport();

	if( bParentChanged )
	{
		RebuildInheritanceList();
	}
	
	UpdatePropertyWindow();

	RefreshOnScreenTextureWarnings();
}

void FMaterialInstanceEditor::PostUndo( bool bSuccess )
{
	MaterialEditorInstance->CopyToSourceInstance();
	RefreshPreviewAsset();
	Refresh();
}

void FMaterialInstanceEditor::PostRedo( bool bSuccess )
{
	MaterialEditorInstance->CopyToSourceInstance();
	RefreshPreviewAsset();
	Refresh();
}

void FMaterialInstanceEditor::NotifyExternalMaterialChange()
{
	MaterialStatsManager->SignalMaterialChanged();
	Refresh();
}

void FMaterialInstanceEditor::NotifyUserSceneTextureLoadOrUnload()
{
	if (PreviewVC.IsValid() && PreviewVC->PreviewMaterial)
	{
		UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(PreviewVC->PreviewMaterial);
		if (MaterialInstance)
		{
			UMaterial* BaseMaterial = MaterialInstance->GetMaterial();
			if (BaseMaterial && BaseMaterial->IsPostProcessMaterial())
			{
				FName Output = MaterialInstance->GetUserSceneTextureOutput(BaseMaterial);

				// Ignore special SceneColor output name -- this just writes to SceneColor, not a transient UserSceneTexture
				if (!Output.IsNone() && Output != FName("SceneColor"))
				{
					FMaterialEditorUtilities::RefreshPostProcessPreviewMaterials(MaterialInstance);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

