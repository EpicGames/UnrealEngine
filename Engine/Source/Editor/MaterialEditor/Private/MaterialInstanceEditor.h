// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Misc/NotifyHook.h"
#include "EditorUndoClient.h"
#include "Toolkits/IToolkitHost.h"
#include "IMaterialEditor.h"
#include "IDetailsView.h"
#include "SMaterialEditorViewport.h"
#include "Widgets/Views/STableRow.h"

#include "TickableEditorObject.h"

class FCanvas;
class FObjectPreSaveContext;
class UToolMenu;
class UMaterialEditorInstanceConstant;
class UMaterialInterface;
class UMaterialInstance;
class UMaterialInstanceConstant;
class UMaterialFunctionInterface;
class UMaterialFunctionInstance;
template <typename ItemType> class SListView;

/**
 * Material Instance Editor class
 */
class FMaterialInstanceEditor : public IMaterialEditor, public FNotifyHook, public FGCObject, public FEditorUndoClient, public FTickableEditorObject
{
public:
	
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Initializes the editor to use a material. Should be the first thing called. */
	void InitEditorForMaterial(UMaterialInstance* InMaterial);

	/** Initializes the editor to use a material function. Should be the first thing called. */
	void InitEditorForMaterialFunction(UMaterialFunctionInstance* InMaterialFunction);

	/**
	 * Edits the specified material instance object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The material instance to edit
	 */
	void InitMaterialInstanceEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit );

	FMaterialInstanceEditor();

	virtual ~FMaterialInstanceEditor();

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMaterialInstanceEditor");
	}

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void InitToolMenuContext(struct FToolMenuContext& MenuContext) override;

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** The material instance applied to the preview mesh. */
	virtual UMaterialInterface* GetMaterialInterface() const override;

	/** Pre edit change notify for properties. */
	virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override;

	/** Post edit change notify for properties. */
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;

	void PreSavePackage(UPackage* Obj, FObjectPreSaveContext ObjectSaveContext);

	/** Rebuilds the inheritance list for this material instance. */
	void RebuildInheritanceList();

	/** Rebuilds the editor when the original material changes */
	void RebuildMaterialInstanceEditor();

	/**
	 * Draws messages on the specified viewport and canvas.
	 */
	virtual void DrawMessages( FViewport* Viewport, FCanvas* Canvas ) override;

	/**
	 * Draws generic on screen messages.
	 * @param Canvas - The canvas on which to draw.
	 * @param DrawPositionY - The Y position at which to draw. Upon return contains the Y value following the last line of text drawn.
	 */
	void DrawOnScreenMessages(FCanvas* Canvas, int32& DrawPositionY);

	/** Passes instructions to the preview viewport */
	bool SetPreviewAsset(UObject* InAsset);
	bool SetPreviewAssetByName(const TCHAR* InMeshName);
	void SetPreviewMaterial(UMaterialInterface* InMaterialInterface);

	/** Returns true if hidden parameters should be shown */
	void GetShowHiddenParameters(bool& bShowHiddenParameters);

	/** Gets the extensibility managers for outside entities to extend material editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() { return ToolBarExtensibilityManager; }

	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

	/** call this to notify the editor that the edited material changed from outside */
	virtual void NotifyExternalMaterialChange() override;

	// IMaterial Editor Interface
	virtual TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const override;
	virtual void GenerateInheritanceMenu(class UToolMenu* Menu) override;

	void RefreshPreviewViewport();

	/**	Refresh the viewport and property window */
	void Refresh();

	bool IsDestructing() const { return bDestructing; }

protected:
	//~ FAssetEditorToolkit interface
	virtual void SaveAsset_Execute() override;
	virtual void SaveAssetAs_Execute() override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;

	/** Saves editor settings. */
	void SaveSettings();

	/** Loads editor settings. */
	void LoadSettings();

	/** Opens the editor for the selected parent. */
	void OpenSelectedParentEditor(UMaterialInterface* InMaterialInterface);
	void OpenSelectedParentEditor(UMaterialFunctionInterface* InMaterialFunction);

	/** Updated the properties pane */
	void UpdatePropertyWindow();

	virtual UObject* GetSyncObject();

private:
	/** Binds our UI commands to delegates */
	void BindCommands();

	/** Command for the apply button */
	void OnApply();
	bool OnApplyEnabled() const;
	bool OnApplyVisible() const;


	/** Commands for the ShowAllMaterialParametersEnabled button */
	void ToggleShowAllMaterialParameters();
	bool IsShowAllMaterialParametersChecked() const;

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Delegate for overriding Show Modified to Show Overridden instead */
	void FilterOverriddenProperties();

	/** Updates the 3D and UI preview viewport visibility based on material domain */
	void UpdatePreviewViewportsVisibility();

	void RegisterToolBar();
	/** Builds the toolbar widget for the material editor */
	void ExtendToolbar();

	/** If re-initializing for a material function instance re-generate the proxy materials */
	void ReInitMaterialFunctionProxies();

	//~ Begin IMaterialEditor Interface
	virtual bool ApproveSetPreviewAsset(UObject* InAsset) override;

	/**	Spawns the preview tab */
	TSharedRef<SDockTab> SpawnTab_Preview( const FSpawnTabArgs& Args );

	/**	Spawns the properties tab */
	TSharedRef<SDockTab> SpawnTab_Properties( const FSpawnTabArgs& Args );

	/**	Spawns the properties tab */
	TSharedRef<SDockTab> SpawnTab_LayerProperties(const FSpawnTabArgs& Args);

	/** Spawns the advanced preview settings tab */
	TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	
	/**	Caches the specified tab for later retrieval */
	void AddToSpawnedToolPanels( const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab );

	/** Refreshes the preview asset */
	void RefreshPreviewAsset();

	void RefreshOnScreenTextureWarnings();

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;
	// End of FEditorUndoClient

	/** Called to notify other material editors when a post process material with a UserSceneTexture output is loaded or unloaded, as the material may be used in their preview */
	void NotifyUserSceneTextureLoadOrUnload();

private:
	struct FOnScreenMessage
	{
		FOnScreenMessage() = default;
		FOnScreenMessage(const FLinearColor& InColor, const FString& InMessage) : Message(InMessage), Color(InColor) {}

		FString Message;
		FLinearColor Color;
	};

	/** List of open tool panels; used to ensure only one exists at any one time */
	TMap< FName, TWeakPtr<class SDockTab> > SpawnedToolPanels;

	/** Preview Viewport widget */
	TSharedPtr<class SMaterialEditor3DPreviewViewport> PreviewVC;

	/** Preview viewport widget used for UI materials */
	TSharedPtr<class SMaterialEditorUIPreviewViewport> PreviewUIViewport;

	/** Property View */
	TSharedPtr<class IDetailsView> MaterialInstanceDetails;

	/** Layer Properties View */
	TSharedPtr<class SMaterialLayersFunctionsInstanceWrapper> MaterialLayersFunctionsInstance;

	/** List of parents used to populate the inheritance list chain. */
	TArray< FAssetData > MaterialParentList;

	/** List of parents used to populate the inheritance list chain. */
	TArray< FAssetData > FunctionParentList;

	/** List of children used to populate the inheritance list chain. */
	TArray< FAssetData > MaterialChildList;

	/** List of children used to populate the inheritance list chain. */
	TArray< FAssetData > FunctionChildList;
	
	/** List of all current on-screen messages to display */
	TArray<FOnScreenMessage> OnScreenMessages;

	/** Object that stores all of the possible parameters we can edit. */
	TObjectPtr<UMaterialEditorInstanceConstant> MaterialEditorInstance;

	/** Whether or not we should be displaying all the material parameters */
	bool bShowAllMaterialParameters;

	/** If editing instance of a function instead of a material. */
	bool bIsFunctionPreviewMaterial;

	/** Set to true when editor is being destructed */
	bool bDestructing = false;

	/** The Substrate control tab */
	TSharedPtr<class SMaterialEditorSubstrateWidget> SubstrateWidget;
	
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	UMaterialFunctionInstance*	MaterialFunctionOriginal;
	UMaterialFunctionInstance*	MaterialFunctionInstance;
	UMaterial*					FunctionMaterialProxy; 
	UMaterialInstanceConstant*	FunctionInstanceProxy; 

	/** Object used as material statistics manager */
	TSharedPtr<class FMaterialStats> MaterialStatsManager;

	/** The text for the number of shaders in the material. */
	TSharedPtr<STextBlock> NumShadersPillText;
};
