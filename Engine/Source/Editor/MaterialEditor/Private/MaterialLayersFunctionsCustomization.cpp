// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialLayersFunctionsCustomization.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"

#include "Engine/GameViewportClient.h"

#include "PropertyHandle.h"
#include "Sound/SoundBase.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "DetailLayoutBuilder.h"

#include "Editor.h"
#include "ScopedTransaction.h"
#include "IDetailGroup.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/SlateDelegates.h"
#include "Misc/Attribute.h"
#include "Styling/SlateTypes.h"
#include "RenderUtils.h"
#include "MaterialPropertyHelpers.h"



#define LOCTEXT_NAMESPACE "MaterialLayerCustomization"

FMaterialLayersFunctionsCustomization::FMaterialLayersFunctionsCustomization(const TSharedPtr<class IPropertyHandle>& StructPropertyHandle, const class IDetailLayoutBuilder* InDetailLayout)
{
	// Save data for later re-use
	SavedStructPropertyHandle = ConstCastSharedPtr<IPropertyHandle>(StructPropertyHandle);
	SavedLayoutBuilder = const_cast<IDetailLayoutBuilder*>(InDetailLayout);
	PropertyUtilities = SavedLayoutBuilder->GetPropertyUtilities();
	TArray<void*> StructPtrs;
	SavedStructPropertyHandle->AccessRawData(StructPtrs);
	auto It = StructPtrs.CreateConstIterator();
	MaterialLayersFunctions = reinterpret_cast<FMaterialLayersFunctions*>(*It);

	FSimpleDelegate RebuildChildrenDelegate = FSimpleDelegate::CreateRaw(this, &FMaterialLayersFunctionsCustomization::RebuildChildren);
	LayerHandle = SavedStructPropertyHandle->GetChildHandle("Layers").ToSharedRef();
	BlendHandle = SavedStructPropertyHandle->GetChildHandle("Blends").ToSharedRef();

	// UI will be refreshed twice, but this means layers and blends can be adjusted in an order-agnostic way
	LayerHandle->AsArray()->SetOnNumElementsChanged(RebuildChildrenDelegate);
	BlendHandle->AsArray()->SetOnNumElementsChanged(RebuildChildrenDelegate);

#if WITH_EDITOR
	//Fixup for adding new bool arrays to the class
	if (MaterialLayersFunctions)
	{
		if (MaterialLayersFunctions->Layers.Num() != MaterialLayersFunctions->EditorOnly.RestrictToLayerRelatives.Num())
		{
			int32 OriginalSize = MaterialLayersFunctions->EditorOnly.RestrictToLayerRelatives.Num();
			for (int32 LayerIt = 0; LayerIt < MaterialLayersFunctions->Layers.Num() - OriginalSize; LayerIt++)
			{
				MaterialLayersFunctions->EditorOnly.RestrictToLayerRelatives.Add(false);
			}
		}
		if (MaterialLayersFunctions->Blends.Num() != MaterialLayersFunctions->EditorOnly.RestrictToBlendRelatives.Num())
		{
			int32 OriginalSize = MaterialLayersFunctions->EditorOnly.RestrictToBlendRelatives.Num();
			for (int32 BlendIt = 0; BlendIt < MaterialLayersFunctions->Blends.Num() - OriginalSize; BlendIt++)
			{
				MaterialLayersFunctions->EditorOnly.RestrictToBlendRelatives.Add(false);
			}
		}
	}
#endif
}

void FMaterialLayersFunctionsCustomization::ResetToDefault()
{
	const FScopedTransaction Transaction(LOCTEXT("ResetMaterialLayersFunctions", "Reset all Layers and Blends"));
	SavedStructPropertyHandle->NotifyPreChange();
	*MaterialLayersFunctions = FMaterialLayersFunctions();
	SavedStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	// Refresh the header so the reset to default button is no longer visible
	SavedLayoutBuilder->ForceRefreshDetails();
}

bool FMaterialLayersFunctionsCustomization::IsResetToDefaultVisible() const
{
	FString DisplayString = TEXT("None");
	if (MaterialLayersFunctions->Layers[0] != nullptr)
	{
		MaterialLayersFunctions->Layers[0]->GetName(DisplayString);
	}
	// Can reset if:
	// There is more than one layer OR
	// There is only one layer but it has a material layer set
	return (MaterialLayersFunctions->Layers.Num() != 1) || 
		(MaterialLayersFunctions->Layers.Num() == 1 && DisplayString != TEXT("None"));
}

void FMaterialLayersFunctionsCustomization::RebuildChildren()
{
	OnRebuildChildren.ExecuteIfBound();
}

void FMaterialLayersFunctionsCustomization::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	TAttribute<bool>::FGetter IsResetButtonEnabledDelegate = TAttribute<bool>::FGetter::CreateSP(this, &FMaterialLayersFunctionsCustomization::IsResetToDefaultVisible);
	TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::Create(IsResetButtonEnabledDelegate);

	NodeRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(3.0f))
			[
				SavedStructPropertyHandle->CreatePropertyNameWidget()
			]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeResetButton(FSimpleDelegate::CreateSP(this, &FMaterialLayersFunctionsCustomization::ResetToDefault), FText(), IsEnabledAttribute)
		]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.VAlign(VAlign_Top)
			[
				PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &FMaterialLayersFunctionsCustomization::AddLayer))
			]

		];
}

void FMaterialLayersFunctionsCustomization::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	uint32 LayerChildren;
	LayerHandle->GetNumChildren(LayerChildren);
	uint32 BlendChildren;
	BlendHandle->GetNumChildren(BlendChildren);
	TSharedRef<SWidget> RemoveWidget = SNullWidget::NullWidget;
	FText GroupName = MaterialLayersFunctions->GetLayerName(LayerChildren - 1);
	FName GroupFName = FName(*(GroupName.ToString()));
	
	IDetailGroup& Group = ChildrenBuilder.AddGroup(GroupFName, GroupName);

	// Users can only remove layers
	// You can never have fewer than one layer
	DetailGroups.Add(&Group);
	FDetailWidgetRow& NewLayerRow = Group.AddWidgetRow();
	TSharedRef<class FMaterialLayerFunctionElement> LayerLayout = MakeShareable(new FMaterialLayerFunctionElement(this, LayerHandle->AsArray()->GetElement(LayerChildren - 1), EMaterialLayerRowType::Layer));
	LayerLayout->GenerateHeaderRowContent(NewLayerRow);
	if (BlendChildren > 0)
	{
		RemoveWidget = PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FMaterialLayersFunctionsCustomization::RemoveLayer, (int32)LayerChildren - 1));
		Group.HeaderRow()
			.NameContent()
			[
				SNew(SInlineEditableTextBlock)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FMaterialLayersFunctionsCustomization::GetLayerName, (int32)LayerChildren - 1)))
				.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FMaterialLayersFunctionsCustomization::OnNameChanged, (int32)LayerChildren-1))
				.Font(FAppStyle::GetFontStyle(TEXT("MaterialEditor.Layers.EditableFont")))
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNullWidget::NullWidget
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					RemoveWidget
				]
			];
	}
	//Only draw the rows when the correct number of blends and layers are present
	if (BlendChildren > 0 && LayerChildren > BlendChildren)
	{
		for (int32 Counter = BlendChildren - 1; Counter >= 0; Counter--)
		{
			FDetailWidgetRow& NewBlendRow = DetailGroups.Last()->AddWidgetRow();
			TSharedRef<class FMaterialLayerFunctionElement> BlendLayout = MakeShareable(new FMaterialLayerFunctionElement(this, BlendHandle->AsArray()->GetElement(Counter), EMaterialLayerRowType::Blend));
			BlendLayout->GenerateHeaderRowContent(NewBlendRow);
			GroupName = MaterialLayersFunctions->GetLayerName(Counter);
			GroupFName = FName(*(GroupName.ToString()));
			IDetailGroup& NewGroup = ChildrenBuilder.AddGroup(GroupFName, GroupName);
			DetailGroups.Add(&NewGroup);

			if (Counter > 0)
			{
				RemoveWidget = PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FMaterialLayersFunctionsCustomization::RemoveLayer, Counter));

				NewGroup.HeaderRow()
					.NameContent()
					[
						SNew(SInlineEditableTextBlock)
						.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FMaterialLayersFunctionsCustomization::GetLayerName, Counter)))
						.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FMaterialLayersFunctionsCustomization::OnNameChanged, Counter))
						.Font(FAppStyle::GetFontStyle(TEXT("MaterialEditor.Layers.EditableFont")))
					]
					.ValueContent()
					.HAlign(HAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SNullWidget::NullWidget
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							RemoveWidget
						]
					];
			}

			FDetailWidgetRow& NextLayerRow = DetailGroups.Last()->AddWidgetRow();
			LayerLayout = MakeShareable(new FMaterialLayerFunctionElement(this, LayerHandle->AsArray()->GetElement(Counter), EMaterialLayerRowType::Layer));
			LayerLayout->GenerateHeaderRowContent(NextLayerRow);
		}
	}

}

#if WITH_EDITOR


FText FMaterialLayersFunctionsCustomization::GetLayerName(int32 Counter) const
{
	return MaterialLayersFunctions->GetLayerName(Counter);
}

void FMaterialLayersFunctionsCustomization::OnNameChanged(const FText& InText, ETextCommit::Type CommitInfo, int32 Counter)
{
	const FScopedTransaction Transaction(LOCTEXT("RenamedSection", "Renamed layer and blend section"));
	SavedStructPropertyHandle->NotifyPreChange();
	MaterialLayersFunctions->EditorOnly.LayerNames[Counter] = InText;
	SavedStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
};
#endif

FString FMaterialLayersFunctionsCustomization::GetLayerAssetPath(FMaterialParameterInfo InInfo) const
{
	FString AssetPath;
	if (InInfo.Association == EMaterialParameterAssociation::BlendParameter && MaterialLayersFunctions->Blends.IsValidIndex(InInfo.Index))
	{
		AssetPath = MaterialLayersFunctions->Blends[InInfo.Index]->GetPathName();
	}
	else if (InInfo.Association == EMaterialParameterAssociation::LayerParameter && MaterialLayersFunctions->Layers.IsValidIndex(InInfo.Index))
	{
		AssetPath = MaterialLayersFunctions->Layers[InInfo.Index]->GetPathName();
	}
	return AssetPath;
}

void FMaterialLayersFunctionsCustomization::AddLayer()
{
	const FScopedTransaction Transaction(LOCTEXT("AddLayerAndBlend", "Add a new Layer and a Blend into it"));
	SavedStructPropertyHandle->NotifyPreChange();
	MaterialLayersFunctions->AppendBlendedLayer();
	SavedStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
}

void FMaterialLayersFunctionsCustomization::RemoveLayer(int32 Index)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveLayerAndBlend", "Remove a Layer and the attached Blend"));
	SavedStructPropertyHandle->NotifyPreChange();
	MaterialLayersFunctions->RemoveBlendedLayerAt(Index);
	SavedStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayRemove);
}


void FMaterialLayersFunctionsCustomization::RefreshOnAssetChange(const struct FAssetData& InAssetData, int32 Index, EMaterialParameterAssociation MaterialType)
{
	FMaterialPropertyHelpers::OnMaterialLayerAssetChanged(InAssetData, Index, MaterialType, SavedStructPropertyHandle, MaterialLayersFunctions);

	if (Index == 0 && MaterialType == EMaterialParameterAssociation::LayerParameter)
	{
		// Refresh the header so the reset to default button is no longer visible
		SavedLayoutBuilder->ForceRefreshDetails();
	}
	RebuildChildren();
}

FMaterialLayerFunctionElement::FMaterialLayerFunctionElement(FMaterialLayersFunctionsCustomization* InCustomization, TWeakPtr<IPropertyHandle> InPropertyHandle, EMaterialLayerRowType InRowType)
{
	ParentCustomization = InCustomization;
	RowPropertyHandle = InPropertyHandle.Pin();
	RowType = InRowType;
	if (InRowType == EMaterialLayerRowType::Layer)
	{
		RowPropertyHandle->GetProperty()->SetMetaData(FName(TEXT("DisplayThumbnail")), TEXT("true"));
	}
}


void FMaterialLayerFunctionElement::ResetLayerAssetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TSharedPtr<IPropertyHandle> InPropertyHandle, FMaterialLayersFunctionsCustomization* InCustomization, int32 InIndex, EMaterialLayerRowType MaterialType)
{
	const FScopedTransaction Transaction(LOCTEXT("ResetLayerOrBlend", "Reset Layer/Blend Value"));
	InPropertyHandle->NotifyPreChange();

	switch (MaterialType)
	{
	case EMaterialLayerRowType::Layer:
	{
		InCustomization->GetMaterialLayersFunctions()->Layers[InIndex] = nullptr;
		break;
	}
	case EMaterialLayerRowType::Blend:
	{
		InCustomization->GetMaterialLayersFunctions()->Blends[InIndex] = nullptr;
		break;
	}
	}

	InPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	InCustomization->RebuildChildren();
}

bool FMaterialLayerFunctionElement::CanResetLayerAssetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TSharedPtr<IPropertyHandle> InPropertyHandle, FMaterialLayersFunctionsCustomization* InCustomization, int32 InIndex, EMaterialLayerRowType MaterialType)
{
	UObject* StoredObject = nullptr;
	switch (MaterialType)
	{
	case EMaterialLayerRowType::Layer:
	{
		if (InCustomization->GetMaterialLayersFunctions()->Layers.IsValidIndex(InIndex))
		{
			StoredObject = InCustomization->GetMaterialLayersFunctions()->Layers[InIndex];
		}
		break;
	}
	case EMaterialLayerRowType::Blend:
	{
		if (InCustomization->GetMaterialLayersFunctions()->Blends.IsValidIndex(InIndex))
		{
			StoredObject = InCustomization->GetMaterialLayersFunctions()->Blends[InIndex];
		}
		break;
	}
	}
	return StoredObject != nullptr;
}

void FMaterialLayerFunctionElement::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	Index = 0;
	FText DisplayText = FText();
	if (RowPropertyHandle.IsValid())
	{
		Index = FCString::Atoi(*(RowPropertyHandle.Get()->GetPropertyDisplayName().ToString()));
	}
	switch (RowType)
	{
	case EMaterialLayerRowType::Layer:
	{
		DisplayText = FText(LOCTEXT("LayerAsset", "Layer Asset"));
	}
	break;
	case EMaterialLayerRowType::Blend:
	{
		DisplayText = FText(LOCTEXT("BlendAsset", "Blend Asset"));
	}
	break;
	default:
		break;
	}

	EMaterialParameterAssociation InAssociation = EMaterialParameterAssociation::GlobalParameter;
	if (RowType == EMaterialLayerRowType::Blend)
	{
		InAssociation = EMaterialParameterAssociation::BlendParameter;
	}
	if (RowType == EMaterialLayerRowType::Layer)
	{
		InAssociation = EMaterialParameterAssociation::LayerParameter;
	}

	TSharedRef<SWidget> PlaceholderIcon = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(10.0f);

	uint32 LayerChildren;
	RowPropertyHandle->GetParentHandle().ToSharedRef().Get().GetNumChildren(LayerChildren);

	FMargin AssetPadding = FMargin(0.0f);

	FIsResetToDefaultVisible IsAssetResetVisible = FIsResetToDefaultVisible::CreateStatic(&FMaterialLayerFunctionElement::CanResetLayerAssetToDefault, RowPropertyHandle, ParentCustomization, Index, RowType);
	FResetToDefaultHandler ResetAssetHandler = FResetToDefaultHandler::CreateStatic(&FMaterialLayerFunctionElement::ResetLayerAssetToDefault, RowPropertyHandle, ParentCustomization, Index, RowType);
	FResetToDefaultOverride ResetAssetOverride = FResetToDefaultOverride::Create(IsAssetResetVisible, ResetAssetHandler);

	FOnShouldFilterAsset AssetFilter = FOnShouldFilterAsset::CreateStatic(&FMaterialPropertyHelpers::FilterLayerAssets, ParentCustomization->GetMaterialLayersFunctions(), InAssociation, Index);

	FIntPoint ThumbnailOverride;
	if (RowType == EMaterialLayerRowType::Layer)
	{
		ThumbnailOverride = FIntPoint(64, 64);
	}
	else if (RowType == EMaterialLayerRowType::Blend)
	{
		ThumbnailOverride = FIntPoint(32, 32);
	}
	FOnSetObject AssetChanged = FOnSetObject::CreateSP(ParentCustomization, &FMaterialLayersFunctionsCustomization::RefreshOnAssetChange, Index, InAssociation);

	FMaterialParameterInfo FunctionInfo;
	FunctionInfo.Index = Index;
	FunctionInfo.Association = InAssociation;

	NodeRow
		.OverrideResetToDefault(ResetAssetOverride)
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				RowPropertyHandle->CreatePropertyNameWidget(DisplayText)
			]
		]
		.ValueContent()
		.MinDesiredWidth(400.0)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UMaterialFunctionInterface::StaticClass())
				.ObjectPath(ParentCustomization, &FMaterialLayersFunctionsCustomization::GetLayerAssetPath, FunctionInfo)
				.OnObjectChanged(AssetChanged)
				.OnShouldFilterAsset(AssetFilter)
				.ThumbnailPool(ParentCustomization->GetPropertyUtilities()->GetThumbnailPool())
				.DisplayCompactSize(true)
				.ThumbnailSizeOverride(ThumbnailOverride)
				.NewAssetFactories(FMaterialPropertyHelpers::GetAssetFactories(InAssociation))
			]
		];
}


#undef LOCTEXT_NAMESPACE
