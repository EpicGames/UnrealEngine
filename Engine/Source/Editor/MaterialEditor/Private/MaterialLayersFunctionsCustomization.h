// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Materials/MaterialLayersFunctions.h"
#include "IDetailCustomNodeBuilder.h"
#include "Input/Reply.h"

struct FAssetData;
enum EMaterialLayerRowType
{
	Layer,
	Blend
};

/** Customize the appearance of an FMaterialLayersFunction */
class FMaterialLayersFunctionsCustomization : public IDetailCustomNodeBuilder, public TSharedFromThis< FMaterialLayersFunctionsCustomization >
{
public:
	FMaterialLayersFunctionsCustomization(const TSharedPtr<class IPropertyHandle>& StructPropertyHandle, const class IDetailLayoutBuilder* InDetailLayout);
	/** Adds a layer and the corresponding blend to the FMaterialLayerFunctions */
	void AddLayer();
	/** Removes a layer and any corresponding blend from the FMaterialLayerFunctions */
	void RemoveLayer(int32 Index);

	void RefreshOnAssetChange(const struct FAssetData& InAssetData, int32 Index, EMaterialParameterAssociation MaterialType);
	/** Getter for the FMaterialLayersFunctions for this customization */
	FMaterialLayersFunctions* GetMaterialLayersFunctions() const
	{
		return MaterialLayersFunctions;
	}
	TSharedPtr<IPropertyUtilities> GetPropertyUtilities() const
	{
		return PropertyUtilities.Pin();
	}
	void RebuildChildren();
protected:
	void OnObjectChanged(const FAssetData&) {};
	void ResetToDefault();
	bool IsResetToDefaultVisible() const;


	/** IDetailCustomNodeBuilder interface */
	virtual bool RequiresTick() const override { return false; }
	virtual void Tick(float DeltaTime) override {};
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual FName GetName() const override { return TEXT("MaterialLayerFunction"); }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override 
	{
		OnRebuildChildren = InOnRegenerateChildren;
	};

	/** Callback when user click the Group reset button */
	void OnDetailGroupReset() {};

#if WITH_EDITOR
	/* Returns the user-editable name of a layer within the Material Layers Function */
	FText GetLayerName(int32 Counter) const;

	/* Called when a layer is renamed by the user */
	void OnNameChanged(const FText& InText, ETextCommit::Type CommitInfo, int32 Counter);
#endif 

private:
	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;
	/** Layer array handle */
	TSharedPtr<IPropertyHandle> LayerHandle;
	/** Blend array handle */
	TSharedPtr<IPropertyHandle> BlendHandle;
	/** Property handle for the FMaterialLayerFunction */
	TSharedPtr<IPropertyHandle> SavedStructPropertyHandle;
	/** Layout builder that contains this custom builder */
	class IDetailLayoutBuilder* SavedLayoutBuilder;
	/** Associated FMaterialLayerFunction utilities */
	TWeakPtr<IPropertyUtilities> PropertyUtilities;
	/** FMaterialLayersFunctions instance this customization is currently editing */
	FMaterialLayersFunctions* MaterialLayersFunctions;
	TArray<class IDetailGroup*> DetailGroups;

public:
	FString GetLayerAssetPath(FMaterialParameterInfo InInfo) const;
};

class FMaterialLayerFunctionElement : public IDetailCustomNodeBuilder, public TSharedFromThis< FMaterialLayerFunctionElement >
{
public:

	FMaterialLayerFunctionElement(FMaterialLayersFunctionsCustomization* InCustomization, TWeakPtr<IPropertyHandle> InPropertyHandle, EMaterialLayerRowType InRowType);
	/** Callback when user click the reset button of the material row property */
	static void ResetLayerAssetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TSharedPtr<IPropertyHandle> InPropertyHandle, FMaterialLayersFunctionsCustomization* InCustomization, int32 InIndex, EMaterialLayerRowType MaterialType);

	static bool CanResetLayerAssetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TSharedPtr<IPropertyHandle> InPropertyHandle, FMaterialLayersFunctionsCustomization* InCustomization, int32 InIndex, EMaterialLayerRowType MaterialType);

	
	/** Getter for the index of this row */
	uint32 GetIndex() const { return Index; };
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;

	/** Function used to filter targets for the value of this row */

private:
	/** IDetailCustomNodeBuilder interface */
	virtual bool RequiresTick() const override { return false; }
	virtual void Tick(float DeltaTime) override {};
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override {};
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {};
	void OnDetailGroupReset() {};



private:
	/** The property handle for the property displayed by this row */
	TSharedPtr<IPropertyHandle> RowPropertyHandle;
	/** The parent customization that created this row */
	FMaterialLayersFunctionsCustomization* ParentCustomization;
	/** Whether this row is for a layer or a blend */
	EMaterialLayerRowType RowType;
	/** The array index of this row in the Layers or Blends array*/
	int32 Index;
};
