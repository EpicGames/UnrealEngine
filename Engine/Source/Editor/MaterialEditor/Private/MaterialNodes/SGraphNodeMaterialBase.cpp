// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/World.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialFunction.h"
#include "MaterialEditor.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RenderGraphUtils.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateRenderer.h"
#include "RenderingThread.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "TutorialMetaData.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UnrealClient.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"
#include "Rendering/SubstrateMaterialShared.h"
#include "MaterialEditor/SGraphSubstrateMaterial.h"
#include "MaterialShared.h"
#include "RenderGraphBuilder.h"
#include "RHIGPUReadback.h"

class FWidgetStyle;
class SWidget;
struct FSlateBrush;

static const FName NAME_Pin_NotConnectable("Graph.Pin.Dummy");
static const FSlateBrush* CacheImg_Pin_NotConnectable = nullptr;

/**
* Simple representation of the backbuffer that the preview canvas renders to
* This class may only be accessed from the render thread
*/
class FSlateMaterialPreviewRenderTarget : public FRenderTarget
{
public:
	/** FRenderTarget interface */
	virtual FIntPoint GetSizeXY() const
	{
		return ClippingRect.Size();
	}

	/** Sets the texture that this target renders to */
	void SetRenderTargetTexture(FRDGTexture* Texture)
	{
		RDGTexture = Texture;
	}

	/** Clears the render target texture */
	void ClearRenderTargetTexture()
	{
		RDGTexture = nullptr;
	}

	FRDGTextureRef GetRenderTargetTexture(FRDGBuilder&) const override
	{
		return RDGTexture;
	}

	/** Sets the viewport rect for the render target */
	void SetViewRect( const FIntRect& InViewRect ) 
	{ 
		ViewRect = InViewRect;
	}

	/** Gets the viewport rect for the render target */
	const FIntRect& GetViewRect() const 
	{
		return ViewRect;
	}

	/** Sets the clipping rect for the render target */
	void SetClippingRect( const FIntRect& InClippingRect ) 
	{ 
		ClippingRect = InClippingRect;
	}

	/** Gets the clipping rect for the render target */
	const FIntRect& GetClippingRect() const 
	{
		return ClippingRect;
	}

private:
	FRDGTexture* RDGTexture = nullptr;
	FIntRect ViewRect;
	FIntRect ClippingRect;
};

/*-----------------------------------------------------------------------------
   FPreviewViewport
-----------------------------------------------------------------------------*/

FPreviewViewport::FPreviewViewport(class UMaterialGraphNode* InNode)
	: MaterialNode(InNode)
	, PreviewElement( new FPreviewElement )
{
	if (MaterialNode)
	{
		MaterialNode->InvalidatePreviewMaterialDelegate.BindRaw(this, &FPreviewViewport::UpdatePreviewNodeRenderProxy);
	}
}

FPreviewViewport::~FPreviewViewport()
{
	if (MaterialNode)
	{
		MaterialNode->InvalidatePreviewMaterialDelegate.Unbind();
	}
	// Pass the preview element to the render thread so that it's deleted after it's shown for the last time
	ENQUEUE_RENDER_COMMAND(SafeDeletePreviewElement)(
		[PreviewElement = PreviewElement](FRHICommandListImmediate& RHICmdList) mutable
		{
			PreviewElement.Reset();
		}
	);
}

TUniquePtr<UE::Slate::FPaintScope> FPreviewViewport::Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled)
{
	FSlateRect SlateCanvasRect = AllottedGeometry.GetLayoutBoundingRect();
	FSlateRect ClippedCanvasRect = SlateCanvasRect.IntersectionWith(MyCullingRect);

	FIntRect CanvasRect(
		FMath::TruncToInt( FMath::Max(0.0f, SlateCanvasRect.Left) ),
		FMath::TruncToInt( FMath::Max(0.0f, SlateCanvasRect.Top) ),
		FMath::TruncToInt( FMath::Max(0.0f, SlateCanvasRect.Right) ), 
		FMath::TruncToInt( FMath::Max(0.0f, SlateCanvasRect.Bottom) ) );

	FIntRect ClippingRect(
		FMath::TruncToInt( FMath::Max(0.0f, ClippedCanvasRect.Left) ),
		FMath::TruncToInt( FMath::Max(0.0f, ClippedCanvasRect.Top) ),
		FMath::TruncToInt( FMath::Max(0.0f, ClippedCanvasRect.Right) ), 
		FMath::TruncToInt( FMath::Max(0.0f, ClippedCanvasRect.Bottom) ) );

	bool bIsRealtime = MaterialNode->RealtimeDelegate.IsBound() ? MaterialNode->RealtimeDelegate.Execute() : false;

	if (PreviewElement->BeginRenderingCanvas(CanvasRect, ClippingRect, MaterialNode, bIsRealtime, HoveredUV.IsSet()))
	{
		// Draw above everything else
		uint32 PreviewLayer = LayerId+1;
		FSlateDrawElement::MakeCustom( OutDrawElements, PreviewLayer, PreviewElement );
	}

	return nullptr;
}

FIntPoint FPreviewViewport::GetSize() const
{
	return FIntPoint(96,96);
}

FReply FPreviewViewport::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const FVector2D Size = MyGeometry.GetLocalSize();
	HoveredUV = FVector2D(LocalPos.X / Size.X, LocalPos.Y / Size.Y);
	return FReply::Unhandled();
}

void FPreviewViewport::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const FVector2D Size = MyGeometry.GetLocalSize();
	HoveredUV = FVector2D(LocalPos.X / Size.X, LocalPos.Y / Size.Y);
}

void FPreviewViewport::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	HoveredUV.Reset();
	if (PreviewElement)
	{
		PreviewElement->ClearReadback();
	}
}

void FPreviewViewport::UpdatePreviewNodeRenderProxy()
{
	if (PreviewElement.IsValid())
	{
		PreviewElement->UpdateExpressionPreview(MaterialNode);
	}
}

/////////////////////////////////////////////////////
// FPreviewElement

FPreviewElement::FPreviewElement()
	: RenderTarget(new FSlateMaterialPreviewRenderTarget)
	, ReadbackRenderTarget(new FSlateMaterialPreviewRenderTarget)
{
}

bool FPreviewElement::BeginRenderingCanvas(const FIntRect& InCanvasRect, const FIntRect& InClippingRect, UMaterialGraphNode* InGraphNode, bool bInIsRealtime, bool bInShouldReadback)
{
	if(InCanvasRect.Size().X > 0 && InCanvasRect.Size().Y > 0 && InClippingRect.Size().X > 0 && InClippingRect.Size().Y > 0 && InGraphNode != NULL)
	{
		/**
		 * Struct to contain all info that needs to be passed to the render thread
		 */
		struct FPreviewRenderInfo
		{
			/** Size of the Canvas tile */
			FIntRect CanvasRect;
			/** How to clip the canvas tile */
			FIntRect ClippingRect;
			/** Render proxy for the expression preview */
			FMaterialRenderProxy* RenderProxy;
			/** Whether preview is using realtime values */
			bool bIsRealtime;
			bool bShouldReadback;
		};

		FPreviewRenderInfo RenderInfo;
		RenderInfo.CanvasRect = InCanvasRect;
		RenderInfo.ClippingRect = InClippingRect;
		RenderInfo.RenderProxy = InGraphNode->GetExpressionPreview();
		RenderInfo.bIsRealtime = bInIsRealtime;
		RenderInfo.bShouldReadback = bInShouldReadback;

		FPreviewElement* PreviewElement = this;
		ENQUEUE_RENDER_COMMAND(BeginRenderingPreviewCanvas)(
			[PreviewElement, RenderInfo](FRHICommandListImmediate& RHICmdList)
			{
				PreviewElement->RenderTarget->SetViewRect(RenderInfo.CanvasRect);
				PreviewElement->RenderTarget->SetClippingRect(RenderInfo.ClippingRect);
				PreviewElement->ExpressionPreview = RenderInfo.RenderProxy;
				PreviewElement->bIsRealtime = RenderInfo.bIsRealtime;
				PreviewElement->bShouldReadback = RenderInfo.bShouldReadback;
			}
			);
		return true;
	}

	return false;
}

void FPreviewElement::UpdateExpressionPreview(UMaterialGraphNode* MaterialNode)
{
	FPreviewElement* PreviewElement = this;
	FMaterialRenderProxy* InRenderProxy = MaterialNode ? MaterialNode->GetExpressionPreview() : nullptr;
	ENQUEUE_RENDER_COMMAND(UpdatePreviewNodeRenderProxy)(
		[PreviewElement, InRenderProxy](FRHICommandListImmediate& RHICmdList)
		{
			PreviewElement->ExpressionPreview = InRenderProxy;
		}
	);
}

TOptional<FLinearColor> FPreviewElement::GetReadbackValue(const FVector2D& UV) const
{
	UE::TScopeLock ReadLock(ReadbackDataGuard);
	if (ReadbackData.IsEmpty())
	{
		return {};
	}

	const int32 X = FMath::Clamp(FMath::CeilToInt(UV.X * ReadbackDimension), 0, ReadbackDimension - 1);
	const int32 Y = FMath::Clamp(FMath::CeilToInt(UV.Y * ReadbackDimension), 0, ReadbackDimension - 1);

	FLinearColor Value = ReadbackData[(Y * ReadbackDimension) + X].GetFloats();

	// Note: Gamma is baked into the preview shader (see FMatExpressionPreview::CompilePropertyAndSetMaterialProperty) so we need to undo it here
	const float Gamma = 2.2f;
	Value.R = FMath::Pow(Value.R, Gamma);
	Value.G = FMath::Pow(Value.G, Gamma);
	Value.B = FMath::Pow(Value.B, Gamma);
	return Value;
}

void FPreviewElement::ClearReadback()
{
	ENQUEUE_RENDER_COMMAND(ClearReadbackPreviewElement)(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			GPUTextureReadback.Reset();
			bReadbackPending = false;

			UE::TScopeLock ReadLock(ReadbackDataGuard);
			ReadbackData.Empty();
		}
	);
}

void FPreviewElement::Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs)
{
	if(!ExpressionPreview)
	{
		return;
	}

	RenderTarget->SetRenderTargetTexture(Inputs.OutputTexture);
	{
		// Check realtime mode for whether to pass current time to canvas
		double CurrentTime = bIsRealtime ? (FApp::GetCurrentTime() - GStartTime) : 0.0;
		float DeltaTime = bIsRealtime ? static_cast<float>(FApp::GetDeltaTime()) : 0.0f;

		FCanvas* Canvas = GraphBuilder.AllocObject<FCanvas>(RenderTarget.Get(), nullptr, FGameTime::CreateUndilated(CurrentTime, DeltaTime), GMaxRHIFeatureLevel);
		{
			Canvas->SetAllowedModes(0);
			Canvas->SetRenderTargetRect(RenderTarget->GetViewRect());
			Canvas->SetRenderTargetScissorRect(RenderTarget->GetClippingRect());

			FCanvasTileItem TileItem(FVector2D::ZeroVector, ExpressionPreview, RenderTarget->GetSizeXY());
			Canvas->DrawItem(TileItem);
			Canvas->Flush_RenderThread(GraphBuilder, true);
		}
	}
	RenderTarget->ClearRenderTargetTexture();

	if (bShouldReadback)
	{
		Readback_RenderThread(GraphBuilder);
		return;
	}
}

void FPreviewElement::Readback_RenderThread(FRDGBuilder& GraphBuilder)
{
	// Do we have a readback pending?  If so don't issue another one until we process the pending one
	if (bReadbackPending)
	{
		if (!GPUTextureReadback->IsReady())
		{
			return;
		}

		int32 RowPitchInPixels = 0;
		const FFloat16Color* SrcData = static_cast<const FFloat16Color*>(GPUTextureReadback->Lock(RowPitchInPixels));
		{
			UE::TScopeLock WriteLock(ReadbackDataGuard);
			ReadbackData.SetNumUninitialized(ReadbackDimension * ReadbackDimension);
			for (int32 Y=0; Y < ReadbackDimension; ++Y)
			{
				FMemory::Memcpy(ReadbackData.GetData() + (Y * ReadbackDimension), SrcData + (Y * RowPitchInPixels), sizeof(FFloat16Color) * ReadbackDimension);
			}
		}
		GPUTextureReadback->Unlock();
	}

	// Issue a new render & readback request
	const FIntPoint ReadbackSize(ReadbackDimension, ReadbackDimension);

	const FRDGTextureDesc ReadbackTextureDesc = FRDGTextureDesc::Create2D(ReadbackSize, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
	FRDGTextureRef ReadbackTexture = GraphBuilder.CreateTexture(ReadbackTextureDesc, TEXT("MaterialNodeReadbackRT"));

	ReadbackRenderTarget->SetViewRect(FIntRect(FIntPoint::ZeroValue, ReadbackSize));
	ReadbackRenderTarget->SetClippingRect(FIntRect(FIntPoint::ZeroValue, ReadbackSize));
	ReadbackRenderTarget->SetRenderTargetTexture(ReadbackTexture);

	{
		const double CurrentTime = bIsRealtime ? (FApp::GetCurrentTime() - GStartTime) : 0.0;
		const float  DeltaTime = bIsRealtime ? static_cast<float>(FApp::GetDeltaTime()) : 0.0f;

		FCanvas* ReadbackCanvas = GraphBuilder.AllocObject<FCanvas>(ReadbackRenderTarget.Get(), nullptr, FGameTime::CreateUndilated(CurrentTime, DeltaTime), GMaxRHIFeatureLevel);
		ReadbackCanvas->SetAllowedModes(0);
		ReadbackCanvas->SetRenderTargetRect(ReadbackRenderTarget->GetViewRect());
		ReadbackCanvas->SetRenderTargetScissorRect(ReadbackRenderTarget->GetClippingRect());
		FCanvasTileItem TileItem(FVector2D::ZeroVector, ExpressionPreview, FVector2D(ReadbackSize));
		ReadbackCanvas->DrawItem(TileItem);
		ReadbackCanvas->Flush_RenderThread(GraphBuilder, true);
	}
	ReadbackRenderTarget->ClearRenderTargetTexture();

	if (!GPUTextureReadback)
	{
		GPUTextureReadback = MakeUnique<FRHIGPUTextureReadback>(TEXT("MaterialNodeReadback"));
	}
	AddEnqueueCopyPass(GraphBuilder, GPUTextureReadback.Get(), ReadbackTexture);
	bReadbackPending = true;
}

/////////////////////////////////////////////////////
// SGraphNodeMaterialBase

void SGraphNodeMaterialBase::Construct(const FArguments& InArgs, UMaterialGraphNode* InNode)
{
	this->TitleBorderMargin = FMargin(10.f, 5.f, 10.f, 3.f);
	this->GraphNode = InNode;
	this->MaterialNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeMaterialBase::CreatePinWidgets()
{
	// Create Pin widgets for each of the pins.
	for( int32 PinIndex=0; PinIndex < GraphNode->Pins.Num(); ++PinIndex )
	{
		UEdGraphPin* CurPin = GraphNode->Pins[PinIndex];

		bool bHideNoConnectionPins = false;

		if (OwnerGraphPanelPtr.IsValid())
		{
			bHideNoConnectionPins = OwnerGraphPanelPtr.Pin()->GetPinVisibility() == SGraphEditor::Pin_HideNoConnection;
		}

		const bool bPinHasConections = CurPin->LinkedTo.Num() > 0;

		bool bPinDesiresToBeHidden = CurPin->bHidden || (bHideNoConnectionPins && !bPinHasConections); 

		UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GraphNode->GetGraph());
		if (MaterialNode && MaterialNode->MaterialExpression && MaterialGraph->MaterialFunction == nullptr && !MaterialGraph->MaterialInputs.IsEmpty())
		{
			bool bIsAMakeAttrNode = MaterialNode->MaterialExpression->IsA(UMaterialExpressionMakeMaterialAttributes::StaticClass());
			bool bIsABreakAttrNode = MaterialNode->MaterialExpression->IsA(UMaterialExpressionBreakMaterialAttributes::StaticClass());
		
			if ((bIsABreakAttrNode && CurPin->Direction == EGPD_Output) || (bIsAMakeAttrNode && CurPin->Direction == EGPD_Input))
			{
				if (CurPin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec)
				{
					bPinDesiresToBeHidden |= !MaterialGraph->MaterialInputs[CurPin->SourceIndex].IsVisiblePin(MaterialGraph->Material, true);
				}
			}
		}

		if (!bPinDesiresToBeHidden)
		{
			TSharedPtr<SGraphPin> NewPin = CreatePinWidget(CurPin);
			check(NewPin.IsValid());

			// Assign an custom icon to not connectible pins
			if (CurPin->bNotConnectable)
			{
				if (!CacheImg_Pin_NotConnectable)
				{
					CacheImg_Pin_NotConnectable = FAppStyle::Get().GetBrush(NAME_Pin_NotConnectable);
				}
				NewPin->SetCustomPinIcon(CacheImg_Pin_NotConnectable, CacheImg_Pin_NotConnectable);
			}

			// Override pin color for Substrate node
			if (Substrate::IsSubstrateEnabled())
			{
				FSubstrateWidget::GetPinColor(NewPin, MaterialNode);
			}

			this->AddPin(NewPin.ToSharedRef());
		}
	}
}

void SGraphNodeMaterialBase::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	MaterialNode->MaterialExpression->MaterialExpressionEditorX = MaterialNode->NodePosX;
	MaterialNode->MaterialExpression->MaterialExpressionEditorY = MaterialNode->NodePosY;
	MaterialNode->MaterialExpression->MarkPackageDirty();
	MaterialNode->MaterialDirtyDelegate.ExecuteIfBound();
}

void SGraphNodeMaterialBase::AddPin( const TSharedRef<SGraphPin>& PinToAdd )
{
	PinToAdd->SetOwner( SharedThis(this) );

	// Set visibility on advanced view pins
	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = (PinObj != nullptr) && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility(TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced));
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		FMargin Padding = Settings->GetInputPinPadding();
		Padding.Left *= 0.5f;
		Padding.Right = 0.0f;

		LeftNodeBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(Padding)
		[
			PinToAdd
		];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		FMargin Padding = Settings->GetOutputPinPadding();
		Padding.Left = 0.0f;
		Padding.Right *= 0.5f;

		RightNodeBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(Padding)
		[
			PinToAdd
		];
		OutputPins.Add(PinToAdd);
	}
}

void SGraphNodeMaterialBase::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	if (GraphNode && MainBox.IsValid())
	{
		// Count the number of visible input pins on the left
		int32 LeftPinCount = 0;
		if (GraphNode->AdvancedPinDisplay == ENodeAdvancedPins::Hidden)
		{
			// Advanced view pins are hidden so exclude them from the pin count
			for (int32 i = 0; i < InputPins.Num(); ++i)
			{
				const UEdGraphPin* PinObj = InputPins[i]->GetPinObj();
				if (!PinObj->bAdvancedView)
				{
					LeftPinCount++;
				}
			}
		}
		else
		{
			LeftPinCount = InputPins.Num();
		}

		int32 RightPinCount = OutputPins.Num();

		const float NegativeHPad = FMath::Max<float>(-Settings->PaddingTowardsNodeEdge, 0.0f);
		const float ExtraPad = 0.0f;

		// Place preview widget based on where the least pins are
		if ((LeftPinCount < RightPinCount) || (RightPinCount == 0))
		{
			LeftNodeBox->AddSlot()
			.Padding(FMargin(NegativeHPad + ExtraPad, 0.0f, 0.0f, 0.0f))
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				CreatePreviewWidget()
			];
		}
		else if (LeftPinCount > RightPinCount)
		{
			RightNodeBox->AddSlot()
			.Padding(FMargin(NegativeHPad + ExtraPad, 0.0f, 0.0f, 0.0f))
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				CreatePreviewWidget()
			];
		}
		else
		{
			MainBox->AddSlot()
			.Padding(Settings->GetNonPinNodeBodyPadding())
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					CreatePreviewWidget()
				]
			];
		}
	}

	// Preview of Substrate nodes topology
	if (Substrate::IsSubstrateEnabled() && MaterialNode)
	{
		TArray<FGuid> Guids;
		if (MaterialNode->MaterialExpression->IsA(UMaterialExpressionSubstrateBSDF::StaticClass()))
		{
			const UMaterialExpression* SubstrateExpression = (const UMaterialExpression*)MaterialNode->MaterialExpression;
			Guids.Add(SubstrateExpression->MaterialExpressionGuid);
		}
		else if (MaterialNode->MaterialExpression->IsA(UMaterialExpressionMaterialFunctionCall::StaticClass()))
		{
			UMaterialExpressionMaterialFunctionCall* FunctionCall = (UMaterialExpressionMaterialFunctionCall*)MaterialNode->MaterialExpression;
			const uint32 OutputCount = FunctionCall->FunctionOutputs.Num();
			for (uint32 OutputIndex = 0; OutputIndex < OutputCount; ++OutputIndex)
			{
				if (FunctionCall->IsResultSubstrateMaterial(OutputIndex))
				{
					FSubstrateMaterialInfo SubstrateMaterialInfo(true/*bGatherGuids*/);
					FunctionCall->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, OutputIndex);
					Guids = SubstrateMaterialInfo.GetGuids();
					break;
				}
			}
		}

		if (Guids.Num() > 0)
		{
			if (const UMaterialExpression* SubstrateExpression = (const UMaterialExpression*)MaterialNode->MaterialExpression)
			{		
				if (UMaterial* MaterialForStats = SubstrateExpression->Material)
				{
					if (const FMaterialResource* MaterialResource = MaterialForStats->GetMaterialResource(GMaxRHIShaderPlatform))
					{
						if (FMaterialShaderMap* ShaderMap = MaterialResource->GetGameThreadShaderMap())
						{
							const FSubstrateMaterialCompilationOutput& CompilationOutput = ShaderMap->GetSubstrateMaterialCompilationOutput();
							MainBox->AddSlot()
							.Padding(Settings->GetNonPinNodeBodyPadding())
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								[							
									FSubstrateWidget::ProcessOperator(CompilationOutput, Guids)
								]
							];
						}
					}
	
				}
			}
		}
	}
}

TSharedRef<SWidget> SGraphNodeMaterialBase::CreateTitleRightWidget()
{
	if (MaterialNode->MaterialExpression->bHidePreviewWindow)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.OnClicked(this, &SGraphNodeMaterialBase::OnTogglePreviewClicked)
		.Cursor(EMouseCursor::Default)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush(TEXT("Icons.Preview")))
			]
		]
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SCheckBox)
		.OnCheckStateChanged( this, &SGraphNodeMaterialBase::OnExpressionPreviewChanged )
		.IsChecked( IsExpressionPreviewChecked() )
		.Cursor(EMouseCursor::Default)
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(GetExpressionPreviewArrow())
			]
		]
	];
}

TSharedRef<SWidget> SGraphNodeMaterialBase::CreateNodeContentArea()
{
	// NODE CONTENT AREA
	return SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(1.0f)
			[
				// LEFT
				SAssignNew(LeftNodeBox, SVerticalBox)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				// RIGHT
				SAssignNew(RightNodeBox, SVerticalBox)
			]
		];
}

void SGraphNodeMaterialBase::OnAdvancedViewChanged(const ECheckBoxState NewCheckedState)
{
	SGraphNode::OnAdvancedViewChanged(NewCheckedState);

	// Update the graph node so that the preview is recreated to update its position
	UpdateGraphNode();
}

TSharedRef<SWidget> SGraphNodeMaterialBase::CreatePreviewWidget()
{
	PreviewViewport.Reset();

	if (MaterialNode->MaterialExpression->ShouldShowPreview())
	{
		const float ExpressionPreviewSize = 106.0f;
		const float CentralPadding = 5.0f;

		TSharedPtr<SViewport> ViewportWidget = 
			SNew( SViewport )
			.RenderDirectlyToWindow(true)
			.EnableGammaCorrection(false);

		PreviewViewport = MakeShareable(new FPreviewViewport(MaterialNode));

		// The viewport widget needs an interface so it knows what should render
		ViewportWidget->SetViewportInterface( PreviewViewport.ToSharedRef() );

		return SNew(SBox)
			.WidthOverride(ExpressionPreviewSize)
			.HeightOverride(ExpressionPreviewSize)
			.MaxAspectRatio(1.0f)
			.MaxDesiredHeight(ExpressionPreviewSize)
			.Visibility(ExpressionPreviewVisibility())
			.ToolTip(
				SNew(SToolTip)
				[
					SNew(STextBlock)
					.Text(this, &SGraphNodeMaterialBase::ExpressionPreviewValueText)
				]
			)
			.Cursor(EMouseCursor::Crosshairs)
			[
				SNew(SBorder)
				.Padding(CentralPadding)
				.BorderImage( FAppStyle::GetBrush("NoBorder") )
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						ViewportWidget.ToSharedRef()
					]
					+ SOverlay::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(0.0f,0.0f,8.0f,0.0f)
					[
						SNew(SBox)
						.Cursor(EMouseCursor::Crosshairs)
						.Visibility(EVisibility::HitTestInvisible)
						[
							SNew(STextBlock)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D(1.0f, 1.0f))
							.Text(this, &SGraphNodeMaterialBase::ExpressionPreviewOverlayText)
						]
					]
				]
			];
	}

	return SNullWidget::NullWidget;
}

FText SGraphNodeMaterialBase::ExpressionPreviewOverlayText() const
{
	UMaterialExpression* MaterialExpression = MaterialNode->MaterialExpression;
	return MaterialExpression->GetPreviewOverlayText();
}

FText SGraphNodeMaterialBase::ExpressionPreviewValueText() const
{
	FText ValueText;
	if (PreviewViewport->HoveredUV.IsSet())
	{
		TOptional<FLinearColor> Value = PreviewViewport->PreviewElement->GetReadbackValue(PreviewViewport->HoveredUV.GetValue());
		if (Value.IsSet())
		{
			int32 NumChannels = 0;
			if (FMaterialRenderProxy* Proxy = MaterialNode->GetExpressionPreview())
			{
				// FMatExpressionPreview is the only concrete type returned here
				if (const FMatExpressionPreview* ExprPreview = static_cast<const FMatExpressionPreview*>(Proxy))
				{
					NumChannels = ExprPreview->GetCompiledPreviewNumChannels();
				}
			}

			switch (NumChannels)
			{
			case 1:
				ValueText = FText::FromString(FString::Printf(TEXT("%.2g"), Value->R));
				break;
			case 2:
				ValueText = FText::FromString(FString::Printf(TEXT("%.2g, %.2g"), Value->R, Value->G));
				break;
			case 3:
				ValueText = FText::FromString(FString::Printf(TEXT("%.2g, %.2g, %.2g"), Value->R, Value->G, Value->B));
				break;
			default:
				ValueText = FText::FromString(FString::Printf(TEXT("%.2g, %.2g, %.2g, %.2g"), Value->R, Value->G, Value->B, Value->A));
				break;
			}
		}
	}

	return ValueText;
}

EVisibility SGraphNodeMaterialBase::ExpressionPreviewVisibility() const
{
	return MaterialNode->MaterialExpression->ShouldShowPreview() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SGraphNodeMaterialBase::OnTogglePreviewClicked()
{
	MaterialNode->ToggleNodePreview();
	return FReply::Handled();
}

void SGraphNodeMaterialBase::OnExpressionPreviewChanged( const ECheckBoxState NewCheckedState )
{
	UMaterialExpression* MaterialExpression = MaterialNode->MaterialExpression;
	const bool bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	if (MaterialExpression->bCollapsed != bCollapsed)
	{
		UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(MaterialNode->GetGraph());
		MaterialGraph->ToggleCollapsedDelegate.ExecuteIfBound(MaterialExpression);

		// Update the graph node so that preview viewport is created
		UpdateGraphNode();
	}
}

ECheckBoxState SGraphNodeMaterialBase::IsExpressionPreviewChecked() const
{
	return MaterialNode->MaterialExpression->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

const FSlateBrush* SGraphNodeMaterialBase::GetExpressionPreviewArrow() const
{
	return FAppStyle::GetBrush(MaterialNode->MaterialExpression->bCollapsed ? TEXT("Icons.ChevronDown") : TEXT("Icons.ChevronUp"));
}

void SGraphNodeMaterialBase::PopulateMetaTag(FGraphNodeMetaData* TagMeta) const
{
	if (GraphNode != nullptr)
	{
		UMaterialGraph* OuterGraph = MaterialNode->GetTypedOuter<UMaterialGraph>();
		if ((OuterGraph != nullptr) && (MaterialNode->MaterialExpression != nullptr) )
		{
			TagMeta->OuterName = OuterGraph->OriginalMaterialFullName;
			TagMeta->GUID = MaterialNode->MaterialExpression->MaterialExpressionGuid;
			TagMeta->Tag = FName(*FString::Printf(TEXT("MaterialExprNode_%s_%s"), *TagMeta->OuterName, *TagMeta->GUID.ToString()));
		}
		TagMeta->FriendlyName = FString::Printf(TEXT("%s expression node in %s"), *GraphNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString(), *TagMeta->OuterName);		
	}
}
