// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialNodes/SGraphNodeMaterialComment.h"

#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "Materials/MaterialExpressionComment.h"
#include "UObject/ObjectPtr.h"

void SGraphNodeMaterialComment::Construct(const FArguments& InArgs, class UMaterialGraphNode_Comment* InNode)
{
	SGraphNodeComment::Construct(SGraphNodeComment::FArguments(), InNode);

	this->CommentNode = InNode;
}

void SGraphNodeMaterialComment::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if( !NodeFilter.Find( SharedThis( this ) ))
	{
		SGraphNodeComment::MoveTo(NewPosition, NodeFilter, bMarkDirty);
		if (CommentNode && CommentNode->MaterialExpressionComment)
		{
			CommentNode->MaterialExpressionComment->MaterialExpressionEditorX = CommentNode->NodePosX;
			CommentNode->MaterialExpressionComment->MaterialExpressionEditorY = CommentNode->NodePosY;
			CommentNode->MaterialExpressionComment->MarkPackageDirty();
			CommentNode->MaterialDirtyDelegate.ExecuteIfBound();
		}
	}
}
