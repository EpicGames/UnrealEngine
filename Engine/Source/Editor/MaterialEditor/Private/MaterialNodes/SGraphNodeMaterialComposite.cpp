// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialComposite.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "Materials/MaterialExpression.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SCommentBubble.h"
#include "SGraphNode.h"
#include "SGraphPreviewer.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

void SGraphNodeMaterialComposite::Construct(const FArguments& InArgs, class UMaterialGraphNode_Composite* InNode)
{
	this->GraphNode = InNode;
	this->CompositeNode = InNode;
	
	//@TODO: Add previews to composites, figure out multiple output UX (scroll?).
	//this->MaterialNode = CompositeNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeMaterialComposite::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	CompositeNode->MaterialExpression->MaterialExpressionEditorX = CompositeNode->NodePosX;
	CompositeNode->MaterialExpression->MaterialExpressionEditorY = CompositeNode->NodePosY;
	CompositeNode->MaterialExpression->MarkPackageDirty();
	CompositeNode->MaterialDirtyDelegate.ExecuteIfBound();
}

void SGraphNodeMaterialComposite::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SetupErrorReporting();
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "Graph.CollapsedNode.Body" ) )
			.Padding(0)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("Graph.CollapsedNode.BodyColorSpill") )
					.ColorAndOpacity( this, &SGraphNode::GetNodeTitleColor )
				]
				+SOverlay::Slot()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder)
							.BorderImage( FAppStyle::GetBrush("NoBorder") )
							.Padding( FMargin(10,5,30,3) )
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Top)
								[
									SNew(SVerticalBox)
									+SVerticalBox::Slot()
										.AutoHeight()
									[
										SAssignNew(InlineEditableText, SInlineEditableTextBlock)
										.Style( FAppStyle::Get(), "Graph.Node.NodeTitleInlineEditableText" )
										.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
										.OnVerifyTextChanged(this, &SGraphNodeMaterialComposite::OnVerifyNameTextChanged)
										.OnTextCommitted(this, &SGraphNodeMaterialComposite::OnNameTextCommited)
										.IsReadOnly( this, &SGraphNodeMaterialComposite::IsNameReadOnly )
										.IsSelected(this, &SGraphNodeMaterialComposite::IsSelectedExclusively)
									]
									+SVerticalBox::Slot()
										.AutoHeight()
									[
										NodeTitle.ToSharedRef()
									]
								]
								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(1.0f)
								[
									ErrorReporting->AsWidget()
								]
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						CreateNodeBody()
					]
				]
			]
		];

	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered( this, &SGraphNode::IsHovered );

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	CreatePinWidgets();
}

UEdGraph* SGraphNodeMaterialComposite::GetInnerGraph() const
{
	return CompositeNode->BoundGraph;
}

TSharedPtr<SToolTip> SGraphNodeMaterialComposite::GetComplexTooltip()
{
	if (UEdGraph* BoundGraph = GetInnerGraph())
	{
		struct LocalUtils
		{
			static bool IsInteractive()
			{
				const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
				return ( ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown() );
			}
		};

		TSharedPtr<SToolTip> FinalToolTip = nullptr;
		TSharedPtr<SVerticalBox> Container = nullptr;
		SAssignNew(FinalToolTip, SToolTip)
		.IsInteractive_Static(&LocalUtils::IsInteractive)
		[
			SAssignNew(Container, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( STextBlock )
				.Text(this, &SGraphNodeMaterialComposite::GetTooltipTextForNode)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				.WrapTextAt(160.0f)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				// Create preview for the tooltip, make sure to disable state overlays to prevent
				// PIE and read-only borders obscuring the graph
				SNew(SGraphPreviewer, BoundGraph)
				.CornerOverlayText(this, &SGraphNodeMaterialComposite::GetPreviewCornerText)
				.ShowGraphStateOverlay(false)
			]
		];

		// Check to see whether this node has a documentation excerpt. If it does, create a doc box for the tooltip
		TSharedRef<IDocumentationPage> DocPage = IDocumentation::Get()->GetPage(GraphNode->GetDocumentationLink(), nullptr);
		if(DocPage->HasExcerpt(GraphNode->GetDocumentationExcerptName()))
		{
			Container->AddSlot()
			.AutoHeight()
			.Padding(FMargin( 0.0f, 5.0f ))
			[
				IDocumentation::Get()->CreateToolTip(FText::FromString("Documentation"), nullptr, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName())
			];
		}

		return FinalToolTip;
	}
	else
	{
		return SNew(SToolTip)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( STextBlock )
					.Text(NSLOCTEXT("CompositeNode", "CompositeNodeInvalidGraphMessage", "ERROR: Invalid Graph"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					.WrapTextAt(160.0f)
				]
			];
	}

}

FText SGraphNodeMaterialComposite::GetPreviewCornerText() const
{
	UEdGraph* BoundGraph = GetInnerGraph();
	return FText::FromString(BoundGraph->GetName());
}

FText SGraphNodeMaterialComposite::GetTooltipTextForNode() const
{
	return GraphNode->GetTooltipText();
}

TSharedRef<SWidget> SGraphNodeMaterialComposite::CreateNodeBody()
{
	if( GraphNode && GraphNode->Pins.Num() > 0 )
	{
		// Create the input and output pin areas if there are pins
		return SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("NoBorder") )
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding( FMargin(0,3) )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				[
					// LEFT
					SAssignNew(LeftNodeBox, SVerticalBox)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			];
	}
	else
	{
		// Create a spacer so the node has some body to it
		return SNew(SSpacer)
			.Size(FVector2D(100.f, 50.f));
	}
}

