// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialConvert.h"

#include "ConnectionDrawingPolicy.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "Materials/MaterialExpressionConvert.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "GraphNodeMaterialConvert"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FConvertDragDropOp: Drag and Drop Operation used to form connections within the convert node
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FConvertDragDropOp::FConvertDragDropOp(TSharedPtr<SConvertInnerPin> InSourcePin)
: SourcePin(InSourcePin)
{
	Construct();
}

void FConvertDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);

	if (SourcePin)
	{
		SourcePin->CancelDragDrop();
	}
}

void FConvertDragDropOp::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	FDragDropOperation::OnDragged(DragDropEvent);

	ScreenPosition = DragDropEvent.GetScreenSpacePosition();
}

TSharedPtr<SConvertInnerPin> FConvertDragDropOp::GetSourcePin() const
{
	return SourcePin;
}

UE::Slate::FDeprecateVector2DResult FConvertDragDropOp::GetScreenPosition() const
{ 
	return ScreenPosition;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// SMaterialExpressionConvertGraphPin: The Outer pins that form connections to other material graph nodes
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SMaterialExpressionConvertGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InPin);

	// Setup custom hover logic to only treat this pin as hovered when you hover over the main pin image
	this->SetHover(TAttribute<bool>::CreateSP(this, &SMaterialExpressionConvertGraphPin::IsHoveredOverPrimaryPin));
}

void SMaterialExpressionConvertGraphPin::CreateInnerPins(TSharedRef<SGraphNodeMaterialConvert> InOwningGraphNodeWidget)
{
	UMaterialGraphNode* MaterialNode = InOwningGraphNodeWidget->GetMaterialGraphNode();
	if (!MaterialNode)
	{
		return;
	}

	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return;
	}

	UEdGraphPin* EdGraphPin = GetPinObj();
	if (!EdGraphPin)
	{
		return;
	}
		
	if (TSharedPtr<SHorizontalBox> PinnedRowWidget = GetFullPinHorizontalRowWidget().Pin())
	{
		const bool bIsInputPin = EdGraphPin->Direction == EGPD_Input;
		const int32 PinIndex = EdGraphPin->SourceIndex;
				
		const EMaterialExpressionConvertType ConvertType 	= bIsInputPin
			? ConvertExpression->ConvertInputs[PinIndex].Type
			: ConvertExpression->ConvertOutputs[PinIndex].Type;
		const int32 PinComponentCount = MaterialExpressionConvertType::GetComponentCount(ConvertType);
				
		InnerPins.Reset(PinComponentCount);
		TSharedPtr<SVerticalBox> InnerPinsVerticalBox = SNew(SVerticalBox);
		for (int32 ComponentIndex = 0; ComponentIndex < PinComponentCount; ++ComponentIndex)
		{
			TSharedRef<SConvertInnerPin> NewInnerPin = SNew(SConvertInnerPin, 
				InOwningGraphNodeWidget,
				SharedThis(this),
				bIsInputPin, 
				PinIndex, 
				ComponentIndex
			);
			InnerPinsVerticalBox->AddSlot()
			.HAlign(bIsInputPin ? HAlign_Right : HAlign_Left)
			[
				NewInnerPin
			];
			InnerPins.Add(NewInnerPin);
		}

		// Insert as last element (INDEX_NONE) for input pins, first element for output pins
		PinnedRowWidget->InsertSlot(bIsInputPin ? INDEX_NONE : 0)
		.HAlign(bIsInputPin ? HAlign_Right : HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			InnerPinsVerticalBox.ToSharedRef()
		];
	}
}

bool SMaterialExpressionConvertGraphPin::IsHoveredOverPrimaryPin() const
{
	TSharedPtr<SWidget> PinImageWidget = this->GetPinImageWidget();
	return PinImageWidget ? PinImageWidget->IsHovered() : false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// SConvertInnerPin: The inner pins used to route values with the convert node
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SConvertInnerPin::Construct(
	const FArguments& InArgs,
	TSharedPtr<SGraphNodeMaterialConvert> InOwningNode,
	TSharedPtr<SMaterialExpressionConvertGraphPin> InOwningPin,
	bool bInIsInputPin, 
	int32 InPinIndex, 
	int32 InComponentIndex
)
{
	WeakOwningNode = InOwningNode;
	WeakOwningPin = InOwningPin;
	bIsInputPin = bInIsInputPin;
	PinIndex = InPinIndex;
	ComponentIndex = InComponentIndex;

	// Construct Horizontal Box and add the PinImage to it
	TSharedPtr<SHorizontalBox> HorizontalBox = 
	SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(2.5f, 0.0f)
	[
		SAssignNew(PinImage, SImage)
		.Image(this, &SConvertInnerPin::GetPinBrush)
	];

	// Add default value entry to either the front or back of the hbox depending on if we're an input pin
	HorizontalBox->InsertSlot(bIsInputPin ? 0 : INDEX_NONE)
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2.5f, 0.0f)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "MonospacedText")
			.Visibility(this, &SConvertInnerPin::GetPinNameVisibility)
			.Text(this, &SConvertInnerPin::GetPinName)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2.5f, 0.0f)
		[
			SNew(SNumericEntryBox<float>)
			.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
			.BorderForegroundColor(FSlateColor::UseForeground())
			.Visibility(this, &SConvertInnerPin::GetDefaultValueVisibility)
			.Value(this, &SConvertInnerPin::GetDefaultValue)
			.OnValueCommitted(this, &SConvertInnerPin::SetDefaultValue)
		]
	];

	// Add inner node connection space to either the front or back of the hbox depending on if we're an input pin
	HorizontalBox->InsertSlot(bIsInputPin ? INDEX_NONE : 0)
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SSpacer)
		.Size(FVector2D(30, 0))
	];

	ChildSlot
	[
		HorizontalBox.ToSharedRef()
	];
}

int32 SConvertInnerPin::OnPaint(
	const FPaintArgs& Args,
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled
) const
{
	LayerId = SCompoundWidget::OnPaint(
		Args, 
		AllottedGeometry, 
		MyCullingRect, 
		OutDrawElements, 
		LayerId, 
		InWidgetStyle, 
		bParentEnabled
	);

	// Update CenterAbsolute after we've painted our PinImage
	if (PinImage.IsValid())
	{
		CenterAbsolute = FGeometryHelper::CenterOf(PinImage->GetPaintSpaceGeometry());
	}

	return LayerId;
}

FReply SConvertInnerPin::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (MouseEvent.IsAltDown())
		{
			BreakConnections();
			return FReply::Handled();
		}

		return FReply::Handled()
		.DetectDrag(SharedThis(this), MouseEvent.GetEffectingButton())
		.CaptureMouse(SharedThis(this));
	}
	
	// We still want to detect right-clicks on mouse button up
	return FReply::Handled();
}

FReply SConvertInnerPin::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// Only show to context menu if we right click the actual InnerPin
		if (PinImage && PinImage->GetCachedGeometry().IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
		{
			FMenuBuilder ContextMenuBuilder(true, NULL);

			const FText BreakConnectionText =
				WeakConnectedPins.Num() == 1
				? LOCTEXT("ContextMenu.BreakInnerConnection", "Break Connection")
				: LOCTEXT("ContextMenu.BreakInnerConnections", "Break Connections");

			ContextMenuBuilder.AddMenuEntry(
				BreakConnectionText,
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SConvertInnerPin::BreakConnections))
			);

			FSlateApplication::Get().PushMenu(
				AsShared(),
				FWidgetPath(),
				ContextMenuBuilder.MakeWidget(),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

			return FReply::Handled();
		}
	}
		
	return FReply::Unhandled();
}

FReply SConvertInnerPin::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<SGraphNodeMaterialConvert> OwningNode = WeakOwningNode.Pin())
	{
		TSharedRef<FConvertDragDropOp> DragDropOp = MakeShareable(new FConvertDragDropOp(SharedThis(this)));
		OwningNode->SetCurrentDragDropOp(DragDropOp);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

FReply SConvertInnerPin::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FConvertDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FConvertDragDropOp>())
	{
		if (TSharedPtr<SConvertInnerPin> SourcePin = DragDropOp->GetSourcePin())
		{
			// Is this a valid connection
			if (SourcePin.Get() != this && SourcePin->WeakOwningNode == this->WeakOwningNode && SourcePin->bIsInputPin != this->bIsInputPin)
			{
				if (TSharedPtr<SGraphNodeMaterialConvert> OwningNode = WeakOwningNode.Pin())
				{
					// Form connection between two pins
					OwningNode->FormConnection(SourcePin, SharedThis(this));
				
					// Clear The Current Drag Drop Op
					OwningNode->SetCurrentDragDropOp(nullptr);
					return FReply::Handled();
				}
			}
		}
	}

	return FReply::Unhandled();
}

const FSlateBrush* SConvertInnerPin::GetPinBrush() const
{
	const bool bHasConnections = WeakConnectedPins.Num() > 0;
	const FSlateBrush* ConnectedBrush = FAppStyle::GetBrush("Graph.Pin.Connected");
	const FSlateBrush* DisconnectedBrush = FAppStyle::GetBrush("Graph.Pin.Disconnected");
	return bHasConnections ? ConnectedBrush : DisconnectedBrush;
}

void SConvertInnerPin::CancelDragDrop()
{
	if (TSharedPtr<SGraphNodeMaterialConvert> OwningNode = WeakOwningNode.Pin())
	{
		OwningNode->SetCurrentDragDropOp(nullptr);
	}
}

UE::Slate::FDeprecateVector2DResult SConvertInnerPin::GetPinCenterAbsolute() const
{ 
	return CenterAbsolute; 
}

void SConvertInnerPin::AddConnection(TSharedPtr<SConvertInnerPin> InOtherPin)
{ 
	WeakConnectedPins.Add(InOtherPin); 
}

void SConvertInnerPin::RemoveConnection(TSharedPtr<SConvertInnerPin> InOtherPin)
{ 
	WeakConnectedPins.Remove(InOtherPin); 
}

void SConvertInnerPin::RemoveAllConnections() 
{
	for (TWeakPtr<SConvertInnerPin> WeakConnectedPin : WeakConnectedPins)
	{
		if (TSharedPtr<SConvertInnerPin> ConnectedPin = WeakConnectedPin.Pin())
		{
			ConnectedPin->RemoveConnection(SharedThis(this));
		}
	}

	WeakConnectedPins.Reset();
}

void SConvertInnerPin::BreakConnections()
{
	if (TSharedPtr<SGraphNodeMaterialConvert> OwningNode = WeakOwningNode.Pin())
	{
		OwningNode->BreakConnections(SharedThis(this));
	}
}

EVisibility SConvertInnerPin::GetDefaultValueVisibility() const
{ 
	bool bIsVisible = false;

	if (bIsInputPin)
	{
		if (TSharedPtr<SMaterialExpressionConvertGraphPin> OwningPin = WeakOwningPin.Pin())
		{
			// Only show Input Default values if the owning SMaterialExpressionConvertGraphPin isn't connected 
			bIsVisible = !OwningPin->IsConnected();
		}
	}
	else
	{
		// Show Output pin default values if they don't yet have a connection to an input inner pin
		bIsVisible = WeakConnectedPins.IsEmpty();
	}

	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

TOptional<float> SConvertInnerPin::GetDefaultValue() const
{
	TSharedPtr<SGraphNodeMaterialConvert> OwningNode = WeakOwningNode.Pin();
	if (!OwningNode)
	{
		return TOptional<float>();
	}
	return OwningNode->GetDefaultValue(SharedThis(this));
}

void SConvertInnerPin::SetDefaultValue(float InDefaultValue, ETextCommit::Type CommitType)
{
	if (TSharedPtr<SGraphNodeMaterialConvert> OwningNode = WeakOwningNode.Pin())
	{
		OwningNode->SetDefaultValue(
			SharedThis(this),
			InDefaultValue
		);
	}
}

EVisibility SConvertInnerPin::GetPinNameVisibility() const
{
	TSharedPtr<SMaterialExpressionConvertGraphPin> OwningPin = WeakOwningPin.Pin();
	if (OwningPin && OwningPin->GetInnerPins().Num() == 1)
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::SelfHitTestInvisible;
}

FText SConvertInnerPin::GetPinName() const
{
	TSharedPtr<SMaterialExpressionConvertGraphPin> OwningPin = WeakOwningPin.Pin();
	if (OwningPin && OwningPin->GetInnerPins().Num() == 1)
	{
		return INVTEXT("");
	}

	switch (ComponentIndex)
	{
		case 0: 	return INVTEXT("R");
		case 1: 	return INVTEXT("G");
		case 2: 	return INVTEXT("B");
		case 3: 	return INVTEXT("A");
		default: 	return INVTEXT("_");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// SGraphNodeMaterialConvert: Custom Slate widget for UMaterialExpressionConvert
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SGraphNodeMaterialConvert::Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode)
{
	this->GraphNode = InNode;
	this->MaterialNode = InNode;
	this->UpdateGraphNode();
}

void SGraphNodeMaterialConvert::CreatePinWidgets()
{
	SGraphNodeMaterialBase::CreatePinWidgets();

	if (!MaterialNode)
	{
		return;
	}

	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return;
	}

	// After we've set up all of our pins, set up initial connections based on our ConvertMappings
	for (const FMaterialExpressionConvertMapping& ConvertMapping : ConvertExpression->ConvertMappings)
	{
		TSharedPtr<SConvertInnerPin> InputInnerPin = GetInnerPin(true, ConvertMapping.InputIndex, ConvertMapping.InputComponentIndex);
		TSharedPtr<SConvertInnerPin> OutputInnerPin = GetInnerPin(false, ConvertMapping.OutputIndex, ConvertMapping.OutputComponentIndex);
		if (InputInnerPin && OutputInnerPin)
		{
			InputInnerPin->AddConnection(OutputInnerPin);
			OutputInnerPin->AddConnection(InputInnerPin);
		}
	}
}

TSharedPtr<SGraphPin> SGraphNodeMaterialConvert::CreatePinWidget(UEdGraphPin* Pin) const
{
	return SNew(SMaterialExpressionConvertGraphPin, Pin);
}

void SGraphNodeMaterialConvert::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	SGraphNodeMaterialBase::AddPin(PinToAdd);

	// Create our Inner Pins, which are used to route input values to output values
	TSharedRef<SMaterialExpressionConvertGraphPin> ConvertOuterPin = StaticCastSharedRef<SMaterialExpressionConvertGraphPin>(PinToAdd);
	ConvertOuterPin->CreateInnerPins(SharedThis(this));
}

void SGraphNodeMaterialConvert::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
{
	TSharedRef<SButton> AddPinButton = StaticCastSharedRef<SButton>(AddPinButtonContent(
		LOCTEXT("GraphNodeMaterialConvert_AddInputPin", "Add Input"),
		LOCTEXT("GraphNodeMaterialConvert_AddInputPin_Tooltip", "Add an input to this convert node."),
		false /* bRightSide */
	));

	AddPinButton->SetOnClicked(FOnClicked::CreateSP(this, &SGraphNodeMaterialConvert::OnAddInputPinClicked));

	InputBox->AddSlot()
	.FillHeight(1.f)
	.Padding(5.f)
	.VAlign(VAlign_Bottom)
	.HAlign(HAlign_Left)
	[
		AddPinButton
	];
}

void SGraphNodeMaterialConvert::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedRef<SButton> AddPinButton = StaticCastSharedRef<SButton>(AddPinButtonContent(
		LOCTEXT("GraphNodeMaterialConvert_AddOutputPin", "Add Output"),
		LOCTEXT("GraphNodeMaterialConvert_AddOutputPin_Tooltip", "Add an output to this convert node."),
		true /* bRightSide */
	));

	AddPinButton->SetOnClicked(FOnClicked::CreateSP(this, &SGraphNodeMaterialConvert::OnAddOutputPinClicked));

	OutputBox->AddSlot()
	.FillHeight(1.f)
	.Padding(5.f)
	.VAlign(VAlign_Bottom)
	.HAlign(HAlign_Right)
	[
		AddPinButton
	];
}

EVisibility SGraphNodeMaterialConvert::IsAddPinButtonVisible() const
{
	return EVisibility::Visible;
}

FReply SGraphNodeMaterialConvert::OnAddInputPinClicked()
{
	FSlateApplication::Get().PushMenu(
		AsShared(), 
		FWidgetPath(), 
		CreateAddPinContextMenu(/* bInputPin = */ true), 
		FSlateApplication::Get().GetCursorPos(), 
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);

	return FReply::Handled();
}

FReply SGraphNodeMaterialConvert::OnAddOutputPinClicked()
{
	FSlateApplication::Get().PushMenu(
		AsShared(), 
		FWidgetPath(), 
		CreateAddPinContextMenu(/* bInputPin = */ false),
		FSlateApplication::Get().GetCursorPos(), 
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);

	return FReply::Handled();
}

TSharedRef<SWidget> SGraphNodeMaterialConvert::CreateAddPinContextMenu(bool bInputPin)
{
	FMenuBuilder ContextMenuBuilder(true, NULL);

	static const EMaterialExpressionConvertType ConvertTypes[] =
	{ 
		EMaterialExpressionConvertType::Scalar, 
		EMaterialExpressionConvertType::Vector2, 
		EMaterialExpressionConvertType::Vector3, 
		EMaterialExpressionConvertType::Vector4 
	};

	for (const EMaterialExpressionConvertType ConvertType : ConvertTypes)
	{
		ContextMenuBuilder.AddMenuEntry(
			MaterialExpressionConvertType::ToText(ConvertType),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(
				this, 
				&SGraphNodeMaterialConvert::AddNewPin, bInputPin, ConvertType
			))
		);
	}
	
	return ContextMenuBuilder.MakeWidget();
}

void SGraphNodeMaterialConvert::AddNewPin(bool bInputPin, EMaterialExpressionConvertType ConvertType)
{
	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddPin", "Add Pin"));
	ConvertExpression->Modify();

	if (bInputPin)
	{
		FMaterialExpressionConvertInput NewInput;
		NewInput.Type = ConvertType;
		ConvertExpression->ConvertInputs.Add(NewInput);
	}
	else
	{
		FMaterialExpressionConvertOutput NewOutput;
		NewOutput.Type = ConvertType;
		ConvertExpression->ConvertOutputs.Add(NewOutput);
	}

	MaterialNode->ReconstructNode();
}

int32 SGraphNodeMaterialConvert::OnPaint(
	const FPaintArgs& Args,
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled
) const
{
	LayerId = SGraphNodeMaterialBase::OnPaint(
		Args, 
		AllottedGeometry, 
		MyCullingRect, 
		OutDrawElements, 
		LayerId, 
		InWidgetStyle, 
		bParentEnabled
	);

	// Only Draw Connections when we're at a high-enough detail. Early-out otherwise
	if (GetCurrentLOD() <= EGraphRenderingLOD::LowDetail)
	{
		return LayerId;
	}

	if (!MaterialNode)
	{
		return LayerId;
	}

	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return LayerId;
	}

	for (const FMaterialExpressionConvertMapping& ConvertMapping : ConvertExpression->ConvertMappings)
	{
		TSharedPtr<SConvertInnerPin> InputInnerPin = GetInnerPin(true, ConvertMapping.InputIndex, ConvertMapping.InputComponentIndex);
		TSharedPtr<SConvertInnerPin> OutputInnerPin = GetInnerPin(false, ConvertMapping.OutputIndex, ConvertMapping.OutputComponentIndex);
		if (InputInnerPin && OutputInnerPin)
		{
			const FVector2f CurveStart = InputInnerPin->GetPinCenterAbsolute();
			const FVector2f CurveEnd = OutputInnerPin->GetPinCenterAbsolute();
			MakeConnectionCurve(AllottedGeometry, OutDrawElements, LayerId, CurveStart, CurveEnd);
		}
	}

	if (CurrentDragDropOp.IsValid())
	{
		if (TSharedPtr<SConvertInnerPin> SourcePin = CurrentDragDropOp->GetSourcePin())
		{
			const FVector2f CurveStart = SourcePin->GetPinCenterAbsolute();
			const FVector2f CurveEnd = CurrentDragDropOp->GetScreenPosition() + Inverse(Args.GetWindowToDesktopTransform());
			MakeConnectionCurve(AllottedGeometry, OutDrawElements, LayerId, CurveStart, CurveEnd);
		}
	}


	return LayerId;
}

void SGraphNodeMaterialConvert::FormConnection(TSharedPtr<SConvertInnerPin> InnerPinA, TSharedPtr<SConvertInnerPin> InnerPinB)
{
	check(InnerPinA->IsInputPin() != InnerPinB->IsInputPin());

	if (!MaterialNode)
	{
		return;
	}

	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddConnection", "Convert Node: Add Connection"));
	ConvertExpression->Modify();

	// Determine Which Pin is the input and which is the output
	TSharedPtr<SConvertInnerPin> InputPin 	= InnerPinA->IsInputPin() ? InnerPinA : InnerPinB;
	TSharedPtr<SConvertInnerPin> OutputPin 	= InnerPinA->IsInputPin() ? InnerPinB : InnerPinA;

	FMaterialExpressionConvertMapping NewConvertMapping;
	NewConvertMapping.InputIndex = InputPin->GetPinIndex();
	NewConvertMapping.InputComponentIndex = InputPin->GetComponentIndex();
	NewConvertMapping.OutputIndex = OutputPin->GetPinIndex();
	NewConvertMapping.OutputComponentIndex = OutputPin->GetComponentIndex();

	// Remove any mappings with the same OutputIndex and OutputComponentIndex
	ConvertExpression->ConvertMappings.RemoveAll(
		[&NewConvertMapping](const FMaterialExpressionConvertMapping& ExistingConvertMapping)
		{
			return ExistingConvertMapping.OutputIndex == NewConvertMapping.OutputIndex
				&& ExistingConvertMapping.OutputComponentIndex == NewConvertMapping.OutputComponentIndex;
		}
	);

	// Add new convert mapping and refresh
	ConvertExpression->ConvertMappings.Add(NewConvertMapping);
	ConvertExpression->RefreshNode();

	// Input Pins can have multiple connections
	InputPin->AddConnection(OutputPin);

	// Output Pins can only have one connection, so clean up any existing connections
	OutputPin->RemoveAllConnections();
	OutputPin->AddConnection(InputPin);
}

void SGraphNodeMaterialConvert::BreakConnections(TSharedPtr<SConvertInnerPin> InnerPin)
{
	check(InnerPin);

	if (!MaterialNode)
	{
		return;
	}

	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("BreakConnections", "Convert Node: Break Connections"));
	ConvertExpression->Modify();

	const bool bIsInputPin = InnerPin->IsInputPin();
	const int32 PinIndex = InnerPin->GetPinIndex();
	const int32 ComponentIndex = InnerPin->GetComponentIndex();

	ConvertExpression->ConvertMappings.RemoveAll(
		[bIsInputPin, PinIndex, ComponentIndex](const FMaterialExpressionConvertMapping& ExistingConvertMapping)
		{
			if (bIsInputPin)
			{
				return ExistingConvertMapping.InputIndex == PinIndex && ExistingConvertMapping.InputComponentIndex == ComponentIndex;
			}
			else
			{
				return ExistingConvertMapping.OutputIndex == PinIndex && ExistingConvertMapping.OutputComponentIndex == ComponentIndex;
			}
		}
	);

	ConvertExpression->RefreshNode();

	InnerPin->RemoveAllConnections();
}

TOptional<float> SGraphNodeMaterialConvert::GetDefaultValue(TSharedPtr<const SConvertInnerPin> InnerPin) const
{
	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return TOptional<float>();
	}

	const int32 PinIndex = InnerPin->GetPinIndex();
	const int32 ComponentIndex = InnerPin->GetComponentIndex();

	if (InnerPin->IsInputPin())
	{
		return ConvertExpression->ConvertInputs[PinIndex].DefaultValue.Component(ComponentIndex);
	}
	else
	{
		return ConvertExpression->ConvertOutputs[PinIndex].DefaultValue.Component(ComponentIndex);
	}
}

void SGraphNodeMaterialConvert::SetDefaultValue(TSharedPtr<SConvertInnerPin> InnerPin, const float InDefaultValue)
{
	UMaterialExpressionConvert* ConvertExpression = Cast<UMaterialExpressionConvert>(MaterialNode->MaterialExpression);
	if (!ConvertExpression)
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "SetPinDefaultValue", "Set Pin Default Value" ) );
	ConvertExpression->Modify();

	const int32 PinIndex = InnerPin->GetPinIndex();
	const int32 ComponentIndex = InnerPin->GetComponentIndex();

	if (InnerPin->IsInputPin())
	{
		ConvertExpression->ConvertInputs[PinIndex].DefaultValue.Component(ComponentIndex) = InDefaultValue;
	}
	else
	{
		ConvertExpression->ConvertOutputs[PinIndex].DefaultValue.Component(ComponentIndex) = InDefaultValue;
	}

	ConvertExpression->RefreshNode();
}

TSharedPtr<SConvertInnerPin> SGraphNodeMaterialConvert::GetInnerPin(bool bInputPin, const int32 InPinIndex, const int32 InComponentIndex) const
{
	// First Find relevant input/output graph pin
	TSharedPtr<SMaterialExpressionConvertGraphPin> GraphPin = nullptr;
	if (bInputPin)
	{
		if (InputPins.IsValidIndex(InPinIndex))
		{
			GraphPin = StaticCastSharedRef<SMaterialExpressionConvertGraphPin>(InputPins[InPinIndex]);
		}
	}
	else
	{
		if (OutputPins.IsValidIndex(InPinIndex))
		{
			GraphPin = StaticCastSharedRef<SMaterialExpressionConvertGraphPin>(OutputPins[InPinIndex]);
		}
	}

	// Then get that the relevant inner pin from that graph pin
	if (GraphPin.IsValid() && GraphPin->GetInnerPins().IsValidIndex(InComponentIndex))
	{
		return GraphPin->GetInnerPins()[InComponentIndex];
	}

	// No valid inner pin found, return nullptr
	return nullptr;
}

void SGraphNodeMaterialConvert::MakeConnectionCurve(
	const FGeometry& InAllottedGeometry,
	FSlateWindowElementList& OutDrawElements,
	int32& InOutLayerId,
	const FVector2f& InCurveStart,
	const FVector2f& InCurveEnd
) const
{
	FVector2f LocalStart = InAllottedGeometry.AbsoluteToLocal(InCurveStart);
	FVector2f LocalEnd = InAllottedGeometry.AbsoluteToLocal(InCurveEnd);

	//If LocalStart.X is > LocalEnd.X, swap values
	if (LocalStart.X > LocalEnd.X)
	{
		std::swap(LocalStart, LocalEnd);
	}

	const bool bAllValuesValid = 
		(LocalStart.X != -FLT_MAX) 
		&& (LocalStart.Y != -FLT_MAX) 
		&& (LocalEnd.X != -FLT_MAX) 
		&& (LocalEnd.Y != -FLT_MAX);

	if (bAllValuesValid)
	{
		/* Because we make sure the left-most position is always the start position,
		 * curve direction is always pointing to the right */
		static const FVector2f Direction = FVector2f(100.0f, 0.0f);

		FSlateDrawElement::MakeSpline(
			OutDrawElements,
			++InOutLayerId,
			InAllottedGeometry.ToPaintGeometry(),
			LocalStart,
			Direction,
			LocalEnd,
			Direction,
			2
		);
	}
}

#undef LOCTEXT_NAMESPACE
