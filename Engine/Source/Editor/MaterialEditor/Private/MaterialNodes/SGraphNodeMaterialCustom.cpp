// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialCustom.h"

#include "GraphEditorSettings.h"
#include "MaterialGraph/MaterialGraphNode_Custom.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SGraphNodeMaterialCustom::Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode)
{
	this->GraphNode = InNode;
	this->MaterialNode = InNode;

	// Create the syntax highlighter
	FHLSLSyntaxHighlighterMarshaller::FSyntaxTextStyle CodeStyle(
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Normal"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Operator"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Keyword"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.String"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Number"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Comment"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.PreProcessorKeyword"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Error"));

	SyntaxHighlighter = FHLSLSyntaxHighlighterMarshaller::Create(CodeStyle);

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeMaterialCustom::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	UMaterialGraphNode_Custom* CustomNode = CastChecked<UMaterialGraphNode_Custom>(GraphNode);
	WeakCustomExpression = Cast<UMaterialExpressionCustom>(MaterialNode->MaterialExpression);

	TAttribute<FText> GetText;
	GetText.BindUObject(CustomNode, &UMaterialGraphNode_Custom::GetHlslText);

	FOnTextCommitted TextCommit;
	TextCommit.BindUObject(CustomNode, &UMaterialGraphNode_Custom::OnCustomHlslTextCommitted);

	SAssignNew(ShaderTextBox, SMultiLineEditableTextBox)
		.AutoWrapText(false)
		.Margin(FMargin(5, 5, 5, 5))
		.Text(GetText)
		.Visibility(this, &SGraphNodeMaterialCustom::CodeVisibility)
		.OnTextCommitted(TextCommit)
		.Marshaller(SyntaxHighlighter);

	TSharedPtr<SVerticalBox> PreviewBox;
	SAssignNew(PreviewBox, SVerticalBox);

	SGraphNodeMaterialBase::CreateBelowPinControls(PreviewBox);

	MainBox->AddSlot()
	.Padding(Settings->GetNonPinNodeBodyPadding())
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(10.f, 5.f, 10.f, 10.f))
		[
			ShaderTextBox.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			PreviewBox.ToSharedRef()
		]
	];

	MainBox->AddSlot()
	.AutoHeight()
	.Padding(FMargin(10.f, 0.f, 10.f, 5.f))
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("MaterialEditor", "CustomNodePixelDiscard", "The code contains clip() or discard instructions."))
		.ToolTipText(NSLOCTEXT("MaterialEditor", "CustomNodePixelDiscardWarning_Tooltip", "Use Blend Mode Masked to create supporting shaders.\nUse ContainsClipInstruction=No to override if this is a false positive."))
		.Visibility(this, &SGraphNodeMaterialCustom::PixelDiscardVisibility)
		.AutoWrapText(true)
	];
}

EVisibility SGraphNodeMaterialCustom::CodeVisibility() const
{
	UMaterialGraphNode_Custom* CastedNode = CastChecked<UMaterialGraphNode_Custom>(GraphNode);
	if (!CastedNode)
	{
		return EVisibility::Collapsed;
	}

	ECheckBoxState State = CastedNode->IsCodeViewChecked();

	return (State == ECheckBoxState::Checked) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SGraphNodeMaterialCustom::PixelDiscardVisibility() const
{
	UMaterialExpressionCustom const* CustomExpression = WeakCustomExpression.Get();
	if (CustomExpression == nullptr)
	{
		return EVisibility::Collapsed;
	}
	return CustomExpression->HasPixelDiscard() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphNodeMaterialCustom::CreateAdvancedViewArrow(TSharedPtr<SVerticalBox> MainBox)
{
	UMaterialGraphNode_Custom* CustomNode = CastChecked<UMaterialGraphNode_Custom>(GraphNode);

	if (!CustomNode)
	{
		return;
	}

	MainBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	.Padding(3, 0, 3, 3)
	[
		SNew(SCheckBox)
		.Visibility(this, &SGraphNodeMaterialCustom::AdvancedViewArrowVisibility)
		.OnCheckStateChanged( this, &SGraphNodeMaterialCustom::OnAdvancedViewChanged )
		.IsChecked( this, &SGraphNodeMaterialCustom::IsAdvancedViewChecked )
		.Cursor(EMouseCursor::Default)
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				. Image(this, &SGraphNodeMaterialCustom::GetAdvancedViewArrow)
			]
		]
	];
}

EVisibility SGraphNodeMaterialCustom::AdvancedViewArrowVisibility() const
{
	// Always show the arrow no matter the configuration of the custom hlsl node
	return GraphNode ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphNodeMaterialCustom::OnAdvancedViewChanged(const ECheckBoxState NewCheckedState)
{
	if (UMaterialGraphNode_Custom* CastedNode = Cast<UMaterialGraphNode_Custom>(GraphNode))
	{
		CastedNode->OnCodeViewChanged(NewCheckedState);
	}
}

ECheckBoxState SGraphNodeMaterialCustom::IsAdvancedViewChecked() const
{
	if (UMaterialGraphNode_Custom* CastedNode = Cast<UMaterialGraphNode_Custom>(GraphNode))
	{
		return CastedNode->IsCodeViewChecked();
	}

	return ECheckBoxState::Unchecked;
}

#define CHEVRON_UP TEXT("Icons.ChevronUp")
#define CHEVRON_DOWN TEXT("Icons.ChevronDown")

const FSlateBrush* SGraphNodeMaterialCustom::GetAdvancedViewArrow() const
{
	UMaterialGraphNode_Custom* CastedNode = Cast<UMaterialGraphNode_Custom>(GraphNode);
	if (!CastedNode)
	{
		return FAppStyle::GetBrush(CHEVRON_UP);
	}

	ECheckBoxState State = CastedNode->IsCodeViewChecked();

	return FAppStyle::GetBrush((State == ECheckBoxState::Checked) ? CHEVRON_UP : CHEVRON_DOWN);
}
