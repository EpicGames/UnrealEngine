// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialOperator.h"
#include "MaterialGraph/MaterialGraphNode_Operator.h"
#include "Materials/MaterialExpressionOperator.h"

#include "Widgets/Text/STextBlock.h" 
#include "Widgets/Input/SEditableTextBox.h"
#include "EditorStyleSet.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "GraphNodeMaterialOperator"

///////////////////////////////////////////////////////////////////////////////
// SMaterialExpressionOperatorGraphPin Implementation
///////////////////////////////////////////////////////////////////////////////

// Instantiation for float
template class SMaterialExpressionOperatorGraphPin<float>;

template<typename NumericType>
void SMaterialExpressionOperatorGraphPin<NumericType>::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	ArrayIndex = InArgs._ArrayIndex;
	OperatorExpression = InArgs._OperatorExpression;
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

template<typename NumericType>
TSharedRef<SWidget> SMaterialExpressionOperatorGraphPin<NumericType>::GetDefaultValueWidget()
{
	return SNew(SBox)
		.MinDesiredWidth(18.0f)
		.MaxDesiredWidth(400.0f)
		[
			SNew(SNumericEntryBox<NumericType>)
				.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
				.BorderForegroundColor(FSlateColor::UseForeground())
				.Visibility(this, &SMaterialExpressionOperatorGraphPin::GetDefaultValueVisibility)
				.IsEnabled(this, &SMaterialExpressionOperatorGraphPin::GetDefaultValueIsEditable)
				.Value(this, &SMaterialExpressionOperatorGraphPin::GetNumericValue)
				.OnValueCommitted(this, &SMaterialExpressionOperatorGraphPin::SetNumericValue)
		];
}

template<typename NumericType>
TOptional<NumericType> SMaterialExpressionOperatorGraphPin<NumericType>::GetNumericValue() const
{
	if (OperatorExpression && OperatorExpression->DynamicInputs.IsValidIndex(ArrayIndex))
	{
		return OperatorExpression->DynamicInputs[ArrayIndex].ConstValue;
	}
	return {};
}

template<typename NumericType>
void SMaterialExpressionOperatorGraphPin<NumericType>::SetNumericValue(NumericType NewValue, ETextCommit::Type)
{
	if (OperatorExpression && OperatorExpression->DynamicInputs.IsValidIndex(ArrayIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("EditNodeConstValue", "Edit Const Value"));
		OperatorExpression->Modify();
		OperatorExpression->DynamicInputs[ArrayIndex].ConstValue = NewValue;
		OperatorExpression->RefreshNode();
		if (UEdGraph* Graph = GraphPinObj->GetOwningNode()->GetGraph())
		{
			Graph->NotifyGraphChanged();
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
// SGraphNodeMaterialOperator Implementation
///////////////////////////////////////////////////////////////////////////////

void SGraphNodeMaterialOperator::Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode)
{
	this->GraphNode = InNode;
	this->MaterialNode = InNode;
	this->UpdateGraphNode();
}

UMaterialExpressionOperator* SGraphNodeMaterialOperator::GetMaterialExpression() const
{
	if (auto GraphOp = Cast<UMaterialGraphNode_Operator>(GraphNode))
	{
		return Cast<UMaterialExpressionOperator>(GraphOp->MaterialExpression);
	}
	return nullptr;
}

/**
* Create "Add Pin" Button on the material expression node
*/
void SGraphNodeMaterialOperator::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	UMaterialExpressionOperator* MaterialExpOperator = GetMaterialExpression();
	if (MaterialExpOperator && MaterialExpOperator->bAllowAddPin)
	{
		TSharedRef<SButton> AddPinButton = StaticCastSharedRef<SButton>(AddPinButtonContent(
			LOCTEXT("GraphNodeMaterial_Operator", "Add Input"),
			LOCTEXT("GraphNodeMaterial_Operator_Tooltip", "Add an input to this convert node."),
			false // bRightSide
		));

		AddPinButton->SetOnClicked(FOnClicked::CreateSP(this, &SGraphNodeMaterialOperator::OnAddPinClicked));

		MainBox->AddSlot()
			.FillHeight(1.f)
			.Padding(5.f)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Left)
			[
				AddPinButton
			];
	}
}

/**
* Create SMaterialExpressionOperatorGraphPin for each input
*/
TSharedPtr<SGraphPin> SGraphNodeMaterialOperator::CreatePinWidget(UEdGraphPin* InPin) const
{
	TArray<UEdGraphPin*> NewInputPins;
	for (UEdGraphPin* Pin : GraphNode->Pins)
	{
		if (Pin->Direction == EGPD_Input)
		{
			NewInputPins.Add(Pin);
		}
	}

	const int32 Index = NewInputPins.IndexOfByKey(InPin);
	UMaterialExpressionOperator* MaterialExpOperator = GetMaterialExpression();

	if (Index != INDEX_NONE && MaterialExpOperator)
	{
		FName PinName = MaterialExpOperator->GetInputName(Index);
		InPin->PinFriendlyName = FText::FromName(PinName);
	}

	return SNew(SMaterialExpressionOperatorGraphPin<float>, InPin)
		.ArrayIndex(Index)
		.OperatorExpression(MaterialExpOperator);
}

/**
* Create Operator DropDown Menu Button on the material expression node
*/
void SGraphNodeMaterialOperator::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	DefaultTitleAreaWidget->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(FMargin(5, 5, 40, 5))
		[
			SAssignNew(OperatorButton, SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &SGraphNodeMaterialOperator::OnOperatorMenuClicked)
				.ContentPadding(FMargin(4, 2))
				[
					SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Dropdown")))
				]
		];
	SGraphNodeMaterialBase::SetDefaultTitleAreaWidget(DefaultTitleAreaWidget);
}

FReply SGraphNodeMaterialOperator::OnAddPinClicked()
{
	UMaterialExpressionOperator* MaterialExpOperator = GetMaterialExpression();
	if (MaterialExpOperator)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddPinOperatorNode", "Add Pin on Operator Node"));
		MaterialExpOperator->Modify();
		MaterialExpOperator->AddInputPin();
		MaterialExpOperator->PostEditChange();
		MaterialExpOperator->RefreshNode();
		this->UpdateGraphNode();
		MaterialNode->ReconstructNode();
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SGraphNodeMaterialOperator::BuildOperatorMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	static const UEnum* EnumPtr = StaticEnum<EMaterialExpressionOperatorKind>();
	check(EnumPtr);
	for (int32 Idx = 0; Idx < EnumPtr->NumEnums() - 1; ++Idx)
	{
		const EMaterialExpressionOperatorKind Kind = static_cast<EMaterialExpressionOperatorKind>(EnumPtr->GetValueByIndex(Idx));
		const FText Display = EnumPtr->GetDisplayNameTextByIndex(Idx);
		MenuBuilder.AddMenuEntry(
			Display,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SGraphNodeMaterialOperator::OnOperatorMenuSelected, Kind))
		);
	}
	return MenuBuilder.MakeWidget();
}

FReply SGraphNodeMaterialOperator::OnOperatorMenuClicked()
{
	if (auto Button = OperatorButton.Pin())
	{
		TSharedRef<SWidget> MenuContent = BuildOperatorMenu();
		FWidgetPath ButtonPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(Button.ToSharedRef(), ButtonPath);
		FSlateApplication::Get().PushMenu(
			Button.ToSharedRef(),
			ButtonPath,
			MenuContent,
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SGraphNodeMaterialOperator::OnOperatorMenuSelected(EMaterialExpressionOperatorKind NewKind)
{
	if (UMaterialExpressionOperator* Expr = GetMaterialExpression())
	{
		if (Expr->Operator != NewKind)
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeOperatorKind", "Change Operator Kind"));
			Expr->Modify();
			Expr->Operator = NewKind;

			static const FName OperatorPropName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionOperator, Operator);
			FProperty* ChangedProp = FindFProperty<FProperty>(UMaterialExpressionOperator::StaticClass(), *OperatorPropName.ToString());
			FPropertyChangedEvent PropEvent(ChangedProp);
			Expr->PostEditChangeProperty(PropEvent);

			Expr->RefreshNode();
			this->UpdateGraphNode();
			MaterialNode->ReconstructNode();
		}
	}
}

FText SGraphNodeMaterialOperator::GetOperatorDisplayName() const
{
	if (UMaterialExpressionOperator* Expr = GetMaterialExpression())
	{
		static const UEnum* EnumPtr = StaticEnum<EMaterialExpressionOperatorKind>();
		if (EnumPtr)
		{
			return EnumPtr->GetDisplayNameTextByValue((int64)Expr->Operator);
		}
	}
	return LOCTEXT("SelectOperator", "Select…");
}

#undef LOCTEXT_NAMESPACE

