// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialResult.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "IDocumentation.h"
#include "Internationalization/Text.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Materials/Material.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "TutorialMetaData.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/SToolTip.h"

class SToolTip;

/////////////////////////////////////////////////////
// SGraphNodeMaterialResult

void SGraphNodeMaterialResult::Construct(const FArguments& InArgs, UMaterialGraphNode_Root* InNode)
{
	this->GraphNode = InNode;
	this->RootNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeMaterialResult::CreatePinWidgets()
{
	// Create Pin widgets for each of the pins.
	UMaterialGraphNode_Base* MaterialGraphNode = Cast<UMaterialGraphNode_Base>(GraphNode);
	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GraphNode->GetGraph());

	bool bHideNoConnectionPins = false;
	if (OwnerGraphPanelPtr.IsValid())
	{
		bHideNoConnectionPins = OwnerGraphPanelPtr.Pin()->GetPinVisibility() == SGraphEditor::Pin_HideNoConnection;
	}

	for (UEdGraphPin* CurPin : MaterialGraphNode->Pins)
	{
		const bool bPinHasConections = CurPin->LinkedTo.Num() > 0;

		bool bPinDesiresToBeHidden = bHideNoConnectionPins && !bPinHasConections;
		if (CurPin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec)
		{
			if (!MaterialGraph->MaterialInputs[CurPin->SourceIndex].IsVisiblePin(MaterialGraph->Material))
			{
				bPinDesiresToBeHidden = true;
			}
		}

		if (!bPinDesiresToBeHidden)
		{
			TSharedPtr<SGraphPin> NewPin = CreatePinWidget(CurPin);
			check(NewPin.IsValid());

			if (CurPin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec)
			{
				TSharedPtr<SToolTip> ToolTipWidget = IDocumentation::Get()->CreateToolTip(MaterialGraph->MaterialInputs[CurPin->SourceIndex].GetToolTip(), nullptr, FString(TEXT("")), FString(TEXT("")));
				NewPin->SetToolTip(ToolTipWidget.ToSharedRef());
			}

			this->AddPin(NewPin.ToSharedRef());
		}
	}
}

void SGraphNodeMaterialResult::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	RootNode->Material->EditorX = RootNode->NodePosX;
	RootNode->Material->EditorY = RootNode->NodePosY;
	RootNode->Material->MarkPackageDirty();
	RootNode->Material->MaterialGraph->MaterialDirtyDelegate.ExecuteIfBound();
}


void SGraphNodeMaterialResult::PopulateMetaTag(FGraphNodeMetaData* TagMeta) const
{
	if( (GraphNode != nullptr) && (RootNode != nullptr) )
	{		
		UMaterialGraph* OuterGraph = RootNode->GetTypedOuter<UMaterialGraph>();
		if (OuterGraph != nullptr)
		{
			TagMeta->OuterName = OuterGraph->OriginalMaterialFullName;
			// There is only one root node - so we dont need a guid. 
			TagMeta->Tag = FName(*FString::Printf(TEXT("MaterialResNode_%s"), *TagMeta->OuterName));
			TagMeta->GUID.Invalidate();
			TagMeta->FriendlyName = FString::Printf(TEXT("Material Result node in %s"), *TagMeta->OuterName);
 		}		
	}
}
