// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/SGraphSubstrateMaterial.h"
#include "Internationalization/Text.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SubstrateMaterialShared.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

#include "SGraphPin.h"
#include "Math/Color.h"
#include "Styling/StyleColors.h"
#include "SubstrateDefinitions.h"
#include "Widgets/SBoxPanel.h"
#include "MaterialValueType.h"

enum class ESubstrateWidgetOutputType : uint8
{
	Node,
	DetailPanel,
};

static EStyleColor GetSubstrateWidgetColor0() { return EStyleColor::AccentBlue;  }
static EStyleColor GetSubstrateWidgetColor1() { return EStyleColor::AccentGreen;  }
FLinearColor FSubstrateWidget::GetConnectionColor() { return FLinearColor(0.16f, 0.015f, 0.24f) * 4.f; }

bool FSubstrateWidget::HasInputSubstrateType(const UEdGraphPin* InPin)
{
	if (InPin == nullptr) return false;

	if (UMaterialGraphNode_Root* RootPinNode = Cast<UMaterialGraphNode_Root>(InPin->GetOwningNode()))
	{
		if(RootPinNode->GetPinMaterialValueType(InPin) == MCT_Substrate)
		{
			return true;
		}
	}
	if (UMaterialGraphNode* PinNode = Cast<UMaterialGraphNode>(InPin->GetOwningNode()))
	{
		FName TargetPinName = PinNode->GetShortenPinName(InPin->PinName);

		for (FExpressionInputIterator It{ PinNode->MaterialExpression }; It; ++It)
		{
			FName InputName = PinNode->MaterialExpression->GetInputName(It.Index);
			InputName = PinNode->GetShortenPinName(InputName);

			if (InputName == TargetPinName)
			{
				switch (PinNode->MaterialExpression->GetInputValueType(It.Index))
				{
					case MCT_Substrate:
						return true;
				}
				break;
			}
		}
	}
	return false;
}

bool FSubstrateWidget::HasOutputSubstrateType(const UEdGraphPin* InPin)
{
	if (InPin == nullptr || InPin->Direction != EGPD_Output) return false;

	if (UMaterialGraphNode* PinNode = Cast<UMaterialGraphNode>(InPin->GetOwningNode()))
	{
		const TArray<FExpressionOutput>& ExpressionOutputs = PinNode->MaterialExpression->GetOutputs();
		if (InPin->SourceIndex < ExpressionOutputs.Num() && PinNode->MaterialExpression->GetOutputValueType(InPin->SourceIndex) == MCT_Substrate)
		{
			return true;
		}
	}
	return false;
}

static const TSharedRef<SWidget> InternalProcessOperator(
	const FSubstrateMaterialCompilationOutput& CompilationOutput, 
	const FSubstrateOperator& Op, 
	ESubstrateWidgetOutputType OutputType,
	const TArray<FGuid>& InGuid,
	EStyleColor OverrideColor, 
	TSharedPtr<SWidget>& OutFeatureWidget)
{
	const bool bIsCurrent = OutputType == ESubstrateWidgetOutputType::Node ? InGuid.Find(Op.MaterialExpressionGuid) != INDEX_NONE : false;
	const EStyleColor Color0 = bIsCurrent ? GetSubstrateWidgetColor0() : OverrideColor;
	const EStyleColor Color1 = bIsCurrent ? GetSubstrateWidgetColor1() : OverrideColor;
	switch (Op.OperatorType)
	{
		case SUBSTRATE_OPERATOR_WEIGHT:
		{
			const EStyleColor Color = bIsCurrent ? EStyleColor::AccentGreen : OverrideColor;
			return InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], OutputType, InGuid, Color, OutFeatureWidget);
		}
		case SUBSTRATE_OPERATOR_VERTICAL:
		{
			auto VerticalOperator = SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 1.0f, 1.0f)
				[
					InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], OutputType, InGuid, Color0, OutFeatureWidget)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 1.0f, 1.0f)
				[
					InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.RightIndex], OutputType, InGuid, Color1, OutFeatureWidget)
				];
			return VerticalOperator->AsShared();
		}
		case SUBSTRATE_OPERATOR_HORIZONTAL:
		case SUBSTRATE_OPERATOR_SELECT:
		{
			auto HorizontalOperator = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 1.0f, 1.0f)
				[
					InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], OutputType, InGuid, Color0, OutFeatureWidget)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 1.0f, 1.0f)
				[
					InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.RightIndex], OutputType, InGuid, Color1, OutFeatureWidget)
				];
			return HorizontalOperator->AsShared();
		}
		case SUBSTRATE_OPERATOR_ADD:
		{
			auto HorizontalOperator = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 1.0f, 1.0f)
				[
					InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], OutputType, InGuid, Color0, OutFeatureWidget)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 1.0f, 1.0f)
				[
					InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[Op.RightIndex], OutputType, InGuid, Color1, OutFeatureWidget)
				];
			return HorizontalOperator->AsShared();
		}
		case SUBSTRATE_OPERATOR_BSDF_LEGACY:	// legacy BSDF should have been converted to BSDF already.
		case SUBSTRATE_OPERATOR_BSDF:
		{
			auto GetEyeDesc = [&Op]()
			{
				FString EyeDesc = TEXT("Eye");
				const bool bIrisN = Op.Has(ESubstrateBsdfFeature::EyeIrisNormalPluggedIn);
				const bool bIrisT = Op.Has(ESubstrateBsdfFeature::EyeIrisTangentPluggedIn);
				if (bIrisN || bIrisT)
				{
					EyeDesc += TEXT("Iris");
				}
				if (bIrisN)
				{
					EyeDesc += TEXT("N");
				}
				if (bIrisT)
				{
					EyeDesc += TEXT("T");
				}
				EyeDesc += TEXT(" ");
				return EyeDesc;
			};

			FString BSDFDesc = OutputType == ESubstrateWidgetOutputType::Node ? 
											 FString(TEXT("BSDF")) : 
											 FString::Printf(TEXT("BSDF (%s%s%s%s%s%s%s%s%s%s%s)")
											, Op.Has(ESubstrateBsdfFeature::EdgeColor) ? TEXT("F90 ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::SSS) ? TEXT("SSS ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::MFPPluggedIn) ? TEXT("MFP ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::Anisotropy) ? TEXT("Ani ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat) ? TEXT("2Ro ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::Fuzz) ? TEXT("Fuz ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::Glint) ? TEXT("Gli ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::SpecularProfile) ? TEXT("Spc ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::Eye) ? *GetEyeDesc() : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::Hair) ? TEXT("Hai ") : TEXT("")
											, Op.Has(ESubstrateBsdfFeature::Toon) ? TEXT("Toon ") : TEXT("")
											);

			static FString ToolTip;
			if (OutputType == ESubstrateWidgetOutputType::DetailPanel && ToolTip.IsEmpty())
			{
				ToolTip += TEXT("SSS means the BSDF features subsurface profile or subsurface setup using MFP.\n");
				ToolTip += TEXT("MFP means the BSDF MFP is specified by the user.\n");
				ToolTip += TEXT("F90 means the BSDF edge specular color representing reflectivity at grazing angle is used.\n");
				ToolTip += TEXT("Fuz means the BSDF fuzz layer is enabled.\n");
				ToolTip += TEXT("2Ro means the BSDF either uses a second specular lob with a second roughness, or the legacy simple clear coat.\n");
				ToolTip += TEXT("Ani means the BSDF anisotropic specular lighting is used.\n");
				ToolTip += TEXT("Gli means the BSDF features glints.\n");
				ToolTip += TEXT("Spc means the BSDF features specular profile.\n");
				ToolTip += TEXT("Eye means the BSDF features eye is used.\n");
				ToolTip += TEXT("Hai means the BSDF features hair is used.\n");
			}

			// Features and SubSurface type
			if (bIsCurrent && OutputType == ESubstrateWidgetOutputType::Node)
			{
				EMaterialSubSurfaceType SubSurfaceType = EMaterialSubSurfaceType(Op.SubSurfaceType);
				if (!Op.bIsBottom)
				{
					SubSurfaceType = EMaterialSubSurfaceType::MSS_SimpleVolume;
				}
				else if (!Op.Has(ESubstrateBsdfFeature::SSS))
				{
					SubSurfaceType = EMaterialSubSurfaceType::MSS_None;
				}

				const ESubstrateBsdfFeature DisplayedFeatureMask = 
					ESubstrateBsdfFeature::EdgeColor |
					ESubstrateBsdfFeature::Fuzz |
					ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat |
					ESubstrateBsdfFeature::Anisotropy |
					ESubstrateBsdfFeature::Glint |
					ESubstrateBsdfFeature::SpecularProfile;

				const bool bHasFeatures = EnumHasAnyFlags(Op.BSDFFeatures, DisplayedFeatureMask);

				if (bHasFeatures || SubSurfaceType != EMaterialSubSurfaceType::MSS_None)
				{
					auto Features = SNew(SVerticalBox);

					if (SubSurfaceType != EMaterialSubSurfaceType::MSS_None)
					{
						const FString BehaviorHint = SubSurfaceType != EMaterialSubSurfaceType::MSS_SimpleVolume ? TEXT("Opaque") : TEXT("Translucent");
						const FText SSSTypeName =StaticEnum<EMaterialSubSurfaceType>()->GetDisplayNameTextByValue(int64(SubSurfaceType));
						const FText Message = FText::Format(FTextFormat::FromString(TEXT("SSS {0} ({1})")), SSSTypeName, FText::FromString(BehaviorHint));

						Features->AddSlot()
						.AutoHeight()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						.Padding(0.0f, 0.0f, 1.0f, 1.0f)
						[
							SNew(SErrorText)
							.ErrorText(Message)
							.BackgroundColor(FSlateColor(FLinearColor(0.65f,0.25f,0.05f)))
						];
					}

					if (bHasFeatures)
					{
						auto HorizontalOperator = SNew(SHorizontalBox);
						auto AddFeatureSlot = [&HorizontalOperator](FText Message)
						{
							HorizontalOperator->AddSlot()
							.AutoWidth()
							.VAlign(VAlign_Fill)
							.HAlign(HAlign_Fill)
							.Padding(0.0f, 0.0f, 1.0f, 1.0f)
							[
								SNew(SErrorText)
								.ErrorText(Message)
								.BackgroundColor(FSlateColor(FLinearColor(0.16f, 0.015f, 0.24f)))
							];
						};

						if (Op.Has(ESubstrateBsdfFeature::EdgeColor))
						{
							AddFeatureSlot(FText::FromString(TEXT("F90")));
						}
						if (Op.Has(ESubstrateBsdfFeature::Fuzz))
						{
							AddFeatureSlot(FText::FromString(TEXT("Fuzz")));
						}
						if (Op.Has(ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat))
						{
							AddFeatureSlot(FText::FromString(TEXT("Dual Spec.")));
						}
						if (Op.Has(ESubstrateBsdfFeature::Anisotropy))
						{
							AddFeatureSlot(FText::FromString(TEXT("Aniso")));
						}
						if (Op.Has(ESubstrateBsdfFeature::Glint))
						{
							AddFeatureSlot(FText::FromString(TEXT("Glints")));
						}
						if (Op.Has(ESubstrateBsdfFeature::SpecularProfile))
						{
							AddFeatureSlot(FText::FromString(TEXT("Spec. Profile")));
						}

						Features->AddSlot()
						.AutoHeight()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						.Padding(0.0f, 0.0f, 1.0f, 1.0f)
						[
							HorizontalOperator
						];
					}

					OutFeatureWidget = Features.ToSharedPtr();
				}
			}

			const EStyleColor Color = OverrideColor != EStyleColor::MAX ? OverrideColor : (bIsCurrent ? EStyleColor::AccentGreen : EStyleColor::AccentGray);
			const FSlateColor SlateColor = OutputType == ESubstrateWidgetOutputType::Node ? FSlateColor(Color) : FSlateColor(FLinearColor(0.16f, 0.015f, 0.24f));
			auto BSDF = SNew(SErrorText)
				.ErrorText(FText::FromString(BSDFDesc))
				.BackgroundColor(SlateColor)
				.ToolTipText(FText::FromString(ToolTip));
			return BSDF->AsShared();
		}
	}

	static FString NoVisualization = OutputType == ESubstrateWidgetOutputType::DetailPanel ? FString(TEXT("Tree Operator Error")) : FString();
	auto TreeOperatorError = SNew(SErrorText)
		.ErrorText(FText::FromString(NoVisualization))
		.BackgroundColor(FSlateColor(EStyleColor::AccentRed));
	return TreeOperatorError->AsShared();
}

static const TSharedRef<SWidget> InternalProcessOperator(
	const FSubstrateMaterialCompilationOutput& CompilationOutput,
	const FSubstrateOperator& Op,
	ESubstrateWidgetOutputType OutputType,
	const TArray<FGuid>& InGuid,
	EStyleColor OverrideColor)
{
	TSharedPtr<SWidget> FeatureWidget;
	TSharedRef<SWidget> Topology = InternalProcessOperator(CompilationOutput, Op, OutputType, InGuid, OverrideColor, FeatureWidget);
	if (FeatureWidget)
	{
		return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.0f, 0.0f, 1.0f, 1.0f)
		[
			FeatureWidget.ToSharedRef()
		]
		+SVerticalBox::Slot()
		.MinHeight(15)
		.FillHeight(1.f)
		.Padding(0.0f, 0.0f, 1.0f, 1.0f)
		[
			SNullWidget::NullWidget
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Padding(0.0f, 0.0f, 1.0f, 1.0f)
		[
			Topology
		];
	}
	else
	{
		return Topology;
	}
}
const TSharedRef<SWidget> FSubstrateWidget::ProcessOperator(const FSubstrateMaterialCompilationOutput& CompilationOutput)
{
	return InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[CompilationOutput.RootOperatorIndex], ESubstrateWidgetOutputType::DetailPanel, TArray<FGuid>(), EStyleColor::MAX);
}

const TSharedRef<SWidget> FSubstrateWidget::ProcessOperator(const FSubstrateMaterialCompilationOutput& CompilationOutput, const TArray<FGuid>& InGuid)
{
	return InternalProcessOperator(CompilationOutput, CompilationOutput.Operators[CompilationOutput.RootOperatorIndex], ESubstrateWidgetOutputType::Node, InGuid, EStyleColor::MAX);
}

void FSubstrateWidget::GetPinColor(TSharedPtr<SGraphPin>& Out, const UMaterialGraphNode* InNode)
{	
	if (!InNode || !InNode->MaterialExpression)
	{
		return;
	}

	const FLinearColor Color0 = USlateThemeManager::Get().GetColor(GetSubstrateWidgetColor0());
	const FLinearColor Color1 = USlateThemeManager::Get().GetColor(GetSubstrateWidgetColor1());

	FLinearColor ColorModifier = FLinearColor::Black;
	bool bHasColorModifier = false;
	// Substrate operator override pin color to ease material topology visualization
	const UEdGraphPin* Pin = Out->SGraphPin::GetPinObj();
	const FName PinName = Pin->PinName;
	if (InNode->MaterialExpression->IsA(UMaterialExpressionSubstrateVerticalLayering::StaticClass()))
	{			
		if (PinName == InNode->MaterialExpression->GetInputName(0)) // Top
		{
			bHasColorModifier = true;
			ColorModifier = Color0;
		}
		else if (PinName == InNode->MaterialExpression->GetInputName(1)) // Base
		{
			bHasColorModifier = true;
			ColorModifier = Color1;
		}
	}
	else if (InNode->MaterialExpression->IsA(UMaterialExpressionSubstrateHorizontalMixing::StaticClass()))
	{
		if (PinName == InNode->MaterialExpression->GetInputName(1)) // Foreground
		{
			bHasColorModifier = true;
			ColorModifier = Color1;
		}
		else if (PinName == InNode->MaterialExpression->GetInputName(0)) // Background
		{
			bHasColorModifier = true;
			ColorModifier = Color0;
		}
	}
	else if (InNode->MaterialExpression->IsA(UMaterialExpressionSubstrateSelect::StaticClass()))
	{
		if (PinName == InNode->MaterialExpression->GetInputName(1)) // B
		{
			bHasColorModifier = true;
			ColorModifier = Color1;
		}
		else if (PinName == InNode->MaterialExpression->GetInputName(0)) // A
		{
			bHasColorModifier = true;
			ColorModifier = Color0;
		}
	}
	else if (InNode->MaterialExpression->IsA(UMaterialExpressionSubstrateAdd::StaticClass()))
	{
		if (PinName == InNode->MaterialExpression->GetInputName(0)) // A
		{
			bHasColorModifier = true;
			ColorModifier = Color0;
		}
		else if (PinName == InNode->MaterialExpression->GetInputName(1)) // B
		{
			bHasColorModifier = true;
			ColorModifier = Color1;
		}
	}
	else if (InNode->MaterialExpression->IsA(UMaterialExpressionMaterialFunctionCall::StaticClass()) ||
			 InNode->MaterialExpression->IsA(UMaterialExpressionFunctionInput::StaticClass()) ||
			 InNode->MaterialExpression->IsA(UMaterialExpressionFunctionOutput::StaticClass()))
	{
		const EMaterialValueType PinType = UMaterialGraphSchema::GetMaterialIOValueType(Pin);
		if ((Out->GetDirection() == EGPD_Input || Out->GetDirection() == EGPD_Output) && (PinType & MCT_Substrate) != 0)
		{
			bHasColorModifier = true;
			ColorModifier = FSubstrateWidget::GetConnectionColor();
		}
	}

	if (InNode->MaterialExpression->IsA(UMaterialExpressionSubstrateBSDF::StaticClass()) && Out->GetDirection() == EGPD_Output && UMaterialGraphSchema::GetMaterialIOValueType(Pin) == MCT_Substrate)
	{
		bHasColorModifier = true;
		ColorModifier = FSubstrateWidget::GetConnectionColor();
	}
	else if (Pin && UMaterialGraphSchema::GetMaterialIOValueType(Pin) == MCT_Substrate)
	{
		if (!Out->IsConnected())
		{
			bHasColorModifier = true;
			ColorModifier = FSubstrateWidget::GetConnectionColor();
		}
	}	
	
	if (bHasColorModifier)
	{
		Out->SetPinColorModifier(ColorModifier);
	}
}