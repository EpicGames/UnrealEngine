// Copyright Epic Games, Inc. All Rights Reserved.


#include "MaterialEditor/MaterialPins/SGraphPinMaterialInput.h"

#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialGraph/MaterialShaderValueTypeObject.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "MaterialEditor/SGraphSubstrateMaterial.h"
#include "MaterialValueType.h"

void SGraphPinMaterialInput::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	bool bUsePinColor = true;
	if (Substrate::IsSubstrateEnabled())
	{
		if (UMaterialGraphSchema::GetMaterialIOValueType(InGraphPinObj) == MCT_Substrate)
		{
			bUsePinColor = false;
		}			
	}
	SGraphPin::Construct(SGraphPin::FArguments().UsePinColorForText(bUsePinColor), InGraphPinObj);
}

FSlateColor SGraphPinMaterialInput::GetPinColor() const
{
	check(GraphPinObj);
	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GraphPinObj->GetOwningNode()->GetGraph());
	const UMaterialGraphSchema* Schema = CastChecked<UMaterialGraphSchema>(MaterialGraph->GetSchema());

	if (MaterialGraph->IsInputActive(GraphPinObj))
	{
		if (Substrate::IsSubstrateEnabled())
		{
			if (UMaterialGraphSchema::GetMaterialIOValueType(GraphPinObj) == MCT_Substrate)
			{
				return FSlateColor(FSubstrateWidget::GetConnectionColor());
			}
		}
		if (const UMaterialShaderValueTypeObject* ValueTypeObj = Cast<UMaterialShaderValueTypeObject>(GraphPinObj->PinType.PinSubCategoryObject.Get()))
		{
			return FSlateColor(UMaterialGraphSchema::GetColorForConnectionType(ValueTypeObj->ValueType));
		}
		return Schema->ActivePinColor;
	}
	else
	{
		return Schema->InactivePinColor;
	}
}
