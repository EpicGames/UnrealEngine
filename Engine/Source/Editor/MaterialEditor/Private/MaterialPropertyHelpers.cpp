// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialPropertyHelpers.h"
#include "Misc/MessageDialog.h"
#include "Misc/Guid.h"
#include "UObject/UnrealType.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"
#include "Styling/AppStyle.h"
#include "Materials/MaterialEnumeration.h"
#include "Materials/MaterialInterface.h"
#include "MaterialEditor/DEditorFontParameterValue.h"
#include "MaterialEditor/DEditorMaterialLayersParameterValue.h"
#include "MaterialEditor/DEditorParameterCollectionParameterValue.h"
#include "MaterialEditor/DEditorRuntimeVirtualTextureParameterValue.h"
#include "MaterialEditor/DEditorScalarParameterValue.h"
#include "MaterialEditor/DEditorStaticComponentMaskParameterValue.h"
#include "MaterialEditor/DEditorStaticSwitchParameterValue.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorVectorParameterValue.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionChannelMaskParameter.h"
#include "Materials/MaterialParameterCollection.h"
#include "EditorSupportDelegates.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "StaticParameterSet.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "Factories/MaterialFunctionInstanceFactory.h"
#include "SMaterialLayersFunctionsTree.h"
#include "Materials/MaterialFunctionMaterialLayer.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "Factories/MaterialFunctionMaterialLayerFactory.h"
#include "Factories/MaterialFunctionMaterialLayerBlendFactory.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Curves/CurveLinearColor.h"
#include "MaterialCachedData.h"
#include "Engine/UserDefinedEnum.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialPropertyHelpers)

#define LOCTEXT_NAMESPACE "MaterialPropertyHelper"

FText FMaterialPropertyHelpers::LayerID = LOCTEXT("LayerID", "Layer Asset");
FText FMaterialPropertyHelpers::BlendID = LOCTEXT("BlendID", "Blend Asset");
FName FMaterialPropertyHelpers::LayerParamName = FName("Global Material Layers Parameter Values");


void SLayerHandle::Construct(const FArguments& InArgs)
{
	OwningStack = InArgs._OwningStack;

	ChildSlot
		[
			InArgs._Content.Widget
		];
}

FReply SLayerHandle::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		{
			TSharedPtr<FDragDropOperation> DragDropOp = CreateDragDropOperation(OwningStack.Pin());
			if (DragDropOp.IsValid())
			{
				OwningStack.Pin()->OnLayerDragDetected();
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}
	}

	return FReply::Unhandled();

}

TSharedPtr<FLayerDragDropOp> SLayerHandle::CreateDragDropOperation(TSharedPtr<IDraggableItem> InOwningStack)
{
	TSharedPtr<FLayerDragDropOp> Operation = MakeShareable(new FLayerDragDropOp(InOwningStack));

	return Operation;
}

EVisibility FMaterialPropertyHelpers::ShouldShowExpression(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, FGetShowHiddenParameters ShowHiddenDelegate)
{
	bool bShowHidden = true;

	ShowHiddenDelegate.ExecuteIfBound(bShowHidden);

	bool bIsCooked = false;
	if (MaterialEditorInstance->SourceInstance)
	{
		if (UMaterial* Material = MaterialEditorInstance->SourceInstance->GetMaterial())
		{
			bIsCooked = Material->GetPackage()->bIsCookedForEditor;
		}
	}

	const bool bShouldShowExpression = bShowHidden || MaterialEditorInstance->VisibleExpressions.Contains(Parameter->ParameterInfo) || bIsCooked;

	if (MaterialEditorInstance->bShowOnlyOverrides)
	{
		return (IsOverriddenExpression(Parameter) && bShouldShowExpression) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return bShouldShowExpression ? EVisibility::Visible: EVisibility::Collapsed;
}

void FMaterialPropertyHelpers::OnMaterialLayerAssetChanged(const struct FAssetData& InAssetData, int32 Index, EMaterialParameterAssociation MaterialType, TSharedPtr<class IPropertyHandle> InHandle, FMaterialLayersFunctions* InMaterialFunction)
{
	const FScopedTransaction Transaction(LOCTEXT("SetLayerorBlendAsset", "Set Layer or Blend Asset"));
	InHandle->NotifyPreChange();
	const FName FilterTag = FName(TEXT("MaterialFunctionUsage"));
	if (InAssetData.TagsAndValues.Contains(FilterTag) || InAssetData.AssetName == NAME_None)
	{
		UMaterialFunctionInterface* LayerFunction = Cast<UMaterialFunctionInterface>(InAssetData.GetAsset());
		switch (MaterialType)
		{
		case EMaterialParameterAssociation::LayerParameter:
			if (InMaterialFunction->Layers[Index] != LayerFunction)
			{
				InMaterialFunction->Layers[Index] = LayerFunction;
				InMaterialFunction->UnlinkLayerFromParent(Index);
			}
			break;
		case EMaterialParameterAssociation::BlendParameter:
			if (InMaterialFunction->Blends[Index] != LayerFunction)
			{
				InMaterialFunction->Blends[Index] = LayerFunction;
				InMaterialFunction->UnlinkLayerFromParent(Index + 1); // Blend indices are offset by 1, no blend for base layer
			}
			break;
		default:
			break;
		}
	}
	InHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

bool FMaterialPropertyHelpers::FilterLayerAssets(const struct FAssetData& InAssetData, FMaterialLayersFunctions* LayerFunction, EMaterialParameterAssociation MaterialType, int32 Index)
{
	bool bShouldAssetBeFilteredOut = true;

	const FName FilterTag = FName(TEXT("MaterialFunctionUsage"));
	FAssetDataTagMapSharedView::FFindTagResult MaterialFunctionUsage = InAssetData.TagsAndValues.FindTag(FilterTag);
	if (MaterialFunctionUsage.IsSet())
	{
		static const UEnum* FunctionUsageEnum = StaticEnum<EMaterialFunctionUsage>();
		if (!FunctionUsageEnum)
		{
			return bShouldAssetBeFilteredOut;
		}

		const FName BaseTag = FName(TEXT("Base"));
		FAssetDataTagMapSharedView::FFindTagResult Base = InAssetData.TagsAndValues.FindTag(BaseTag);
		FSoftObjectPath AssetPath = LayerFunction->LayerStackCache != nullptr && Base.IsSet() ? FSoftObjectPath(Base.GetValue()) : InAssetData.ToSoftObjectPath();

		FString RightPath;
		bool bLegacyShouldRestrict = false;
		switch (MaterialType)
		{
		case EMaterialParameterAssociation::LayerParameter:
		{
			if (MaterialFunctionUsage.GetValue() == FunctionUsageEnum->GetNameStringByValue((int64)EMaterialFunctionUsage::Default) 
				|| MaterialFunctionUsage.GetValue() == FunctionUsageEnum->GetNameStringByValue((int64)EMaterialFunctionUsage::MaterialLayer))
			{
				//If we are using the new layer stack node, we only need to ensure whether the asset is usable.
				if (LayerFunction->LayerStackCache)
				{
					bShouldAssetBeFilteredOut = !LayerFunction->LayerStackCache->AvailableLayerPaths.Contains(AssetPath);
				}
				else
				{
					bShouldAssetBeFilteredOut = false;
					bLegacyShouldRestrict = LayerFunction->EditorOnly.RestrictToLayerRelatives[Index];
					if (bLegacyShouldRestrict && LayerFunction->Layers[Index] != nullptr)
					{
						RightPath = LayerFunction->Layers[Index]->GetBaseFunction()->GetFName().ToString();
						if (RightPath.IsEmpty())
						{
							RightPath = LayerFunction->Layers[Index]->GetFName().ToString();
						}
					}
				}
			}
			break;
		}
		case EMaterialParameterAssociation::BlendParameter:
		{
			if (MaterialFunctionUsage.GetValue() == FunctionUsageEnum->GetNameStringByValue((int64)EMaterialFunctionUsage::MaterialLayerBlend))
			{
				if (LayerFunction->LayerStackCache)
				{
					bShouldAssetBeFilteredOut = !LayerFunction->LayerStackCache->AvailableBlendPaths.Contains(AssetPath);
				}
				else
				{
					bShouldAssetBeFilteredOut = false;
					bLegacyShouldRestrict = LayerFunction->EditorOnly.RestrictToBlendRelatives[Index];
					if (bLegacyShouldRestrict && LayerFunction->Blends[Index] != nullptr)
					{
						RightPath = LayerFunction->Blends[Index]->GetBaseFunction()->GetFName().ToString();
						if (RightPath.IsEmpty())
						{
							RightPath = LayerFunction->Blends[Index]->GetFName().ToString();
						}
					}
				}
			}
			break;
		}
		default:
			break;
		}

		if (bLegacyShouldRestrict)
		{
			FString BaseString;
			FString DiscardString;
			FString CleanString;
			if (Base.IsSet())
			{
				BaseString = Base.GetValue();
				BaseString.Split(".", &DiscardString, &CleanString);
				CleanString.Split("'", &CleanString, &DiscardString);
			}
			else
			{
				CleanString = InAssetData.AssetName.ToString();
			}

			bShouldAssetBeFilteredOut = CleanString != RightPath;
		}
	}
	return bShouldAssetBeFilteredOut;
}

FReply FMaterialPropertyHelpers::OnClickedSaveNewMaterialInstance(UMaterialInterface* Parent, UObject* EditorObject)
{
	const FString DefaultSuffix = TEXT("_Inst");
	TArray<FEditorParameterGroup> ParameterGroups;
	UMaterialEditorInstanceConstant* MaterialInstanceEditor = Cast<UMaterialEditorInstanceConstant>(EditorObject);
	if (MaterialInstanceEditor)
	{
		ParameterGroups = MaterialInstanceEditor->ParameterGroups;
	}
	UMaterialEditorPreviewParameters* MaterialEditor = Cast<UMaterialEditorPreviewParameters>(EditorObject);
	if (MaterialEditor)
	{
		ParameterGroups = MaterialEditor->ParameterGroups;
	}
	if (!MaterialEditor && !MaterialInstanceEditor)
	{
		return FReply::Unhandled();
	}

	if (Parent)
	{
		// Create an appropriate and unique name 
		FString Name;
		FString PackageName;
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(Parent->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

		UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
		Factory->InitialParent = Parent;

		UObject* Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialInstanceConstant::StaticClass(), Factory);
		UMaterialInstanceConstant* ChildInstance = Cast<UMaterialInstanceConstant>(Child);
		CopyMaterialToInstance(ChildInstance, ParameterGroups);

	}
	return FReply::Handled();
}

void FMaterialPropertyHelpers::CopyMaterialToInstance(UMaterialInstanceConstant* ChildInstance, TArray<FEditorParameterGroup> &ParameterGroups)
{
	if (ChildInstance)
	{
		if (ChildInstance->IsTemplate(RF_ClassDefaultObject) == false)
		{
			ChildInstance->MarkPackageDirty();
			ChildInstance->ClearParameterValuesEditorOnly();

			FMaterialInstanceParameterUpdateContext UpdateContext(ChildInstance);

			//propagate changes to the base material so the instance will be updated if it has a static permutation resource
			for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
			{
				FEditorParameterGroup& Group = ParameterGroups[GroupIdx];
				for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
				{
					UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
					if (Parameter && Parameter->bOverride)
					{
						FMaterialParameterMetadata ParameterResult;
						if (Parameter->GetValue(ParameterResult))
						{
							UpdateContext.SetParameterValueEditorOnly(Parameter->ParameterInfo, ParameterResult);
						}
						// This is called to initialize newly created child MIs
						// Don't need to copy layers from parent, since they will not initially be changed from parent values
					}
				}
			}
		}
	}
}


void FMaterialPropertyHelpers::TransitionAndCopyParameters(UMaterialInstanceConstant* ChildInstance, TArray<FEditorParameterGroup> &ParameterGroups, bool bForceCopy)
{
	if (ChildInstance)
	{
		if (ChildInstance->IsTemplate(RF_ClassDefaultObject) == false)
		{
			ChildInstance->MarkPackageDirty();
			ChildInstance->ClearParameterValuesEditorOnly();
			//propagate changes to the base material so the instance will be updated if it has a static permutation resource
			//FStaticParameterSet NewStaticParameters;
			FMaterialInstanceParameterUpdateContext UpdateContext(ChildInstance);
			for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
			{
				FEditorParameterGroup& Group = ParameterGroups[GroupIdx];
				for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
				{
					UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
					if (Parameter && (Parameter->bOverride || bForceCopy))
					{
						FMaterialParameterMetadata EditorValue;
						if (Parameter->GetValue(EditorValue))
						{
							FMaterialParameterInfo TransitionedScalarInfo = FMaterialParameterInfo();
							TransitionedScalarInfo.Name = Parameter->ParameterInfo.Name;
							UpdateContext.SetParameterValueEditorOnly(TransitionedScalarInfo, EditorValue);
						}
					}
				}
			}
		}
	}
}

FReply FMaterialPropertyHelpers::OnClickedSaveNewFunctionInstance(class UMaterialFunctionInterface* Object, class UMaterialInterface* PreviewMaterial, UObject* EditorObject)
{
	const FString DefaultSuffix = TEXT("_Inst");
	TArray<FEditorParameterGroup> ParameterGroups;
	UMaterialEditorInstanceConstant* MaterialInstanceEditor = Cast<UMaterialEditorInstanceConstant>(EditorObject);
	UMaterialInterface* FunctionPreviewMaterial = nullptr;
	if (MaterialInstanceEditor)
	{
		ParameterGroups = MaterialInstanceEditor->ParameterGroups;
		FunctionPreviewMaterial = MaterialInstanceEditor->SourceInstance;
	}
	UMaterialEditorPreviewParameters* MaterialEditor = Cast<UMaterialEditorPreviewParameters>(EditorObject);
	if (MaterialEditor)
	{
		ParameterGroups = MaterialEditor->ParameterGroups;
		FunctionPreviewMaterial = PreviewMaterial;
	}
	if (!MaterialEditor && !MaterialInstanceEditor)
	{
		return FReply::Unhandled();
	}

	if (Object)
	{
		UMaterial* EditedMaterial = Cast<UMaterial>(FunctionPreviewMaterial);
		if (EditedMaterial)
		{

			UMaterialInstanceConstant* ProxyMaterial = NewObject<UMaterialInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);
			ProxyMaterial->SetParentEditorOnly(EditedMaterial);
			ProxyMaterial->PreEditChange(NULL);
			ProxyMaterial->PostEditChange();
			CopyMaterialToInstance(ProxyMaterial, ParameterGroups);
			FunctionPreviewMaterial = ProxyMaterial;
		}
		// Create an appropriate and unique name 
		FString Name;
		FString PackageName;
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

		
		UObject* Child;
		if (Object->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayer)
		{
			UMaterialFunctionMaterialLayerInstanceFactory* LayerFactory = NewObject<UMaterialFunctionMaterialLayerInstanceFactory>();
			LayerFactory->InitialParent = Object;
			Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunctionMaterialLayerInstance::StaticClass(), LayerFactory);
		}
		else if (Object->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayerBlend)
		{
			UMaterialFunctionMaterialLayerBlendInstanceFactory* BlendFactory = NewObject<UMaterialFunctionMaterialLayerBlendInstanceFactory>();
			BlendFactory->InitialParent = Object;
			Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunctionMaterialLayerBlendInstance::StaticClass(), BlendFactory);
		}
		else
		{
			UMaterialFunctionInstanceFactory* Factory = NewObject<UMaterialFunctionInstanceFactory>();
			Factory->InitialParent = Object;
			Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunctionInstance::StaticClass(), Factory);
		}

		UMaterialFunctionInstance* ChildInstance = Cast<UMaterialFunctionInstance>(Child);
		if (ChildInstance)
		{
			if (ChildInstance->IsTemplate(RF_ClassDefaultObject) == false)
			{
				ChildInstance->MarkPackageDirty();
				ChildInstance->SetParent(Object);
				UMaterialInstance* EditedInstance = Cast<UMaterialInstance>(FunctionPreviewMaterial);
				if (EditedInstance)
				{
					ChildInstance->ScalarParameterValues = EditedInstance->ScalarParameterValues;
					ChildInstance->VectorParameterValues = EditedInstance->VectorParameterValues;
					ChildInstance->DoubleVectorParameterValues = EditedInstance->DoubleVectorParameterValues;
					ChildInstance->TextureParameterValues = EditedInstance->TextureParameterValues;
					ChildInstance->TextureCollectionParameterValues = EditedInstance->TextureCollectionParameterValues;
					ChildInstance->RuntimeVirtualTextureParameterValues = EditedInstance->RuntimeVirtualTextureParameterValues;
					ChildInstance->SparseVolumeTextureParameterValues = EditedInstance->SparseVolumeTextureParameterValues;
					ChildInstance->FontParameterValues = EditedInstance->FontParameterValues;

					const FStaticParameterSet& StaticParameters = EditedInstance->GetStaticParameters();
					ChildInstance->StaticSwitchParameterValues = StaticParameters.StaticSwitchParameters;
					ChildInstance->StaticComponentMaskParameterValues = StaticParameters.EditorOnly.StaticComponentMaskParameters;
				}
			}
		}
	}
	return FReply::Handled();
}


FReply FMaterialPropertyHelpers::OnClickedSaveNewLayerInstance(class UMaterialFunctionInterface* Object, TSharedPtr<FSortedParamData> InSortedData)
{
	const FString DefaultSuffix = TEXT("_Inst");
	TArray<FEditorParameterGroup> ParameterGroups;
	UMaterialInterface* FunctionPreviewMaterial = nullptr;
	if (Object)
	{
		FunctionPreviewMaterial = Object->GetPreviewMaterial();
	}
	for (TSharedPtr<FSortedParamData> Group : InSortedData->Children)
	{
		FEditorParameterGroup DuplicatedGroup = FEditorParameterGroup();
		DuplicatedGroup.GroupAssociation = Group->Group.GroupAssociation;
		DuplicatedGroup.GroupName = Group->Group.GroupName;
		DuplicatedGroup.GroupSortPriority = Group->Group.GroupSortPriority;
		for (UDEditorParameterValue* Parameter : Group->Group.Parameters)
		{
			if (Parameter->ParameterInfo.Index == InSortedData->ParameterInfo.Index)
			{
				DuplicatedGroup.Parameters.Add(Parameter);
			}
		}
		ParameterGroups.Add(DuplicatedGroup);
	}

	if (Object)
	{
		UMaterialInterface* EditedMaterial = FunctionPreviewMaterial;
		if (EditedMaterial)
		{
			UMaterialInstanceConstant* ProxyMaterial = NewObject<UMaterialInstanceConstant>(GetTransientPackage(), NAME_None, RF_Transactional);
			ProxyMaterial->SetParentEditorOnly(EditedMaterial);
			ProxyMaterial->PreEditChange(NULL);
			ProxyMaterial->PostEditChange();
			TransitionAndCopyParameters(ProxyMaterial, ParameterGroups);
			FunctionPreviewMaterial = ProxyMaterial;
		}
		// Create an appropriate and unique name 
		FString Name;
		FString PackageName;
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);


		UObject* Child;
		if (Object->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayer)
		{
			UMaterialFunctionMaterialLayerInstanceFactory* LayerFactory = NewObject<UMaterialFunctionMaterialLayerInstanceFactory>();
			LayerFactory->InitialParent = Object;
			Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunctionMaterialLayerInstance::StaticClass(), LayerFactory);
		}
		else if (Object->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayerBlend)
		{
			UMaterialFunctionMaterialLayerBlendInstanceFactory* BlendFactory = NewObject<UMaterialFunctionMaterialLayerBlendInstanceFactory>();
			BlendFactory->InitialParent = Object;
			Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunctionMaterialLayerBlendInstance::StaticClass(), BlendFactory);
		}
		else
		{
			UMaterialFunctionInstanceFactory* Factory = NewObject<UMaterialFunctionInstanceFactory>();
			Factory->InitialParent = Object;
			Child = AssetToolsModule.Get().CreateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialFunctionInstance::StaticClass(), Factory);
		}

		UMaterialFunctionInstance* ChildInstance = Cast<UMaterialFunctionInstance>(Child);
		if (ChildInstance)
		{
			if (ChildInstance->IsTemplate(RF_ClassDefaultObject) == false)
			{
				ChildInstance->MarkPackageDirty();
				ChildInstance->SetParent(Object);
				UMaterialInstance* EditedInstance = Cast<UMaterialInstance>(FunctionPreviewMaterial);
				if (EditedInstance)
				{
					ChildInstance->ScalarParameterValues = EditedInstance->ScalarParameterValues;
					ChildInstance->VectorParameterValues = EditedInstance->VectorParameterValues;
					ChildInstance->DoubleVectorParameterValues = EditedInstance->DoubleVectorParameterValues;
					ChildInstance->TextureParameterValues = EditedInstance->TextureParameterValues;
					ChildInstance->TextureCollectionParameterValues = EditedInstance->TextureCollectionParameterValues;
					ChildInstance->RuntimeVirtualTextureParameterValues = EditedInstance->RuntimeVirtualTextureParameterValues;
					ChildInstance->SparseVolumeTextureParameterValues = EditedInstance->SparseVolumeTextureParameterValues;
					ChildInstance->FontParameterValues = EditedInstance->FontParameterValues;

					const FStaticParameterSet& StaticParameters = EditedInstance->GetStaticParameters();
					ChildInstance->StaticSwitchParameterValues = StaticParameters.StaticSwitchParameters;
					ChildInstance->StaticComponentMaskParameterValues = StaticParameters.EditorOnly.StaticComponentMaskParameters;
				}
			}
		}
	}
	return FReply::Handled();
}

bool FMaterialPropertyHelpers::ShouldCreatePropertyRowForParameter(UDEditorParameterValue* Parameter)
{
	if (UDEditorMaterialLayersParameterValue* LayersParam = Cast<UDEditorMaterialLayersParameterValue>(Parameter))
	{
		// No widget for layers
		return false;
	}
	else if (UDEditorParameterCollectionParameterValue* ParamCollectionParam = Cast<UDEditorParameterCollectionParameterValue>(Parameter))
	{
		// No widget for MPCs, special override category in MI
		return false;
	}
	else if (FMaterialPropertyHelpers::UsesCustomPrimitiveData(Parameter))
	{
		// No widget for CPD
		return false;
	}

	return true;
}

bool FMaterialPropertyHelpers::ShouldCreateChildPropertiesForParameter(UDEditorParameterValue* Parameter)
{
	if (UDEditorStaticComponentMaskParameterValue* CompMaskParam = Cast<UDEditorStaticComponentMaskParameterValue>(Parameter))
	{
		// Static component mask has a checkbox for each channel
		return false;
	}

	return true;
}

void FMaterialPropertyHelpers::ConfigurePropertyRowForParameter(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorInstanceConstant* MaterialEditorInstance, const FMaterialParameterPropertyRowOverrides& RowOverrides /*= FMaterialParameterPropertyRowOverrides()*/)
{
	// Set name and tooltip
	PropertyRow
		.DisplayName(FText::FromName(Parameter->ParameterInfo.Name))
		.ToolTip(FMaterialPropertyHelpers::GetParameterTooltip(Parameter, MaterialEditorInstance))
		.OverrideResetToDefault(FMaterialPropertyHelpers::CreateResetToDefaultOverride(Parameter, MaterialEditorInstance, ParameterValueProperty));

	// Set Visibility
	if (RowOverrides.Visibility.IsSet())
	{
		PropertyRow.Visibility(RowOverrides.Visibility);
	}
	else
	{
		PropertyRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FMaterialPropertyHelpers::ShouldShowExpression, Parameter, MaterialEditorInstance, RowOverrides.ShowHiddenDelegate)));
	}

	// Set EditCondition
	TAttribute<bool> EditConditionValue = RowOverrides.EditConditionValue;
	if (!EditConditionValue.IsSet())
	{
		EditConditionValue = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(&FMaterialPropertyHelpers::IsOverriddenExpression, Parameter));
	}

	FOnBooleanValueChanged OnEditConditionValueChanged;
	if (RowOverrides.OnEditConditionValueChanged.IsSet())
	{
		OnEditConditionValueChanged = RowOverrides.OnEditConditionValueChanged.GetValue();
	}
	else
	{
		OnEditConditionValueChanged = FOnBooleanValueChanged::CreateStatic(&FMaterialPropertyHelpers::OnOverrideParameter, Parameter, MaterialEditorInstance);
	}

	PropertyRow.EditCondition(EditConditionValue, OnEditConditionValueChanged);
}

void FMaterialPropertyHelpers::SetPropertyRowParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters)
{
	bool bUseDefaultWidget = false;

	// Set custom widgets if this parameter needs one. Otherwise, the default widget for the property will be used
	if (UDEditorStaticComponentMaskParameterValue* CompMaskParam = Cast<UDEditorStaticComponentMaskParameterValue>(Parameter))
	{
		// Custom widget for static component masks
		SetPropertyRowMaskParameterWidget(PropertyRow, Parameter, ParameterValueProperty);
	}
	else if (UDEditorTextureParameterValue* TextureParam = Cast<UDEditorTextureParameterValue>(Parameter))
	{
		// Custom widget for textures
		SetPropertyRowTextureParameterWidget(PropertyRow, Parameter, ParameterValueProperty, MaterialEditorParameters);
	}
	else if (UDEditorVectorParameterValue* VectorParam = Cast<UDEditorVectorParameterValue>(Parameter))
	{
		// Don't display custom primitive data parameters in the details panel.
		// This data is pulled from the primitive and can't be changed on the material.
		if (VectorParam->bUseCustomPrimitiveData)
		{
			return;
		}

		// Set channel names if specified
		static const FName Red("R");
		static const FName Green("G");
		static const FName Blue("B");
		static const FName Alpha("A");
		if (!VectorParam->ChannelNames.R.IsEmpty())
		{
			ParameterValueProperty->GetChildHandle(Red)->SetPropertyDisplayName(VectorParam->ChannelNames.R);
		}
		if (!VectorParam->ChannelNames.G.IsEmpty())
		{
			ParameterValueProperty->GetChildHandle(Green)->SetPropertyDisplayName(VectorParam->ChannelNames.G);
		}
		if (!VectorParam->ChannelNames.B.IsEmpty())
		{
			ParameterValueProperty->GetChildHandle(Blue)->SetPropertyDisplayName(VectorParam->ChannelNames.B);
		}
		if (!VectorParam->ChannelNames.A.IsEmpty())
		{
			ParameterValueProperty->GetChildHandle(Alpha)->SetPropertyDisplayName(VectorParam->ChannelNames.A);
		}

		// Custom widget for channel masks
		if (VectorParam->bIsUsedAsChannelMask)
		{
			SetPropertyRowVectorChannelMaskParameterWidget(PropertyRow, Parameter, ParameterValueProperty, MaterialEditorParameters);
		}
		else
		{
			bUseDefaultWidget = true;
		}
	}
	else if (UDEditorScalarParameterValue* ScalarParam = Cast<UDEditorScalarParameterValue>(Parameter))
	{
		// Don't display custom primitive data parameters in the details panel.
		// This data is pulled from the primitive and can't be changed on the material.
		if (ScalarParam->bUseCustomPrimitiveData)
		{
			return;
		}

		// Atlas position params handled separately
		if (ScalarParam->AtlasData.bIsUsedAsAtlasPosition)
		{
			SetPropertyRowScalarAtlasPositionParameterWidget(PropertyRow, Parameter, ParameterValueProperty, MaterialEditorParameters);
		}
		else
		{
			// Configure slider if specified
			if (ScalarParam->SliderMax > ScalarParam->SliderMin)
			{
				ParameterValueProperty->SetInstanceMetaData("UIMin", FString::Printf(TEXT("%f"), ScalarParam->SliderMin));
				ParameterValueProperty->SetInstanceMetaData("UIMax", FString::Printf(TEXT("%f"), ScalarParam->SliderMax));
			}

			// Custom widget for enums
			TSoftObjectPtr<UObject> Enumeration;
			if (!ScalarParam->Enumeration.IsNull())
			{
				Enumeration = ScalarParam->Enumeration;
			}
			else if (ScalarParam->EnumerationIndex >= 0)
			{
				if (UMaterialEditorInstanceConstant* MaterialEditorInstance = Cast<UMaterialEditorInstanceConstant>(MaterialEditorParameters))
				{
					Enumeration = MaterialEditorInstance->GetEnumerationObject(ScalarParam->EnumerationIndex);
				}
			}

			if (!Enumeration.IsNull())
			{
				SetPropertyRowScalarEnumerationParameterWidget(PropertyRow, Parameter, ParameterValueProperty, Enumeration);
			}
			else
			{
				bUseDefaultWidget = true;
			}
		}
	}
	else
	{
		bUseDefaultWidget = true;
	}

	// Fix up property name if not using a custom widget. Otherwise, the name will display as "Parameter Value"
	if (bUseDefaultWidget)
	{
		if (FDetailWidgetDecl* NameWidgetPtr = PropertyRow.CustomNameWidget())
		{
			// Already customized, just set the name widget
			(*NameWidgetPtr)
			[
				ParameterValueProperty->CreatePropertyNameWidget()
			];
		}
		else
		{
			// Not customized, grab default widgets and customize
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget);

			const bool bShowChildren = true;
			PropertyRow.CustomWidget(bShowChildren)
				.NameContent()
				[
					ParameterValueProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					ValueWidget.ToSharedRef()
				];
		}
	}
}

void FMaterialPropertyHelpers::SetPropertyRowMaskParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty)
{
	TSharedPtr<SHorizontalBox> ValueHorizontalBox;

	// Set up the custom widget
	PropertyRow.CustomWidget()
		.NameContent()
		[
			ParameterValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(200.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ValueHorizontalBox, SHorizontalBox)
			]
		];

	// Add checkboxes for each mask channel
	auto AddChannelWidgets =
		[&ValueHorizontalBox](TSharedPtr<IPropertyHandle> MaskProperty)
		{
			FMargin NameWidgetPadding = FMargin(10.0f, 0.0f, 2.0f, 0.0f);
			if (ValueHorizontalBox->NumSlots() == 0)
			{
				// No left padding on the first widget
				NameWidgetPadding.Left = 0.0f;
			}

			ValueHorizontalBox->AddSlot()
				.HAlign(HAlign_Left)
				.Padding(NameWidgetPadding)
				.AutoWidth()
				[
					MaskProperty->CreatePropertyNameWidget()
				];

			ValueHorizontalBox->AddSlot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					MaskProperty->CreatePropertyValueWidget()
				];
		};

	AddChannelWidgets(ParameterValueProperty->GetChildHandle("R"));
	AddChannelWidgets(ParameterValueProperty->GetChildHandle("G"));
	AddChannelWidgets(ParameterValueProperty->GetChildHandle("B"));
	AddChannelWidgets(ParameterValueProperty->GetChildHandle("A"));
}

void FMaterialPropertyHelpers::SetPropertyRowVectorChannelMaskParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters)
{
	// Combo box hooks for converting between our "enum" and colors
	FOnGetPropertyComboBoxStrings GetMaskStrings = FOnGetPropertyComboBoxStrings::CreateStatic(&FMaterialPropertyHelpers::GetVectorChannelMaskComboBoxStrings);
	FOnGetPropertyComboBoxValue GetMaskValue = FOnGetPropertyComboBoxValue::CreateStatic(&FMaterialPropertyHelpers::GetVectorChannelMaskValue, Parameter);
	FOnPropertyComboBoxValueSelected SetMaskValue = FOnPropertyComboBoxValueSelected::CreateStatic(&FMaterialPropertyHelpers::SetVectorChannelMaskValue, ParameterValueProperty, Parameter, (UObject*)MaterialEditorParameters);

	// Widget replaces color picker with combo box
	const bool bShowChildren = true;
	PropertyRow.CustomWidget(bShowChildren)
		.NameContent()
		[
			ParameterValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(200.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					PropertyCustomizationHelpers::MakePropertyComboBox(ParameterValueProperty, GetMaskStrings, GetMaskValue, SetMaskValue)
				]
			]
		];
}

void FMaterialPropertyHelpers::SetPropertyRowScalarAtlasPositionParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters)
{
	const FText ParameterName = FText::FromName(Parameter->ParameterInfo.Name);
	UDEditorScalarParameterValue* AtlasParameter = Cast<UDEditorScalarParameterValue>(Parameter);

	// Create the value widget first for GetDesiredWidth
	TSharedRef<SObjectPropertyEntryBox> ObjectPropertyEntryBox = SNew(SObjectPropertyEntryBox)
		.ObjectPath_UObject(AtlasParameter, &UDEditorScalarParameterValue::GetAtlasCurvePath)
		.AllowedClass(UCurveLinearColor::StaticClass())
		.NewAssetFactories(TArray<UFactory*>())
		.DisplayThumbnail(true)
		.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
		.OnShouldFilterAsset(FOnShouldFilterAsset::CreateStatic(&FMaterialPropertyHelpers::OnShouldFilterCurveAsset, AtlasParameter->AtlasData.Atlas))
		.OnShouldSetAsset(FOnShouldSetAsset::CreateStatic(&FMaterialPropertyHelpers::OnShouldSetCurveAsset, AtlasParameter->AtlasData.Atlas))
		.OnObjectChanged(FOnSetObject::CreateStatic(&FMaterialPropertyHelpers::SetPositionFromCurveAsset, AtlasParameter->AtlasData.Atlas, AtlasParameter, ParameterValueProperty, (UObject*)MaterialEditorParameters))
		.DisplayCompactSize(true);

	float MinDesiredWidth, MaxDesiredWidth;
	ObjectPropertyEntryBox->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);

	// Set up the custom widget
	PropertyRow.CustomWidget()
		.NameContent()
		[
			ParameterValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(MinDesiredWidth)
		.MaxDesiredWidth(MaxDesiredWidth)
		[
			ObjectPropertyEntryBox
		];
}

void FMaterialPropertyHelpers::SetPropertyRowScalarEnumerationParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, const TSoftObjectPtr<UObject>& SoftEnumeration)
{
	if (SoftEnumeration.IsNull())
	{
		return;
	}

	// Scalar parameters can present an enumeration combo box.
	PropertyRow.CustomWidget()
		.NameContent()
		[
			ParameterValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			FMaterialPropertyHelpers::CreateScalarEnumerationParameterValueWidget(SoftEnumeration, ParameterValueProperty)
		];
}

void FMaterialPropertyHelpers::SetPropertyRowTextureParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters)
{
	TWeakObjectPtr<UMaterialExpressionTextureSampleParameter> SamplerExpression;

	// Cache the SamplerExpression for asset filtering below
	UDEditorTextureParameterValue* TextureParam = Cast<UDEditorTextureParameterValue>(Parameter);
	if (TextureParam)
	{
		UMaterial* Material = Cast<UMaterial>(MaterialEditorParameters->GetMaterialInterface());
		if (Material != nullptr)
		{
			UMaterialExpressionTextureSampleParameter* Expression = Material->FindExpressionByGUID<UMaterialExpressionTextureSampleParameter>(TextureParam->ExpressionId);
			if (Expression != nullptr)
			{
				SamplerExpression = Expression;
			}
		}
	}

	// Create the value widget first for GetDesiredWidth
	TSharedRef<SObjectPropertyEntryBox> ObjectPropertyEntryBox = SNew(SObjectPropertyEntryBox)
		.PropertyHandle(ParameterValueProperty)
		.AllowedClass(UTexture::StaticClass())
		.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
		.OnShouldFilterAsset_Lambda([SamplerExpression](const FAssetData& AssetData)
			{
				if (SamplerExpression.Get())
				{
					bool VirtualTextured = false;
					AssetData.GetTagValue<bool>("VirtualTextureStreaming", VirtualTextured);

					bool ExpressionIsVirtualTextured = IsVirtualSamplerType(SamplerExpression->SamplerType);

					return VirtualTextured != ExpressionIsVirtualTextured;
				}
				else
				{
					return false;
				}
			});

	float MinDesiredWidth, MaxDesiredWidth;
	ObjectPropertyEntryBox->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);

	TSharedPtr<SVerticalBox> NameVerticalBox;

	// Set up the custom widget
	PropertyRow.CustomWidget()
		.NameContent()
		[
			SAssignNew(NameVerticalBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					ParameterValueProperty->CreatePropertyNameWidget()
				]
		]
		.ValueContent()
		.MinDesiredWidth(MinDesiredWidth)
		.MaxDesiredWidth(MaxDesiredWidth)
		[
			ObjectPropertyEntryBox
		];

	// Add extra widgets for specified channel names, if any
	auto AddChannelNameWidgets =
		[&NameVerticalBox](const FText& ChannelName, FName ChannelKey)
		{
			if (ChannelName.IsEmpty())
			{
				return;
			}

			NameVerticalBox->AddSlot()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(20.0, 2.0, 4.0, 2.0)
					[
						SNew(STextBlock)
							.Text(FText::FromName(ChannelKey))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.Padding(4.0, 2.0)
					[
						SNew(STextBlock)
							.Text(ChannelName)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					]
			];
		};

	static const FName Red("R");
	static const FName Green("G");
	static const FName Blue("B");
	static const FName Alpha("A");
	AddChannelNameWidgets(TextureParam->ChannelNames.R, Red);
	AddChannelNameWidgets(TextureParam->ChannelNames.G, Green);
	AddChannelNameWidgets(TextureParam->ChannelNames.B, Blue);
	AddChannelNameWidgets(TextureParam->ChannelNames.A, Alpha);
}

void FMaterialPropertyHelpers::SetPropertyRowParameterCollectionParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, class UMaterialParameterCollection* BaseParameterCollection, TArray<TSharedRef<IPropertyHandle>>&& AllPropertyHandles)
{
	// Display the parameter collection's name on the widget.
	ParameterValueProperty->SetPropertyDisplayName(FText::FromName(BaseParameterCollection->GetFName()));

	// Create a delegate to invoke when the property value changes.
	TDelegate<void(const FPropertyChangedEvent&)> OnPropertyValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateWeakLambda(BaseParameterCollection,
		[BaseParameterCollection, PropertyHandles = MoveTemp(AllPropertyHandles)](const FPropertyChangedEvent& PropertyChangedEvent)
		{
			// Determine if this is the final setting of the property value.
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet)
			{
				return;
			}

			if (PropertyHandles.IsEmpty())
			{
				return;
			}

			// Get the new value from the first property handle.
			UObject* Value = nullptr;
			if (PropertyHandles[0]->GetValue(Value) != FPropertyAccess::Success)
			{
				return;
			}

			// Verify that the new value is valid.
			if (auto* Collection = Cast<UMaterialParameterCollection>(Value))
			{
				if (Collection->IsInstanceOf(BaseParameterCollection->StateId))
				{
					// Set the same value on all of the collection's referencing parameters.
					for (int32 PropertyHandleIndex = 1; PropertyHandleIndex < PropertyHandles.Num(); ++PropertyHandleIndex)
					{
						PropertyHandles[PropertyHandleIndex]->SetValue(Value);
					}

					return;
				}
			}

			// Reset the value to the default parameter collection.
			Value = BaseParameterCollection;
			PropertyHandles[0]->SetValue(Value);
		});

	// Invoke the delegate when the property value or one of its child property values changes.
	ParameterValueProperty->SetOnPropertyValueChangedWithData(OnPropertyValueChangedDelegate);
	ParameterValueProperty->SetOnChildPropertyValueChangedWithData(OnPropertyValueChangedDelegate);

	FSoftObjectPath BaseParameterCollectionPath = BaseParameterCollection;

	// Create the value widget first for GetDesiredWidth
	TSharedRef<SObjectPropertyEntryBox> ObjectPropertyEntryBox = SNew(SObjectPropertyEntryBox)
		.PropertyHandle(ParameterValueProperty)
		.AllowedClass(UMaterialParameterCollection::StaticClass())
		.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
		.OnShouldFilterAsset_Lambda([BaseParameterCollectionPath](const FAssetData& AssetData)
			{
				// Allow the base MPC itself
				if (AssetData.GetSoftObjectPath() == BaseParameterCollectionPath)
				{
					return false;
				}

				IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

				// Traverse up the hierarchy of UMaterialParameterCollection::Base until we find the base material's MPC
				FSoftObjectPath ParentAssetPath = FSoftObjectPath(AssetData.GetTagValueRef<FString>(UMaterialParameterCollection::GetBaseParameterCollectionMemberName()));
				while (!ParentAssetPath.IsNull())
				{
					if (ParentAssetPath == BaseParameterCollectionPath)
					{
						return false;
					}

					FAssetData ParentAssetData = AssetRegistry.GetAssetByObjectPath(ParentAssetPath);
					ParentAssetPath = FSoftObjectPath(ParentAssetData.GetTagValueRef<FString>(UMaterialParameterCollection::GetBaseParameterCollectionMemberName()));
				}

				// No matching MPC in hierarchy, filter this MPC out
				return true;
			});

	float MinDesiredWidth, MaxDesiredWidth;
	ObjectPropertyEntryBox->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);

	// Set up the custom widget
	PropertyRow.CustomWidget()
		.NameContent()
		[
			ParameterValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(MinDesiredWidth)
		.MaxDesiredWidth(MaxDesiredWidth)
		[
			ObjectPropertyEntryBox
		];
}

bool FMaterialPropertyHelpers::IsOverriddenExpression(UDEditorParameterValue* Parameter)
{
	return Parameter && Parameter->bOverride != 0;
}

ECheckBoxState FMaterialPropertyHelpers::IsOverriddenExpressionCheckbox(UDEditorParameterValue* Parameter)
{
	return IsOverriddenExpression(Parameter) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FMaterialPropertyHelpers::UsesCustomPrimitiveData(UDEditorParameterValue* Parameter)
{
	if (UDEditorScalarParameterValue* ScalarParameter = Cast<UDEditorScalarParameterValue>(Parameter))
	{
		return ScalarParameter->bUseCustomPrimitiveData;
	}
	else if (UDEditorVectorParameterValue* VectorParameter = Cast<UDEditorVectorParameterValue>(Parameter))
	{
		return VectorParameter->bUseCustomPrimitiveData;
	}

	return false;
}

void FMaterialPropertyHelpers::OnOverrideParameter(bool NewValue, class UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance)
{
	// If the material instance disallows the creation of new shader permutations, prevent overriding the static parameter.
	if (NewValue && Parameter->IsStaticParameter() && MaterialEditorInstance->SourceInstance->bDisallowStaticParameterPermutations)
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "OverrideParameter", "Override Parameter" ) );
	Parameter->Modify();
	Parameter->bOverride = NewValue;

	// Fire off a dummy event to the material editor instance, so it knows to update the material, then refresh the viewports.
	FPropertyChangedEvent OverrideEvent(NULL);
	MaterialEditorInstance->PostEditChangeProperty( OverrideEvent );
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	FEditorSupportDelegates::UpdateUI.Broadcast();
}

FText FMaterialPropertyHelpers::GetParameterExpressionDescription(UDEditorParameterValue* Parameter, UObject* MaterialEditorInstance)
{
	return FText::FromString(Parameter->Description);
}

FText FMaterialPropertyHelpers::GetParameterTooltip(UDEditorParameterValue* Parameter, UObject* MaterialEditorInstance)
{
	return GetParameterTooltip(Parameter, static_cast<UMaterialEditorInstanceConstant*>(MaterialEditorInstance));
}

FText FMaterialPropertyHelpers::GetParameterTooltip(UDEditorParameterValue* Parameter, UMaterial* Material)
{
	const FMaterialCachedExpressionData* CachedExpressionData = Material ? &Material->GetCachedExpressionData() : nullptr;

	FText AssetPath;
	if (CachedExpressionData)
	{
		const TSharedPtr<FMaterialCachedExpressionEditorOnlyData>& EditorOnlyData = CachedExpressionData->EditorOnlyData;
		if (EditorOnlyData)
		{
			if (const FParameterAssetPathsEntry* Entry = EditorOnlyData->AssetPathsMap.Find(Parameter->ParameterInfo.Name))
			{
				AssetPath = FText::FromString(Entry->ToString());
			}
		}
	}

	FText TooltipText;
	if (!Parameter->Description.IsEmpty())
	{
		TooltipText = FText::Format(LOCTEXT("ParameterInfoDescAndLocation", "{0} \nFound in: {1}"), FText::FromString(Parameter->Description), AssetPath);
	}
	else
	{
		TooltipText = FText::Format(LOCTEXT("ParameterInfoLocationOnly", "Found in: {0}"), AssetPath);
	}

	return TooltipText;
}

FText FMaterialPropertyHelpers::GetParameterTooltip(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance)
{
	// If the material instance disallows the creation of new shader permutations, prevent overriding the static parameter.
	if (Parameter->IsStaticParameter() && MaterialEditorInstance->SourceInstance && MaterialEditorInstance->SourceInstance->bDisallowStaticParameterPermutations)
	{
		return FText::FromString(TEXT("This material instance parent restricts the creation of new shader permutations. Overriding this parameter would result in the generation of additional shader permutations."));
	}
	
	// Otherwise, fallback to the generic material parameter tooltip 
	return GetParameterTooltip(Parameter, MaterialEditorInstance->Parent ? MaterialEditorInstance->Parent->GetMaterial() : nullptr);
}

FResetToDefaultOverride FMaterialPropertyHelpers::CreateResetToDefaultOverride(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, TSharedPtr<IPropertyHandle> ParameterValueProperty)
{
	UDEditorScalarParameterValue* ScalarParam = Cast<UDEditorScalarParameterValue>(Parameter);
	if (ScalarParam && ScalarParam->AtlasData.bIsUsedAsAtlasPosition)
	{
		TAttribute<bool> IsResetVisible = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(&FMaterialPropertyHelpers::ShouldShowResetToDefault, Parameter, MaterialEditorInstance));
		FSimpleDelegate ResetHandler = FSimpleDelegate::CreateStatic(&FMaterialPropertyHelpers::ResetCurveToDefault, Parameter, MaterialEditorInstance);
		return FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
	}

	TAttribute<bool> IsResetVisible = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(&FMaterialPropertyHelpers::ShouldShowResetToDefault, Parameter, MaterialEditorInstance));
	FSimpleDelegate ResetHandler = FSimpleDelegate::CreateStatic(&FMaterialPropertyHelpers::ResetToDefault, Parameter, MaterialEditorInstance, ParameterValueProperty);
	return FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
}

void FMaterialPropertyHelpers::ResetToDefault(class UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, TSharedPtr<IPropertyHandle> ParameterValueProperty)
{
	const FScopedTransaction Transaction( LOCTEXT( "ResetToDefault", "Reset To Default" ) );
	Parameter->Modify();

	FMaterialParameterMetadata EditorValue;
	if (Parameter->GetValue(EditorValue))
	{
		FMaterialParameterMetadata DefaultValue;
		if (MaterialEditorInstance->SourceInstance->GetParameterDefaultValue(EditorValue.Value.Type, Parameter->ParameterInfo, DefaultValue))
		{
			ParameterValueProperty->NotifyPreChange();
			Parameter->SetValue(DefaultValue.Value);
			ParameterValueProperty->NotifyPostChange(EPropertyChangeType::ValueSet|EPropertyChangeType::ResetToDefault);

			MaterialEditorInstance->CopyToSourceInstance();
		}
	}
}

void FMaterialPropertyHelpers::ResetLayerAssetToDefault(UDEditorParameterValue* InParameter, TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 Index, UMaterialEditorInstanceConstant* MaterialEditorInstance)
{
	const FScopedTransaction Transaction(LOCTEXT("ResetToDefault", "Reset To Default"));
	InParameter->Modify();

	UDEditorMaterialLayersParameterValue* LayersParam = Cast<UDEditorMaterialLayersParameterValue>(InParameter);
	if (LayersParam)
	{
		FMaterialLayersFunctions LayersValue;
		if (MaterialEditorInstance->Parent->GetMaterialLayers(LayersValue))
		{
			FMaterialLayersFunctions StoredValue = LayersParam->ParameterValue;

			if (InAssociation == EMaterialParameterAssociation::BlendParameter)
			{
				if (Index < LayersValue.Blends.Num())
				{
					StoredValue.Blends[Index] = LayersValue.Blends[Index];
				}
				else
				{
					StoredValue.Blends[Index] = nullptr;
					MaterialEditorInstance->StoredBlendPreviews[Index] = nullptr;
				}
			}
			else if (InAssociation == EMaterialParameterAssociation::LayerParameter)
			{
				if (Index < LayersValue.Layers.Num())
				{
					StoredValue.Layers[Index] = LayersValue.Layers[Index];
				}
				else
				{
					StoredValue.Layers[Index] = nullptr;
					MaterialEditorInstance->StoredLayerPreviews[Index] = nullptr;
				}
			}
			LayersParam->ParameterValue = StoredValue;
		}
	}

	FPropertyChangedEvent OverrideEvent(NULL);
	MaterialEditorInstance->PostEditChangeProperty(OverrideEvent);
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	FEditorSupportDelegates::UpdateUI.Broadcast();
	
}

bool FMaterialPropertyHelpers::ShouldLayerAssetShowResetToDefault(TSharedPtr<FSortedParamData> InParameterData, UMaterialInstanceConstant* InMaterialInstance)
{
	if (!InParameterData->Parameter)
	{
		return false;
	}

	TArray<class UMaterialFunctionInterface*> StoredAssets;
	TArray<class UMaterialFunctionInterface*> ParentAssets;

	int32 Index = InParameterData->ParameterInfo.Index;
	UDEditorMaterialLayersParameterValue* LayersParam = Cast<UDEditorMaterialLayersParameterValue>(InParameterData->Parameter);
	if (LayersParam)
	{
		FMaterialLayersFunctions LayersValue;
		UMaterialInterface* Parent = InMaterialInstance->Parent;
		if (Parent && Parent->GetMaterialLayers(LayersValue))
		{
			FMaterialLayersFunctions StoredValue = LayersParam->ParameterValue;

			if (InParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
			{
				StoredAssets = StoredValue.Blends;
				ParentAssets = LayersValue.Blends;

			}
			else if (InParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
			{
				StoredAssets = StoredValue.Layers;
				ParentAssets = LayersValue.Layers;
			}

			// Compare to the parent MaterialFunctionInterface array
			if (Index < ParentAssets.Num())
			{
				if (StoredAssets[Index] == ParentAssets[Index])
				{
					return false;
				}
				else
				{
					return true;
				}
			}
			else if (StoredAssets[Index] != nullptr)
			{
				return true;
			}
		}
	}

	return false;
}

bool FMaterialPropertyHelpers::ShouldShowResetToDefault(UDEditorParameterValue* InParameter, UMaterialEditorInstanceConstant* MaterialEditorInstance)
{
	//const FMaterialParameterInfo& ParameterInfo = InParameter->ParameterInfo;

	FMaterialParameterMetadata EditorValue;
	if (InParameter->GetValue(EditorValue))
	{
		FMaterialParameterMetadata SourceValue;
		MaterialEditorInstance->SourceInstance->GetParameterDefaultValue(EditorValue.Value.Type, InParameter->ParameterInfo, SourceValue);
		if (EditorValue.Value != SourceValue.Value)
		{
			return true;
		}
	}

	return false;
}

FEditorParameterGroup&  FMaterialPropertyHelpers::GetParameterGroup(UMaterial* InMaterial, FName& ParameterGroup, TArray<struct FEditorParameterGroup>& ParameterGroups)
{
	if (ParameterGroup == TEXT(""))
	{
		ParameterGroup = TEXT("None");
	}
	for (int32 i = 0; i < ParameterGroups.Num(); i++)
	{
		FEditorParameterGroup& Group = ParameterGroups[i];
		if (Group.GroupName == ParameterGroup)
		{
			return Group;
		}
	}
	int32 ind = ParameterGroups.AddZeroed(1);
	FEditorParameterGroup& Group = ParameterGroups[ind];
	Group.GroupName = ParameterGroup;
	UMaterial* ParentMaterial = InMaterial;
	int32 NewSortPriority;
	if (ParentMaterial->GetGroupSortPriority(ParameterGroup.ToString(), NewSortPriority))
	{
		Group.GroupSortPriority = NewSortPriority;
	}
	else
	{
		Group.GroupSortPriority = 0;
	}
	Group.GroupAssociation = EMaterialParameterAssociation::GlobalParameter;

	return Group;
}

void FMaterialPropertyHelpers::GetVectorChannelMaskComboBoxStrings(TArray<TSharedPtr<FString>>& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems)
{
	const UEnum* ChannelEnum = StaticEnum<EChannelMaskParameterColor::Type>();
	check(ChannelEnum);

	// Add RGBA string options (Note: Exclude the "::Max" entry)
	const int32 NumEnums = ChannelEnum->NumEnums() - 1;
	for (int32 Entry = 0; Entry < NumEnums; ++Entry)
	{
		FText EnumName = ChannelEnum->GetDisplayNameTextByIndex(Entry);

		OutComboBoxStrings.Add(MakeShared<FString>(EnumName.ToString()));
		OutToolTips.Add(SNew(SToolTip).Text(EnumName));
		OutRestrictedItems.Add(false);
	}
}

FString FMaterialPropertyHelpers::GetVectorChannelMaskValue(UDEditorParameterValue* InParameter)
{
	UDEditorVectorParameterValue* VectorParam = Cast<UDEditorVectorParameterValue>(InParameter);
	check(VectorParam && VectorParam->bIsUsedAsChannelMask);

	const UEnum* ChannelEnum = StaticEnum<EChannelMaskParameterColor::Type>();
	check(ChannelEnum);

	// Convert from vector to RGBA string
	int64 ChannelType = 0;

	if (VectorParam->ParameterValue.R > 0.0f)
	{
		ChannelType = EChannelMaskParameterColor::Red;
	}
	else if (VectorParam->ParameterValue.G > 0.0f)
	{
		ChannelType = EChannelMaskParameterColor::Green;
	}
	else if (VectorParam->ParameterValue.B > 0.0f)
	{
		ChannelType = EChannelMaskParameterColor::Blue;
	}
	else
	{
		ChannelType = EChannelMaskParameterColor::Alpha;
	}

	return ChannelEnum->GetDisplayNameTextByValue(ChannelType).ToString();
}

void FMaterialPropertyHelpers::SetVectorChannelMaskValue(const FString& StringValue, TSharedPtr<IPropertyHandle> PropertyHandle, UDEditorParameterValue* InParameter, UObject* MaterialEditorInstance)
{
	UDEditorVectorParameterValue* VectorParam = Cast<UDEditorVectorParameterValue>(InParameter);
	check(VectorParam && VectorParam->bIsUsedAsChannelMask);

	const UEnum* ChannelEnum = StaticEnum<EChannelMaskParameterColor::Type>();
	check(ChannelEnum);

	// Convert from RGBA string to vector
	int64 ChannelValue = ChannelEnum->GetValueByNameString(StringValue);
	FLinearColor NewValue;

	switch (ChannelValue)
	{
	case EChannelMaskParameterColor::Red:
		NewValue = FLinearColor(1.0f, 0.0f, 0.0f, 0.0f); break;
	case EChannelMaskParameterColor::Green:
		NewValue = FLinearColor(0.0f, 1.0f, 0.0f, 0.0f); break;
	case EChannelMaskParameterColor::Blue:
		NewValue = FLinearColor(0.0f, 0.0f, 1.0f, 0.0f); break;
	default:
		NewValue = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	}

	// If changed, propagate the update
	if (VectorParam->ParameterValue != NewValue)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetVectorChannelMaskValue", "Set Vector Channel Mask Value"));
		VectorParam->Modify();

		PropertyHandle->NotifyPreChange();
		VectorParam->ParameterValue = NewValue;

		UMaterialEditorInstanceConstant* MaterialInstanceEditor = Cast<UMaterialEditorInstanceConstant>(MaterialEditorInstance);
		if (MaterialInstanceEditor)
		{
			MaterialInstanceEditor->CopyToSourceInstance();
		}

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

TArray<UFactory*> FMaterialPropertyHelpers::GetAssetFactories(EMaterialParameterAssociation AssetType)
{
	TArray<UFactory*> NewAssetFactories;
	switch (AssetType)
	{
	case LayerParameter:
	{
	//	NewAssetFactories.Add(NewObject<UMaterialFunctionMaterialLayerFactory>());
		break;
	}
	case BlendParameter:
	{
	//	NewAssetFactories.Add(NewObject<UMaterialFunctionMaterialLayerBlendFactory>());
		break;
	}
	case GlobalParameter:
		break;
	default:
		break;
	}
	
	return NewAssetFactories;
}


TSharedRef<SWidget> FMaterialPropertyHelpers::MakeStackReorderHandle(TSharedPtr<IDraggableItem> InOwningStack)
{
	TSharedRef<SLayerHandle> Handle = SNew(SLayerHandle)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.Padding(5.0f, 0.0f)
		[
			SNew(SImage)
			.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
		]
		]
	.OwningStack(InOwningStack);
	return Handle;
}

bool FMaterialPropertyHelpers::OnShouldSetCurveAsset(const FAssetData& AssetData, TSoftObjectPtr<class UCurveLinearColorAtlas> InAtlas)
{
	UCurveLinearColorAtlas* Atlas = Cast<UCurveLinearColorAtlas>(InAtlas.Get());
	if (!Atlas)
	{
		return false;
	}

	for (UCurveLinearColor* GradientCurve : Atlas->GradientCurves)
	{
		if (!GradientCurve || !GradientCurve->GetOutermost())
		{
			continue;
		}

		if (GradientCurve->GetOutermost()->GetPathName() == AssetData.PackageName.ToString())
		{
			return true;
		}
	}

	return false;
}

bool FMaterialPropertyHelpers::OnShouldFilterCurveAsset(const FAssetData& AssetData, TSoftObjectPtr<class UCurveLinearColorAtlas> InAtlas)
{
	return !OnShouldSetCurveAsset(AssetData, InAtlas);
}

void FMaterialPropertyHelpers::SetPositionFromCurveAsset(const FAssetData& AssetData, TSoftObjectPtr<class UCurveLinearColorAtlas> InAtlas, UDEditorScalarParameterValue* InParameter, TSharedPtr<IPropertyHandle> PropertyHandle, UObject* MaterialEditorInstance)
{
	UCurveLinearColorAtlas* Atlas = Cast<UCurveLinearColorAtlas>(InAtlas.Get());
	UCurveLinearColor* Curve = Cast<UCurveLinearColor>(AssetData.GetAsset());
	if (!Atlas || !Curve)
	{
		return;
	}
	int32 Index = Atlas->GradientCurves.Find(Curve);
	if (Index == INDEX_NONE)
	{
		return;
	}

	float NewValue = (float)Index;

	// If changed, propagate the update
	if (InParameter->ParameterValue != NewValue)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetScalarAtlasPositionValue", "Set Scalar Atlas Position Value"));
		InParameter->Modify();

		InParameter->AtlasData.Curve = TSoftObjectPtr<UCurveLinearColor>(FSoftObjectPath(Curve->GetPathName()));
		InParameter->ParameterValue = NewValue;
		UMaterialEditorInstanceConstant* MaterialInstanceEditor = Cast<UMaterialEditorInstanceConstant>(MaterialEditorInstance);
		if (MaterialInstanceEditor)
		{
			MaterialInstanceEditor->CopyToSourceInstance();
		}
	}
}

void FMaterialPropertyHelpers::ResetCurveToDefault(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance)
{
	const FScopedTransaction Transaction(LOCTEXT("ResetToDefault", "Reset To Default"));
	Parameter->Modify();
	const FMaterialParameterInfo& ParameterInfo = Parameter->ParameterInfo;

	UDEditorScalarParameterValue* ScalarParam = Cast<UDEditorScalarParameterValue>(Parameter);

	if (ScalarParam)
	{
		float OutValue;
		if (MaterialEditorInstance->SourceInstance->GetScalarParameterDefaultValue(ParameterInfo, OutValue))
		{
			ScalarParam->ParameterValue = OutValue;
			
			// Purge cached values, which will cause non-default values for the atlas data to be returned by IsScalarParameterUsedAsAtlasPosition
			MaterialEditorInstance->SourceInstance->ClearParameterValuesEditorOnly();

			// Update the atlas data from default values
			bool TempBool;
			MaterialEditorInstance->SourceInstance->IsScalarParameterUsedAsAtlasPosition(ParameterInfo, TempBool, ScalarParam->AtlasData.Curve, ScalarParam->AtlasData.Atlas);
			MaterialEditorInstance->CopyToSourceInstance();
		}
	}
}

TSharedRef<SWidget> FMaterialPropertyHelpers::CreateScalarEnumerationParameterValueWidget(TSoftObjectPtr<UObject> const& SoftEnumeration, TSharedPtr<IPropertyHandle> ParameterValueProperty)
{
	// Load the enumeration object before use.
	// We support two object types: UEnum and IMaterialEnumerationProvider.
	UObject* Enumeration = SoftEnumeration.LoadSynchronous();
	TWeakObjectPtr<UObject> WeakEnumeration(Enumeration);

	FOnGetPropertyComboBoxStrings GetStrings = FOnGetPropertyComboBoxStrings::CreateLambda([WeakEnumeration](TArray<TSharedPtr<FString>>& OutStrings, TArray<TSharedPtr<SToolTip>>&, TArray<bool>&)
	{
		UObject* Enumeration = WeakEnumeration.Get();
		if (UEnum* Enum = Cast<UEnum>(Enumeration))
		{
			// UUserDefinedEnum always add a _MAX entry to the enum. We want to ignore that for populating the combo box.
			// C++ UEnum usually use UMETA(Hidden) to hide any _MAX entry (and other entries too).
			const bool bUserDefinedEnum = Enum->IsA<UUserDefinedEnum>();
			const int32 NumEnums = bUserDefinedEnum ? Enum->NumEnums() - 1 : Enum->NumEnums();
			for (int32 EnumIndex = 0; EnumIndex < NumEnums; ++EnumIndex)
			{
				if (!Enum->HasMetaData(TEXT("Hidden"), EnumIndex))
				{
					OutStrings.Emplace(MakeShared<FString>(Enum->GetDisplayNameTextByIndex(EnumIndex).ToString()));
				}
			}
		}
		else if (IMaterialEnumerationProvider* EnumProvider = Cast<IMaterialEnumerationProvider>(Enumeration))
		{
			EnumProvider->ForEachEntry([&OutStrings](FName InEntryName, int32 InEntryValue)
			{
				OutStrings.Emplace(MakeShared<FString>(InEntryName.ToString()));
			});
		}
	});

	FOnGetPropertyComboBoxValue GetValue = FOnGetPropertyComboBoxValue::CreateLambda([WeakEnumeration, ParameterValueProperty]()
	{
		FString EntryName = TEXT("None");

		float Value = 0.f;
		ParameterValueProperty->GetValue(Value);

		UObject* Enumeration = WeakEnumeration.Get();
		if (UEnum* Enum = Cast<UEnum>(Enumeration))
		{
			EntryName = Enum->GetDisplayNameTextByValue((int64)Value).ToString();
		}
		else if (IMaterialEnumerationProvider* EnumProvider = Cast<IMaterialEnumerationProvider>(Enumeration))
		{
			EnumProvider->ForEachEntry([Value, &EntryName](FName InEntryName, int32 InEntryValue)
			{
				if (InEntryValue == (int32)Value)
				{
					EntryName = InEntryName.ToString();
				}
			});
		}
		
		return EntryName;
	});

	FOnPropertyComboBoxValueSelected SetValue = FOnPropertyComboBoxValueSelected::CreateLambda([WeakEnumeration, ParameterValueProperty](const FString& StringValue)
	{
		UObject* Enumeration = WeakEnumeration.Get();
		if (UEnum* Enum = Cast<UEnum>(Enumeration))
		{
			for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums(); ++EnumIndex)
			{
				if (Enum->GetDisplayNameTextByIndex(EnumIndex).ToString() == StringValue)
				{
					const float Value = Enum->GetValueByIndex(EnumIndex);
					ParameterValueProperty->SetValue(Value);
					break;
				}
			}
		}
		else if (IMaterialEnumerationProvider* EnumProvider = Cast<IMaterialEnumerationProvider>(Enumeration))
		{
			const float Value = EnumProvider->GetValueOrDefault(*StringValue);
			ParameterValueProperty->SetValue(Value);
		}
	});

	return PropertyCustomizationHelpers::MakePropertyComboBox(ParameterValueProperty, GetStrings, GetValue, SetValue);
}

#undef LOCTEXT_NAMESPACE

