// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialStats.h"
#include "MaterialStatsGrid.h"
#include "SMaterialEditorStatsWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Text/HLSLSyntaxHighlighterMarshaller.h"
#include "HAL/PlatformApplicationMisc.h"
#include "MaterialEditorActions.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInsights.h"
#include "IMaterialEditor.h"
#include "MaterialEditorSettings.h"
#include "ShaderCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"
#include "Interfaces/IShaderFormat.h"

#define LOCTEXT_NAMESPACE "MaterialStats"

const FName FMaterialStats::StatsTabId(TEXT("MaterialStats_Grid"));
const FName FMaterialStats::OldStatsTabId(TEXT("OldMaterialStats_Grid"));
const FName FMaterialStats::HLSLCodeTabId(TEXT("MaterialStats_HLSLCode"));

/***********************************************************************************************************************/
/*begin FShaderPlatformSettings functions*/

FShaderPlatformSettings::FShaderPlatformSettings(
	const EPlatformCategoryType _PlatformType,
	const EShaderPlatform _ShaderPlatformID,
	const FName _Name,
	const bool _bAllowPresenceInGrid,
	const bool _bAllowCodeView,
	const FString& _Description,
	const bool bAlwaysOn
	)
	: PlatformType(_PlatformType)
	, PlatformShaderID(_ShaderPlatformID)
	, PlatformName(_Name)
	, PlatformDescription(_Description)
	, bAlwaysOn(bAlwaysOn)
	, bAllowCodeView(_bAllowCodeView)
	, bAllowPresenceInGrid(_bAllowPresenceInGrid)
{
	PlatformNameID = *FMaterialStatsUtils::ShaderPlatformTypeName(PlatformShaderID);
}

void FShaderPlatformSettings::ClearResources()
{
	TArray<TRefCountPtr<FMaterial>> MaterialsToDeleteOnRenderThread;

	// free material resources
	for (int32 i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		for (int32 InstanceIndex = 0; InstanceIndex < PlatformData[i].Instances.Num(); ++InstanceIndex)
		{
			auto& Instance = PlatformData[i].Instances[InstanceIndex];
			FMaterialResourceStats* Resource = Instance.MaterialResourcesStats;
			if (Resource != nullptr)
			{
				if (Resource->PrepareDestroy_GameThread())
				{
					MaterialsToDeleteOnRenderThread.Add(Resource);
				}
				else
				{
					delete Resource;
				}
			}

			Instance.ArrShaderEntries.Empty();
		}

		PlatformData[i].Instances.Empty();
	}

	FMaterial::DeleteMaterialsOnRenderThread(MaterialsToDeleteOnRenderThread);
}

FText FShaderPlatformSettings::GetSelectedShaderViewComboText(EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) const
{
	const auto& Instance = PlatformData[QualityLevel].Instances[InstanceIndex];
	if (Instance.ArrShaderEntries.Num() == 0)
	{
		return FText::FromString(TEXT("-Compiling-Shaders-"));
	}

	return FText::FromString(Instance.ComboBoxSelectedEntry.ShaderName);
}

void FShaderPlatformSettings::OnShaderViewComboSelectionChanged(TSharedPtr<FMaterialShaderEntry> Item, EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex)
{
	if (Item.IsValid())
	{
		auto& Instance = PlatformData[QualityType].Instances[InstanceIndex];
		Instance.ComboBoxSelectedEntry = *Item.Get();
		Instance.bUpdateShaderCode = true;
	}
}

FText FShaderPlatformSettings::GetShaderCodeText(const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex)
{
	if (PlatformData[QualityType].Instances.IsEmpty())
	{
		return FText::GetEmpty();
	}

	check(InstanceIndex >= 0 && InstanceIndex < PlatformData[QualityType].Instances.Num());
	auto& Instance = PlatformData[QualityType].Instances[InstanceIndex];
	// if there were no change to the material return the cached shader code
	if (!Instance.bUpdateShaderCode)
	{
		return Instance.ShaderCode;
	}

	Instance.ShaderCode = LOCTEXT("ShaderCodeMsg", "Shader code compiling or not available!");

	FMaterialResource *Resource = Instance.MaterialResourcesStats;
	const FMaterialShaderMap* MaterialShaderMap = Resource->GetGameThreadShaderMap();

	const bool bCompilationFinished = Resource->IsCompilationFinished() && (MaterialShaderMap != nullptr);

	// check if shader compilation is done and extract shader code
	if (bCompilationFinished)
	{
		TMap<FShaderId, TShaderRef<FShader>> ShaderMap;
		MaterialShaderMap->GetShaderList(ShaderMap);

		const FShaderId& ShaderId = Instance.ComboBoxSelectedEntry.ShaderId;

		const auto Entry = ShaderMap.Find(ShaderId);
		if (Entry != nullptr)
		{
			const TShaderRef<FShader>& Shader = *Entry;
			const FMemoryImageString* ShaderSource = MaterialShaderMap->GetShaderSource(Shader.GetVertexFactoryType(), Shader.GetType(), ShaderId.PermutationId);
			if (ShaderSource != nullptr)
			{
				Instance.bUpdateShaderCode = false;
				Instance.ShaderCode = FText::FromString(*ShaderSource);
			}
		}
	}

	return Instance.ShaderCode;
}

FText FShaderPlatformSettings::GetShaderCountText(const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex)
{
	// Preview material always gets translated, so use its parameters
	const UMaterialInterface* CurrentMaterial = MaterialInstance ? static_cast<UMaterialInterface*>(MaterialInstance) : static_cast<UMaterialInterface*>(Material);

	if (!CurrentMaterial)
	{
		return FText::GetEmpty();
	}

	const FMaterialResource* CurrentMatResource = CurrentMaterial->GetMaterialResource(PlatformShaderID, QualityType);

	if (!CurrentMatResource)
	{
		return FText::GetEmpty();
	}

	const FMaterialShaderMap* ShaderMap = CurrentMatResource->GetGameThreadShaderMap();

	if (!ShaderMap)
	{
		return FText::GetEmpty();
	}

	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(nullptr);

	// Dummy FMaterialResource to use Preview Material Shader Parameters without
	// FPreviewMaterial::ShouldCache, which eliminates shaders for editor performance
	FMaterialResource MatResShaderCount;
	MatResShaderCount.SetMaterial(nullptr, nullptr, CurrentMatResource->GetShaderPlatform(), CurrentMatResource->GetQualityLevel());

	TArray<FDebugShaderTypeInfo> ShaderInfos;
	const FCompilationShaderParameters MaterialParameters(CurrentMatResource, ShaderMap->GetCompilationOutput());
	MatResShaderCount.GetShaderTypes(LayoutParams, ShaderInfos, MaterialParameters);

	int TotalShadersForMaterial = 0;
	for (const FDebugShaderTypeInfo& ShaderInfo : ShaderInfos)
	{
		TotalShadersForMaterial += ShaderInfo.ShaderTypes.Num();

		for (const FDebugShaderPipelineInfo& PipelineInfo : ShaderInfo.Pipelines)
		{
			TotalShadersForMaterial += PipelineInfo.ShaderTypes.Num();
		}
	}

	return FText::Format(
		NSLOCTEXT("MaterialEditor", "NumShadersText", "Shaders: {0}"),
		TotalShadersForMaterial
	);
}

/** returns all shaders' stats concatenated together*/
FString FShaderPlatformSettings::GetShadersStats() const
{
	static FString ExtraLine("---------------------------------------------\n");
	FString ShadersStats;
	for (const FPlatformData& LevelData : PlatformData)
	{
		for (int32 ShaderType = 0; ShaderType < (int32)ERepresentativeShader::Num; ++ShaderType)
		{
			for (const FInstanceData& InstanceData : LevelData.Instances)
			{
				if (InstanceData.ShaderStatsInfo.ShaderInstructionCount.IsEmpty())
				{
					continue;
				}
				
				if (auto* Count = InstanceData.ShaderStatsInfo.ShaderInstructionCount.Find((ERepresentativeShader)ShaderType))
				{
					ShadersStats += FString("\n") + ExtraLine + FMaterialStatsUtils::RepresentativeShaderTypeToString((ERepresentativeShader)ShaderType);
					ShadersStats += Count->StrDescriptionLong;
				}
			}
		}
	}
	
	return ShadersStats;
}

void FShaderPlatformSettings::AllocateMaterialResources()
{
	ClearResources();

	const ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(PlatformShaderID);

	for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; QualityLevelIndex++)
	{
		auto& Data = PlatformData[QualityLevelIndex].Instances.AddDefaulted_GetRef();

		Data.MaterialResourcesStats = new FMaterialResourceStats();
		Data.MaterialResourcesStats->SetMaterial(Material, MaterialInstance, PlatformShaderID, (EMaterialQualityLevel::Type)QualityLevelIndex);

		Data.ShaderStatsInfo.Reset();

		Data.bNeedShaderRecompilation = true;

		for (int32 i = 0; i < DerivedMaterialInstances.Num(); ++i)
		{
			auto& Data2 = PlatformData[QualityLevelIndex].Instances.AddDefaulted_GetRef();

			Data2.MaterialResourcesStats = new FMaterialResourceStats();
			Data2.MaterialResourcesStats->SetMaterial(DerivedMaterialInstances[i]->GetBaseMaterial(), DerivedMaterialInstances[i], PlatformShaderID, (EMaterialQualityLevel::Type)QualityLevelIndex);

			Data2.ShaderStatsInfo.Reset();

			Data2.bNeedShaderRecompilation = true;
		}

		// ensure we start compiling shaders soon due to bNeedShaderRecompilation being set above
		PlatformData[QualityLevelIndex].LastTimeCompilationRequested = 0.0;
	}
}

void FShaderPlatformSettings::SetMaterial(UMaterial *InBaseMaterial, UMaterialInstance *InBaseMaterialInstance, const TArray<TObjectPtr<UMaterialInstance>>& InDerivedMaterialInstances)
{
	bool bReallocate = false;

	if (InBaseMaterial != nullptr && InBaseMaterialInstance == nullptr && Material != InBaseMaterial)
	{
		Material = InBaseMaterial;
		MaterialInstance = nullptr;
		bReallocate = true;
	}

	if (InBaseMaterial == nullptr && 
		InBaseMaterialInstance != nullptr && 
		(MaterialInstance != InBaseMaterialInstance || Material != InBaseMaterialInstance->GetMaterial())) // If the Material Instance changed or its parent changed
	{
		Material = InBaseMaterialInstance->GetMaterial();
		MaterialInstance = InBaseMaterialInstance;
		bReallocate = true;
	}

	if (DerivedMaterialInstances != InDerivedMaterialInstances)
	{
		// TODO avoid copy
		DerivedMaterialInstances = InDerivedMaterialInstances;
		bReallocate = true;
	}

	if (bReallocate)
	{
		AllocateMaterialResources();
	}
}

bool FShaderPlatformSettings::CheckShaders(bool bIgnoreCooldown)
{
	if (Material == nullptr)
	{
		return false;
	}

	bool bRefreshStatUI = false;

	// don't refresh stats too often
	const double kMinimumTimeBetweenCompilationsSeconds = 1.5;

	const double CurrentTime = FPlatformTime::Seconds();

	// check and triggers shader recompilation if needed
	for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; QualityLevelIndex++)
	{
		auto &Data = PlatformData[QualityLevelIndex];

		const bool bNeedsShaders = (bPresentInGrid && Data.bExtractStats) || Data.bExtractCode;
		const bool bCooledDown = bIgnoreCooldown || (CurrentTime - Data.LastTimeCompilationRequested) > kMinimumTimeBetweenCompilationsSeconds;

		bool bTriggeredCompilation = false;

		for (int32 InstanceIndex = 0; InstanceIndex < Data.Instances.Num(); ++InstanceIndex)
		{
			auto& Instance = Data.Instances[InstanceIndex]; 

			if (Instance.bNeedShaderRecompilation && bNeedsShaders)
			{
				bRefreshStatUI = true;

				// reset even if we don't immediately recompile to improve UI feedback time
				Instance.ShaderStatsInfo.Reset();

				if (bCooledDown)
				{
					Instance.MaterialResourcesStats->CancelCompilation();

					Material->UpdateCachedExpressionData();

					if (MaterialInstance != nullptr)
					{
						MaterialInstance->UpdateCachedData();
					}

					// Prepare the resource for compilation, but don't compile the completed shader map.
					Instance.MaterialResourcesStats->CacheShaders();
					Instance.bCompilingShaders = true;
					Instance.bUpdateShaderCode = true;
					Instance.bNeedShaderRecompilation = false;

					bTriggeredCompilation = true;
				}
			}
		}

		if (bTriggeredCompilation)
		{
			Data.LastTimeCompilationRequested = CurrentTime;
		}
	}

	return bRefreshStatUI;
}

bool FShaderPlatformSettings::Update()
{
	bool bRetValue = CheckShaders(false);

	// if a shader compilation has been requested check if completed and extract shader names needed by code viewer combo-box
	for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; ++QualityLevelIndex)
	{
		auto& Data = PlatformData[QualityLevelIndex];

		for (int32 InstanceIndex = 0; InstanceIndex < Data.Instances.Num(); ++InstanceIndex)
		{
			auto& Instance = Data.Instances[InstanceIndex];

			if (!Instance.bCompilingShaders)
			{
				continue;
			}

			FMaterialResource* Resource = Instance.MaterialResourcesStats;

			// if compilation is complete extract the list of compiled shader names
			const bool bCompilationFinished = Resource->IsCompilationFinished();
			if (!bCompilationFinished)
			{
				continue;
			}

			Instance.bCompilingShaders = false;
			Instance.bUpdateShaderCode = true;

			if (const FMaterialShaderMap * MaterialShaderMap = Resource->GetGameThreadShaderMap())
			{
				TMap<FShaderId, TShaderRef<FShader>> ShaderMap;
				MaterialShaderMap->GetShaderList(ShaderMap);

				Instance.ArrShaderEntries.Empty();
				for (const auto& Entry : ShaderMap)
				{
					FShaderType* EntryShader = Entry.Value.GetType();
					FVertexFactoryType* VertexFactory = Entry.Value.GetVertexFactoryType();
					
					FString ShaderName = FString::Printf(TEXT("%s/%s"), VertexFactory ? VertexFactory->GetName() : TEXT("NullVF"), EntryShader->GetName());
					FString ShaderFullName = ShaderName;

					if (FString PermutationName = EntryShader->GetPermutationIdString(Entry.Key.PermutationId, /*bFullNames = */false); !PermutationName.IsEmpty())
					{
						// Add a (shortened) permutation string to the shader name if applicable, so that we don't see multiple identical shader names when their permutation differs
						ShaderName += FString::Printf(TEXT("[%s]"), *PermutationName);
					}
					if (FString PermutationFullName = EntryShader->GetPermutationIdString(Entry.Key.PermutationId, /*bFullNames = */true); !PermutationFullName.IsEmpty())
					{
						// In the tooltip, we have room to display the full permutation name, which is already line-formatted 
						ShaderFullName += FString::Printf(TEXT("\n%s"), *PermutationFullName);
					}

					Instance.ArrShaderEntries.Add(MakeShareable(new FMaterialShaderEntry{ .ShaderId = Entry.Key, .ShaderName = ShaderName, .ShaderFullName = ShaderFullName }));
				}

				if (Instance.ArrShaderEntries.Num() > 0)
				{
					Instance.ArrShaderEntries.Sort([](const TSharedPtr<FMaterialShaderEntry>& First, const TSharedPtr<FMaterialShaderEntry>& Second)
						{
							return First->ShaderName < Second->ShaderName;
						});

					Instance.ComboBoxSelectedEntry = *Instance.ArrShaderEntries[0];
				}

				FMaterialStatsUtils::ExtractMatertialStatsInfo(PlatformShaderID, Instance.ShaderStatsInfo, Resource);

				Instance.bNeedToWarnAboutCompilationErrors = Resource->GetCompileErrors().Num() > 0;

				bRetValue = true;
			}

		}
	}

	return bRetValue;
}

bool FShaderPlatformSettings::CachePendingShaders()
{
	const bool bNeedsGridRefresh = CheckShaders(true);

	FString MaterialName;
	TArray<FMaterial*> MaterialsToCompile;
	for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; QualityLevelIndex++)
	{
		for (int32 InstanceIndex = 0; InstanceIndex < PlatformData[QualityLevelIndex].Instances.Num(); ++InstanceIndex)
		{
			auto& Instance = PlatformData[QualityLevelIndex].Instances[InstanceIndex]; 

			if (Instance.bCompilingShaders)
			{
				if (MaterialName.IsEmpty())
				{
					MaterialName = Instance.MaterialResourcesStats->GetFriendlyName(); 
				}

				MaterialsToCompile.Add(Instance.MaterialResourcesStats);
			}
		}
	}

	FMaterial::FinishCompilation(*MaterialName, MaterialsToCompile);

	return bNeedsGridRefresh;
}

/*end FShaderPlatformSettings functions*/
/***********************************************************************************************************************/

/***********************************************************************************************************************/
/*begin FMaterialStats functions*/

FMaterialStats::~FMaterialStats()
{
	SaveSettings();
}

void FMaterialStats::Initialize(IMaterialEditor* InMaterialEditor, const bool bShowMaterialInstancesMenu, const bool bAllowIgnoringCompilationErrors)
{
	MaterialEditor = InMaterialEditor;

	StatsGrid = MakeShareable(new FMaterialStatsGrid(AsShared()));

	BuildShaderPlatformDB(bAllowIgnoringCompilationErrors);

	LoadSettings(bAllowIgnoringCompilationErrors);

	StatsGrid->BuildGrid();

	GridStatsWidget = SNew(SMaterialEditorStatsWidget)
		.MaterialStatsWPtr(SharedThis(this))
		.ShowMaterialInstancesMenu(bShowMaterialInstancesMenu)
		.AllowIgnoringCompilationErrors(bAllowIgnoringCompilationErrors);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogOptions;
	// Show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false; //TODO - Provide custom filters? E.g. "Critical Errors" vs "Errors" needed for materials?
	LogOptions.bAllowClear = false;
	LogOptions.MaxPageCount = 1;
	OldStatsListing = MessageLogModule.CreateLogListing("MaterialEditorStats", LogOptions);
	OldStatsWidget = MessageLogModule.CreateLogListingWidget(OldStatsListing.ToSharedRef());

	auto ToolkitCommands = MaterialEditor->GetToolkitCommands();
	const FMaterialEditorCommands& Commands = FMaterialEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.TogglePlatformStats,
		FExecuteAction::CreateSP(this, &FMaterialStats::ToggleStats),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialStats::IsShowingStats));

	ToolkitCommands->MapAction(
		Commands.ToggleMaterialStats,
		FExecuteAction::CreateSP(this, &FMaterialStats::ToggleOldStats),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMaterialStats::IsShowingOldStats));
}

void FMaterialStats::LoadSettings(const bool bAllowIgnoringCompilationErrors)
{
	Options = NewObject<UMaterialStatsOptions>();

	for (const auto& PlatformEntry : ShaderPlatformStatsDB)
	{
		const EShaderPlatform PlatformID = PlatformEntry.Key;
		const bool bPresentInGrid = !!Options->bPlatformUsed[PlatformID];

		auto Platform = PlatformEntry.Value;
		Platform->SetPresentInGrid(bPresentInGrid);
	}

	for (int32 i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		const bool bUsed = !!Options->bMaterialQualityUsed[i];
		bArrStatsQualitySelector[(EMaterialQualityLevel::Type)i] = bUsed || bArrStatsQualitySelectorAlwaysOn[(EMaterialQualityLevel::Type)i];

		for (const auto& PlatformEntry : ShaderPlatformStatsDB)
		{
			TSharedPtr<FShaderPlatformSettings> SomePlatform = PlatformEntry.Value;
			if (SomePlatform.IsValid())
			{
				SomePlatform->SetExtractStatsFlag((EMaterialQualityLevel::Type)i, bUsed);
			}
		}
	}

	MaterialStatsDerivedMIOption = Options->MaterialStatsDerivedMIOption;

	// force compilation of derived instances if we're not allowed to ignore compilation errors
	if (!bAllowIgnoringCompilationErrors && (MaterialStatsDerivedMIOption == EMaterialStatsDerivedMIOption::Ignore || MaterialStatsDerivedMIOption == EMaterialStatsDerivedMIOption::InvalidOrMax))
	{
		MaterialStatsDerivedMIOption = EMaterialStatsDerivedMIOption::CompileOnly;
	}
}

void FMaterialStats::SaveSettings()
{
	for (const auto& PlatformEntry : ShaderPlatformStatsDB)
	{
		const EShaderPlatform PlatformID = PlatformEntry.Key;
		const auto Platform = PlatformEntry.Value;

		const bool bPresentInGrid = Platform->IsPresentInGrid();
		Options->bPlatformUsed[PlatformID] = bPresentInGrid ? 1 : 0;
	}

	for (int32 i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		const bool bQualityPresent = GetStatsQualityFlag((EMaterialQualityLevel::Type)i);

		Options->bMaterialQualityUsed[i] = bQualityPresent ? 1 : 0;
	}

	Options->MaterialStatsDerivedMIOption = MaterialStatsDerivedMIOption;

	Options->SaveConfig();
}

void FMaterialStats::SetShowStats(const bool bValue)
{
	bShowStats = bValue;

	// open/close stats tab
	DisplayStatsGrid(bShowStats);

	GetGridStatsWidget()->RequestRefresh();
}

void FMaterialStats::SetShowOldStats(const bool bValue)
{
	bShowOldStats = bValue;

	// open/close stats tab
	DisplayOldStats(bShowOldStats);
}

void FMaterialStats::ToggleStats()
{
	// Toggle the showing of material stats each time the user presses the show stats button
	SetShowStats(!bShowStats);
}

void FMaterialStats::ToggleOldStats()
{
	// Toggle the showing of material stats each time the user presses the show stats button
	SetShowOldStats(!bShowOldStats);
}

void FMaterialStats::DisplayOldStats(const bool bShow)
{
	if (bShow)
	{
		MaterialEditor->GetTabManager()->TryInvokeTab(OldStatsTabId);
	}
	else if (!bShowOldStats && OldStatsTab.IsValid())
	{
		OldStatsTab.Pin()->RequestCloseTab();
	}
}

void FMaterialStats::DisplayStatsGrid(const bool bShow)
{
	if (bShow)
	{
		MaterialEditor->GetTabManager()->TryInvokeTab(StatsTabId);
	}
	else if (!bShowStats && StatsTab.IsValid())
	{
		StatsTab.Pin()->RequestCloseTab();
	}
}

void FMaterialStats::RefreshStatsGrid()
{
	GetGridStatsWidget()->RequestRefresh();
}

void FMaterialStats::BuildShaderPlatformDB(const bool bAllowIgnoringCompilationErrors)
{
	// set High quality level as always on if we're not allowed to ignore compilation errors in UI
	// this will not allow to disable this particular quality level 
	if (!bAllowIgnoringCompilationErrors)
	{
		bArrStatsQualitySelectorAlwaysOn[EMaterialQualityLevel::High] = true;
		bArrStatsQualitySelector[EMaterialQualityLevel::High] = true;
	}

#if PLATFORM_WINDOWS
	// DirectX
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_PCD3D_SM5, TEXT("DirectX SM5"), true, TEXT("Desktop, DirectX, Shader Model 5"));
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_PCD3D_SM6, TEXT("DirectX SM6"), true, TEXT("Desktop, DirectX, Shader Model 6"), !bAllowIgnoringCompilationErrors);
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_PCD3D_ES3_1, TEXT("DirectX  Mobile"), true, TEXT("Desktop, DirectX, Mobile"), !bAllowIgnoringCompilationErrors);
#endif

	// Vulkan
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_VULKAN_SM5, TEXT("Vulkan SM5"), true, TEXT("Desktop, Vulkan, Shader Model 5"));
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_VULKAN_SM6, TEXT("Vulkan SM6"), true, TEXT("Desktop, Vulkan, Shader Model 6"));
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_VULKAN_PCES3_1, TEXT("Vulkan Mobile"), true, TEXT("Desktop, Vulkan, Mobile"));

	// Android
	AddShaderPlatform(EPlatformCategoryType::Android, SP_OPENGL_ES3_1_ANDROID, TEXT("Android GLES Mobile"), true, TEXT("Android, OpenGLES Mobile"));
	AddShaderPlatform(EPlatformCategoryType::Android, SP_VULKAN_ES3_1_ANDROID, TEXT("Android Vulkan Mobile"), true, TEXT("Android, Vulkan Mobile"));
	AddShaderPlatform(EPlatformCategoryType::Android, SP_VULKAN_SM5_ANDROID, TEXT("Android Vulkan SM5"), true, TEXT("Android, Vulkan SM5"));

#if PLATFORM_MAC
	// macOS
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_METAL_SM5, TEXT("Metal SM5"), true, TEXT("macOS, Metal, Shader Model 5"));
	AddShaderPlatform(EPlatformCategoryType::Desktop, SP_METAL_SM6, TEXT("Metal SM6"), true, TEXT("macOS, Metal, Shader Model 6"), !bAllowIgnoringCompilationErrors);
#endif

	// iOS
	AddShaderPlatform(EPlatformCategoryType::IOS, SP_METAL_ES3_1_IOS, TEXT("IOS Metal ES3_1"), true, TEXT("iOS, Metal, Mobile"));
	AddShaderPlatform(EPlatformCategoryType::IOS, SP_METAL_SM5_IOS, TEXT("IOS Metal SM5"), true, TEXT("iOS, Metal, Shader Model 5"));
	AddShaderPlatform(EPlatformCategoryType::IOS, SP_METAL_SM6_IOS, TEXT("IOS Metal SM6"), true, TEXT("iOS, Metal, Shader Model 6"));
	AddShaderPlatform(EPlatformCategoryType::IOS, SP_METAL_SIM, TEXT("Metal Simulator"), true, TEXT("iOS, Metal, Mobile"));

	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();

	// Add platform extensions
	for (int32 StaticPlatform = SP_StaticPlatform_First; StaticPlatform <= SP_StaticPlatform_Last; ++StaticPlatform)
	{
		EShaderPlatform ShaderPlatform = (EShaderPlatform)StaticPlatform;
		if (FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform))
		{
			bool bIsConsole = FDataDrivenShaderPlatformInfo::GetIsConsole(ShaderPlatform);

			const FName ShaderFormat = LegacyShaderPlatformToShaderFormat(ShaderPlatform);
			if (TPM.FindShaderFormat(ShaderFormat) != nullptr)
			{
				const FName PlatformName = ShaderPlatformToPlatformName(ShaderPlatform);
				AddShaderPlatform(bIsConsole ? EPlatformCategoryType::Console : EPlatformCategoryType::Desktop, ShaderPlatform, PlatformName, true, PlatformName.ToString());
			}
		}
	}

}

TSharedPtr<FShaderPlatformSettings> FMaterialStats::AddShaderPlatform(const EPlatformCategoryType PlatformType, const EShaderPlatform PlatformID, const FName PlatformName, const bool bAllowCodeView, const FString& Description, const bool bAlwaysOn)
{
	if (!FDataDrivenShaderPlatformInfo::IsValid(PlatformID))
	{
		return TSharedPtr<FShaderPlatformSettings>();
	}

	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	const IShaderFormat* ShaderFormat = TPM.FindShaderFormat(LegacyShaderPlatformToShaderFormat(PlatformID));
	const bool bAllowPresenceInGrid = ShaderFormat ? ShaderFormat->CanCompileBinaryShaders() : false;

	TSharedPtr<FShaderPlatformSettings> PlatformPtr = MakeShareable(new FShaderPlatformSettings(PlatformType, PlatformID, PlatformName, bAllowPresenceInGrid, bAllowCodeView, Description, bAlwaysOn));
	ShaderPlatformStatsDB.Add(PlatformID, PlatformPtr);

	auto& ArrayPlatforms = PlatformTypeDB.FindOrAdd(PlatformType);
	ArrayPlatforms.Add(PlatformPtr);

	for (int32 i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		PlatformPtr->SetExtractStatsFlag((EMaterialQualityLevel::Type)i, bAlwaysOn && bArrStatsQualitySelectorAlwaysOn[i]);
	}

	return PlatformPtr;
}

void FMaterialStats::SignalMaterialChanged()
{
	ExtractHLSLCode();

	for (auto Entry : ShaderPlatformStatsDB)
	{
		for (int32 i = 0; i < EMaterialQualityLevel::Num; ++i)
			Entry.Value->SetNeedShaderCompilation((EMaterialQualityLevel::Type)i, true, MaterialStatsDerivedMIOption == EMaterialStatsDerivedMIOption::CompileOnly);
	}
}

bool FMaterialStats::SwitchShaderPlatformUseStats(const EShaderPlatform PlatformID)
{
	bool bRetValue = false;

	auto* Item = ShaderPlatformStatsDB.Find(PlatformID);
	if (Item != nullptr)
	{
		bRetValue = (*Item)->FlipPresentInGrid();

		GetStatsGrid()->OnAddOrRemovePlatform(*Item);
		SaveSettings();
	}

	return bRetValue;
}

void FMaterialStats::SetStatusQualityFlag(const EMaterialQualityLevel::Type QualityLevel, const bool bValue)
{
	check(QualityLevel < EMaterialQualityLevel::Num);

	bArrStatsQualitySelector[QualityLevel] = bValue || bArrStatsQualitySelectorAlwaysOn[QualityLevel];

	for (const auto& PlatformEntry : ShaderPlatformStatsDB)
	{
		TSharedPtr<FShaderPlatformSettings> SomePlatform = PlatformEntry.Value;
		if (SomePlatform.IsValid())
		{
			SomePlatform->SetExtractStatsFlag(QualityLevel, bValue);
		}
	}

	SaveSettings();
}

bool FMaterialStats::SwitchStatsQualityFlag(EMaterialQualityLevel::Type Quality)
{
	check(Quality < EMaterialQualityLevel::Num);

	const bool bValue = !bArrStatsQualitySelector[Quality];
	SetStatusQualityFlag(Quality, bValue);

	return bValue;
}

void FMaterialStats::SetMaterialStatsDerivedMIOption(const EMaterialStatsDerivedMIOption value)
{
	MaterialStatsDerivedMIOption = value;

	if (MaterialEditor != nullptr)
	{
		MaterialEditor->RefreshStatsMaterials();
	}

	SaveSettings();
}

void FMaterialStats::SetShaderPlatformUseCodeView(const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type Quality, const bool bValue)
{
	auto* Item = ShaderPlatformStatsDB.Find(PlatformID);
	if (Item != nullptr)
	{
		(*Item)->SetCodeViewNeeded(Quality, bValue);
	}
}

FName FMaterialStats::GetPlatformName(const EShaderPlatform InEnumValue) const
{
	FName PlatformName = NAME_None;

	auto* Entry = ShaderPlatformStatsDB.Find(InEnumValue);
	if (Entry != nullptr && Entry->IsValid())
	{
		PlatformName = (*Entry)->GetPlatformName();
	}

	return PlatformName;
}

EShaderPlatform FMaterialStats::GetShaderPlatformID(const FName InName) const
{
	for (auto Entry : ShaderPlatformStatsDB)
	{
		if (Entry.Value.Get()->GetPlatformName() == InName)
		{
			return Entry.Key;
		}
	}

	return SP_NumPlatforms;
}

TSharedPtr<FShaderPlatformSettings> FMaterialStats::GetPlatformSettings(const EShaderPlatform PlatformID)
{
	auto* Entry = ShaderPlatformStatsDB.Find(PlatformID);
	if (Entry == nullptr)
	{
		return TSharedPtr<FShaderPlatformSettings>(nullptr);
	}

	return *Entry;
}

TSharedPtr<FShaderPlatformSettings> FMaterialStats::GetPlatformSettings(const FName PlatformName)
{
	const EShaderPlatform PlatformID = GetShaderPlatformID(PlatformName);

	return GetPlatformSettings(PlatformID);
}

FText FMaterialStats::GetShaderCodeText(const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex)
{
	auto* Entry = ShaderPlatformStatsDB.Find(PlatformID);
	if (Entry == nullptr)
	{
		return NSLOCTEXT("MaterialStats", "ShaderStatsNotAvailableMsg", "Shader code compiling or not available!");
	}

	return (*Entry)->GetShaderCodeText(QualityType, InstanceIndex);
}

FText FMaterialStats::GetShaderCountText(const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex)
{
	auto* Entry = ShaderPlatformStatsDB.Find(PlatformID);
	if (Entry == nullptr)
	{
		return NSLOCTEXT("MaterialStats", "ShaderCountNotAvailableMsg", "Shader count not available!");
	}

	return (*Entry)->GetShaderCountText(QualityType, InstanceIndex);
}

FString FMaterialStats::GetShadersStats() const
{
	static FString ExtraLine("=============================================\n");
	FString ShadersStats;
	int Index = 0; 
	for (const auto& MapEntry : ShaderPlatformStatsDB)
	{
		TSharedPtr<FShaderPlatformSettings> PlatformPtr = MapEntry.Value;
		FString PlatformShadersStats = PlatformPtr->GetShadersStats();
		if (PlatformShadersStats.Len())
		{
			ShadersStats += (Index ? FString("\n\n") : FString("")) + ExtraLine + ExtraLine + PlatformPtr->GetPlatformName().GetPlainNameString();
			ShadersStats += PlatformShadersStats;
			++Index;
		}
	}

	return ShadersStats;
}

void FMaterialStats::Update()
{
	const bool bNeedsUpdate = IsShowingStats() || IsCodeViewWindowActive();

	// don't update materials if we don't need to verify derived MI's
	// this is to preserve old behaviour of speeding up material editor by hiding platform stats
	if (!bNeedsUpdate && !GetProvideDerivedMIFlag())
	{
		return;
	}
	
	// otherwise always update material compilation info so we can check if shaders are compiling at material editor save
	for (const auto& Entry : ShaderPlatformStatsDB)
	{
		auto PlatformStats = Entry.Value;
		bNeedsGridRefresh |= PlatformStats->Update();
	}

	if (!bNeedsUpdate)
	{
		return;
	}

	if (bNeedsGridRefresh)
	{
		// in compile only mode the amount of columns might change if some of MI's fail to compile 
		if (GetMaterialStatsDerivedMIOption() == EMaterialStatsDerivedMIOption::CompileOnly)
		{
			auto GridPtr = GetGridStatsWidget();
			if (GridPtr.IsValid())
			{
				GridPtr->OnColumnNumChanged();
			}
		}
		else
		{
			GetStatsGrid()->OnShaderChanged();
		}

		// update any dependent tabs
		RefreshDependentTabs.Broadcast();

		bNeedsGridRefresh = false;
	}

	ComputeGridWarnings();
}

void FMaterialStats::CacheAndCompilePendingShaders()
{
	for (const auto& Entry : ShaderPlatformStatsDB)
	{
		auto PlatformStats = Entry.Value;
		bNeedsGridRefresh |= PlatformStats->CachePendingShaders();
	}
}

void FMaterialStats::SetMaterial(UMaterial *InMaterial, const TArray<TObjectPtr<UMaterialInstance>>& InDerivedMaterialInstances)
{
	if (MaterialInterface != InMaterial || DerivedMaterialInstances != InDerivedMaterialInstances)
	{
		MaterialInterface = InMaterial;
		DerivedMaterialInstances = InDerivedMaterialInstances;

		for (const auto& Entry : ShaderPlatformStatsDB)
		{
			auto& Platform = Entry.Value;
			if (Platform.IsValid())
			{
				Platform->SetMaterial(InMaterial, nullptr, DerivedMaterialInstances);
			}
		}

		auto GridPtr = GetGridStatsWidget();
		if (GridPtr.IsValid())
		{
			GridPtr->OnColumnNumChanged();
		}
	}
}

void FMaterialStats::SetMaterial(UMaterialInstance *InMaterialInstance)
{
	MaterialInterface = InMaterialInstance;
	DerivedMaterialInstances.Empty();

	for (const auto& Entry : ShaderPlatformStatsDB)
	{
		auto& Platform = Entry.Value;
		if (Platform.IsValid())
		{
			Platform->SetMaterial(nullptr, InMaterialInstance, DerivedMaterialInstances);
		}
	}

	auto GridPtr = GetGridStatsWidget();
	if (GridPtr.IsValid())
	{
		GridPtr->OnColumnNumChanged();
	}
}

bool FMaterialStats::AnyNewCompilationErrors(const int32 StartingFromInstanceIndex)
{
	bool bNewCompilationErrors = false;

	for (const auto& Entry : ShaderPlatformStatsDB)
	{
		auto& Platform = Entry.Value;
		if (Platform.IsValid())
		{
			for (int32 QualityLevel = 0; QualityLevel < EMaterialQualityLevel::Num; ++QualityLevel)
			{
				for (int32 InstanceIndex = StartingFromInstanceIndex; InstanceIndex < Platform->GetPlatformData(static_cast<EMaterialQualityLevel::Type>(QualityLevel)).Instances.Num(); ++InstanceIndex)
				{
					auto &Data = Platform->GetInstanceData(static_cast<EMaterialQualityLevel::Type>(QualityLevel), InstanceIndex);
					bNewCompilationErrors |= Data.bNeedToWarnAboutCompilationErrors;
					Data.bNeedToWarnAboutCompilationErrors = false;
				}
			}
		}
	}

	return bNewCompilationErrors;
}

// Generates a string of line numbers by counting the number of new-line characters in input source code.
static FText GenerateLineNoTextForCodeView(const FString& InSourceCode)
{
	FString LineNumberStr;
	int32 LineNo = 1;

	for (TCHAR Chr : InSourceCode)
	{
		if (Chr == TEXT('\n'))
		{
			LineNumberStr += FString::Printf(TEXT("% 6d\n"), LineNo);
			++LineNo;
		}
	}

	return FText::FromString(MoveTemp(LineNumberStr));
}

TSharedRef<SWidget> FMaterialStats::BuildShaderCodeWidget(FText TextLabel, TFunction<FText(void)>&& InShaderCodeCallback, TFunction<FReply(void)>&& InCopyCodeCallback, TAttribute<EVisibility> InVisibilityAttribute)
{
	TSharedRef<SVerticalBox> Box = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)
				// Copy Button
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 0.0f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(SButton)
								.Text(LOCTEXT("CopyHLSLButton", "Copy"))
								.ToolTipText(LOCTEXT("CopyHLSLButtonToolTip", "Copies all HLSL code to the clipboard."))
								.ContentPadding(3)
								.OnClicked_Lambda(InCopyCodeCallback)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 0.0f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
								.Text(TextLabel)
						]
				]
				// Separator
				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SNew(SSeparator)
				]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SScrollBox)
				+ SScrollBox::Slot().Padding(5)
				[
					SNew(SSplitter)
						.Orientation(EOrientation::Orient_Horizontal)
						+ SSplitter::Slot()
						.Resizable(false)
						.SizeRule(SSplitter::SizeToContent)
						[
							SNew(STextBlock)
								.Text_Lambda([InShaderCodeCallback]()
									{
										return GenerateLineNoTextForCodeView(InShaderCodeCallback().ToString());
									})
								.TextStyle(FAppStyle::Get(), "MessageLog")
								.ColorAndOpacity(FLinearColor::Gray)
						]
						+ SSplitter::Slot()
						[
							SNew(SMultiLineEditableText)
								.Text_Lambda([InShaderCodeCallback]() { return InShaderCodeCallback(); })
								.TextStyle(FAppStyle::Get(), "MessageLog")
								.IsReadOnly(true)
								.Marshaller(SyntaxHighlighter)
						]
				]
		];

	Box->SetVisibility(InVisibilityAttribute);

	return Box;
}

TSharedRef<class SDockTab> FMaterialStats::SpawnTab_HLSLCode(const class FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("HLSLCodeTitle", "HLSL Code"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)
				+SSplitter::Slot()
				[
					SNew(SSplitter)
					.Orientation(EOrientation::Orient_Horizontal)
					+SSplitter::Slot()
					[
						BuildShaderCodeWidget(LOCTEXT("CurrentTranslatorLabel", "New Translator HLSL Code"),
							[Code = &HLSLCode]() { return FText::FromString(*Code); },
							[Code = &HLSLCode]() {
								FPlatformApplicationMisc::ClipboardCopy(**Code);
								return FReply::Handled();
							},
							MakeAttributeLambda([Code = &HLSLCode]() { return (*Code).Len() == 0 ? EVisibility::Collapsed : EVisibility::Visible; })
						)
					]
					+SSplitter::Slot()
					[
						BuildShaderCodeWidget(LOCTEXT("PreviousTranslatorLabel", "HLSL Code"),
							[Code = &PreviousTranslatorHLSLCode]() { return FText::FromString(*Code); },
							[Code = &PreviousTranslatorHLSLCode]() {
								FPlatformApplicationMisc::ClipboardCopy(**Code);
								return FReply::Handled();
							},
							MakeAttributeLambda([Code = &PreviousTranslatorHLSLCode]() { return (*Code).Len() == 0 ? EVisibility::Collapsed : EVisibility::Visible; })
						)
					]
				]
				+SSplitter::Slot()
				[
					BuildShaderCodeWidget(LOCTEXT("MaterialIRLabel", "Intermediate Representation"),
						[Code = &MaterialIR]() { return FText::FromString(*Code); },
						[Code = &MaterialIR]() {
							FPlatformApplicationMisc::ClipboardCopy(**Code);
							return FReply::Handled();
						},
						MakeAttributeLambda([Code = &MaterialIR]() { return (*Code).Len() == 0 ? EVisibility::Collapsed : EVisibility::Visible; })
					)
				]
			]
		];

	HLSLTab = SpawnedTab;

	ExtractHLSLCode();

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialStats::SpawnTab_ShaderCode(const FSpawnTabArgs& Args, const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	SetShaderPlatformUseCodeView(PlatformID, QualityLevel, true);

	const FString PlatformName = GetPlatformName(PlatformID).ToString();
	const FString FullPlatformName = PlatformName + TEXT(" -- ") + FMaterialStatsUtils::MaterialQualityToString(QualityLevel);

	TSharedPtr<FShaderPlatformSettings> PlatformPtr = GetPlatformSettings(PlatformID);
	check(PlatformPtr.IsValid());

	const TArray<TSharedPtr<FMaterialShaderEntry>> *ShaderEntriesPtr = PlatformPtr->GetShaderEntries(QualityLevel, InstanceIndex);

	if (ShaderEntriesPtr == nullptr)
	{
		// SListView needs a valid pointer otherwise follow up SetItemsSource will fail.
		static TArray<TSharedPtr<FMaterialShaderEntry>> EmptyShaderEntriesArray = {};
		ShaderEntriesPtr = &EmptyShaderEntriesArray;
	}

	// Use an external scrollbar for the SListView, so that the vertical scrollbar remains visible all the time (and the SListView's horizontal one can still be used to peek at the full shader names)
	TSharedRef<SScrollBar> ExternalScrollbar = SNew(SScrollBar).Orientation(EOrientation::Orient_Vertical);

	ensure(ShaderEntriesPtr != nullptr);
	TSharedRef<SListView<TSharedPtr<FMaterialShaderEntry>>> ShaderListView = SNew(SListView<TSharedPtr<FMaterialShaderEntry>>)
		.ListItemsSource(ShaderEntriesPtr)
		.SelectionMode(ESelectionMode::Single)
		.ExternalScrollbar(ExternalScrollbar)
		.OnGenerateRow_Lambda([](TSharedPtr<FMaterialShaderEntry> Value, const TSharedRef<STableViewBase>& OwnerTable) {
			return SNew(STableRow<TSharedPtr<FMaterialShaderEntry>>, OwnerTable)
			[
				SNew(STextBlock)
					.AutoWrapText(false)
					.Text(FText::FromString(Value->ShaderName))
					.ToolTipText(FText::FromString(Value->ShaderFullName))
			];
		})
		.OnSelectionChanged_Lambda([PlatformPtr, QualityLevel, InstanceIndex](TSharedPtr<FMaterialShaderEntry> Item, ESelectInfo::Type SelectInfo) { PlatformPtr->OnShaderViewComboSelectionChanged(Item, QualityLevel, InstanceIndex); });

	// Refresh ShaderList when shaders are updated
	FDelegateHandle ShaderListUpdater = RefreshDependentTabs.AddLambda([PlatformPtrWeakPtr = PlatformPtr.ToWeakPtr(), ShaderListWeakPtr = ShaderListView.ToWeakPtr(), QualityLevel, InstanceIndex]()
	{
		TSharedPtr<FShaderPlatformSettings> PlatformPtr = PlatformPtrWeakPtr.Pin();
		TSharedPtr<SListView<TSharedPtr<FMaterialShaderEntry>>> ShaderListView = ShaderListWeakPtr.Pin();
		if (PlatformPtr.IsValid() && ShaderListView.IsValid())
		{
			ShaderListView->SetItemsSource(PlatformPtr->GetShaderEntries(QualityLevel, InstanceIndex));
			ShaderListView->RequestListRefresh();
		}
	});

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.OnTabClosed_Lambda([MaterialStatsWeakPtr = this->AsWeak(), ShaderListUpdater](TSharedRef<SDockTab> Tab)
		{
			TSharedPtr<FMaterialStats> MaterialStats = MaterialStatsWeakPtr.Pin();
			if (MaterialStats.IsValid() && ShaderListUpdater.IsValid())
			{
				MaterialStats->RefreshDependentTabs.Remove(ShaderListUpdater);
			}
		})
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 2.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.Text(FText::FromString(FullPlatformName))
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SScrollBox)
						.Orientation(EOrientation::Orient_Horizontal)
						.ConsumeMouseWheel(EConsumeMouseWheel::Never)
						+ SScrollBox::Slot()
						[
							ShaderListView
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ExternalScrollbar
					]
				]
				+ SSplitter::Slot()
				.Value(0.7f)
				[
					PlatformPtr->GetShaderViewerScrollBox(QualityLevel).ToSharedRef()
				]
			]
		];

	PlatformPtr->SetCodeViewerTab(QualityLevel, SpawnedTab);

	SpawnedTab->SetLabel(FText::FromString(PlatformName));

	return SpawnedTab;
}

FName FMaterialStats::MakeTabName(const EPlatformCategoryType PlatformType, const EShaderPlatform ShaderPlatformType, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	const FString TabName = FMaterialStatsUtils::GetPlatformTypeName(PlatformType) + FMaterialStatsUtils::ShaderPlatformTypeName(ShaderPlatformType) + FMaterialStatsUtils::MaterialQualityToString(QualityLevel) + FString::FromInt(InstanceIndex);

	return FName(*TabName);
}

void FMaterialStats::BuildViewShaderCodeMenus()
{
	auto TabManager = MaterialEditor->GetTabManager();
	auto ParentCategoryRef = MaterialEditor->GetWorkspaceMenuCategory();

	TSharedPtr<FWorkspaceItem> PlatformGroupMenuItem = ParentCategoryRef->AddGroup(LOCTEXT("ViewShaderCodePlatformsGroupMenu", "Shader Code"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "MaterialEditor.Tabs.HLSLCode"));

	// Create the syntax highlighter
	FHLSLSyntaxHighlighterMarshaller::FSyntaxTextStyle CodeStyle(
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Normal"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Operator"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Keyword"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.String"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Number"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Comment"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.PreProcessorKeyword"),
		FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.SourceCode.Error"));

	SyntaxHighlighter = FHLSLSyntaxHighlighterMarshaller::Create(CodeStyle);

	// add hlsl code viewer tab
	TabManager->RegisterTabSpawner( HLSLCodeTabId, FOnSpawnTab::CreateSP(this, &FMaterialStats::SpawnTab_HLSLCode))
		.SetDisplayName( LOCTEXT("HLSLCodeTab", "HLSL Code") )
		.SetGroup( PlatformGroupMenuItem.ToSharedRef() )
		.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "MaterialEditor.Tabs.HLSLCode") );

	const int32 InstanceIndex = 0; // only work on base material

	for (auto MapEntry : PlatformTypeDB)
	{
		const EPlatformCategoryType PlatformType = MapEntry.Key;

		const FString PlatformName = FMaterialStatsUtils::GetPlatformTypeName(PlatformType);
		TSharedPtr<FWorkspaceItem> PlatformMenuItem = PlatformGroupMenuItem->AddGroup(FText::FromString(PlatformName), FSlateIcon(FAppStyle::GetAppStyleSetName(), "MaterialEditor.Tabs.HLSLCode"));

		const auto& ArrShaderPlatforms = MapEntry.Value;

		for (int32 i = 0; i < ArrShaderPlatforms.Num(); ++i)
		{
			const auto& PlatformPtr = ArrShaderPlatforms[i];

			if (!PlatformPtr.IsValid())
			{
				continue;
			}

			const EShaderPlatform PlatformID = PlatformPtr->GetPlatformShaderType();

			if (PlatformID == SP_NumPlatforms || !PlatformPtr->IsCodeViewAllowed())
			{
				continue;
			}

			const FString ShaderPlatformName = PlatformPtr->GetPlatformName().ToString();
			TSharedPtr<FWorkspaceItem> ShaderPlatformMenuItem = PlatformMenuItem->AddGroup(FText::FromString(ShaderPlatformName), FSlateIcon(FAppStyle::GetAppStyleSetName(), "MaterialEditor.Tabs.HLSLCode"));

			for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
			{
				const EMaterialQualityLevel::Type QualityLevel = (EMaterialQualityLevel::Type)q;

				const FString MaterialQualityName = FMaterialStatsUtils::MaterialQualityToString(QualityLevel);
				const FName TabName = MakeTabName(PlatformType, PlatformID, QualityLevel, InstanceIndex);

				TabManager->RegisterTabSpawner(TabName, FOnSpawnTab::CreateSP(this, &FMaterialStats::SpawnTab_ShaderCode, PlatformID, QualityLevel, InstanceIndex))
					.SetGroup(ShaderPlatformMenuItem.ToSharedRef())
					.SetDisplayName(FText::FromString(MaterialQualityName));

				TSharedRef<SScrollBox> CodeScrollBox = SNew(SScrollBox)
					+ SScrollBox::Slot().Padding(5)
					[
						BuildShaderCodeWidget(LOCTEXT("ShaderCodeLabel", "Shader Code"), 
						[MaterialStats = TWeakPtr<FMaterialStats>(SharedThis(this)), PlatformID, QualityLevel]()
						{
							auto StatsPtr = MaterialStats.Pin();
							if (StatsPtr.IsValid())
							{
								return StatsPtr->GetShaderCodeText(PlatformID, QualityLevel, InstanceIndex);
							}
							return FText::FromString(TEXT("Error reading shader code!"));
						},
						[MaterialStats = TWeakPtr<FMaterialStats>(SharedThis(this)), PlatformID, QualityLevel]() {
							auto StatsPtr = MaterialStats.Pin();
							if (StatsPtr.IsValid())
							{
								FText ShaderCode = StatsPtr->GetShaderCodeText(PlatformID, QualityLevel, InstanceIndex);
								FPlatformApplicationMisc::ClipboardCopy(*ShaderCode.ToString());
								return FReply::Handled();
							}

							return FReply::Unhandled();
						},
						MakeAttributeLambda([]() { return EVisibility::Visible; }))
					];

				PlatformPtr->GetPlatformData(QualityLevel).CodeScrollBox = CodeScrollBox;
			}
		}
	}
}

bool FMaterialStats::IsCodeViewWindowActive() const
{
	for (const auto& MapEntry : ShaderPlatformStatsDB)
	{
		TSharedPtr<FShaderPlatformSettings> PlatformPtr = MapEntry.Value;

		for (int q = 0; q < EMaterialQualityLevel::Num; ++q)
		{
			if (PlatformPtr->GetCodeViewerTab((EMaterialQualityLevel::Type)q).IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

TSharedRef<SDockTab> FMaterialStats::SpawnTab_Stats(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == StatsTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("Platform Stats", "Platform Stats"))
		.OnTabClosed_Lambda([&bShowStats = bShowStats](TSharedRef<SDockTab>) { bShowStats = false; })
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MaterialStats")))
			[
				GetGridStatsWidget().ToSharedRef()
			]
		];

	StatsTab = SpawnedTab;

	// like so because the material editor will automatically restore this tab if it was still opened when the editor shuts down
	bShowStats = true;

	return SpawnedTab;
}

TSharedRef<SDockTab> FMaterialStats::SpawnTab_OldStats(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == OldStatsTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("Stats", "Stats"))
		.OnTabClosed_Lambda([&bShowStats = bShowOldStats](TSharedRef<SDockTab>) { bShowStats = false; })
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MaterialStats")))
			[
				OldStatsWidget.ToSharedRef()
			]
		];

	OldStatsTab = SpawnedTab;

	// like so because the material editor will automatically restore this tab if it was still opened when the editor shuts down
	bShowOldStats = true;

	return SpawnedTab;
}

void FMaterialStats::BuildStatsTab()
{
	const auto ParentCategoryRef = MaterialEditor->GetWorkspaceMenuCategory();
	auto TabManager = MaterialEditor->GetTabManager();

	TabManager->RegisterTabSpawner(StatsTabId, FOnSpawnTab::CreateSP(this, &FMaterialStats::SpawnTab_Stats))
		.SetDisplayName(LOCTEXT("StatsTab", "Platform Stats"))
		.SetGroup(ParentCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(),"MaterialEditor.TogglePlatformStats.Tab"));
}

void FMaterialStats::BuildOldStatsTab()
{
	const auto ParentCategoryRef = MaterialEditor->GetWorkspaceMenuCategory();
	auto TabManager = MaterialEditor->GetTabManager();

	TabManager->RegisterTabSpawner(OldStatsTabId, FOnSpawnTab::CreateSP(this, &FMaterialStats::SpawnTab_OldStats))
		.SetDisplayName(LOCTEXT("OldStatsTab", "Stats"))
		.SetGroup(ParentCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MaterialEditor.ToggleMaterialStats.Tab"));
}

void FMaterialStats::RegisterTabs()
{
	BuildStatsTab();
	BuildOldStatsTab();
	BuildViewShaderCodeMenus();
}

void FMaterialStats::UnregisterTabs()
{
	auto TabManager = MaterialEditor->GetTabManager();
	
	const int32 InstanceIndex = 0;

	for (auto MapEntry : PlatformTypeDB)
	{
		const EPlatformCategoryType PlatformType = MapEntry.Key;
		TArray<TSharedPtr<FShaderPlatformSettings>>& ArrShaderPlatforms = MapEntry.Value;

		for (int32 i = 0; i < ArrShaderPlatforms.Num(); ++i)
		{
			TSharedPtr<FShaderPlatformSettings> PlatformPtr = ArrShaderPlatforms[i];

			if (!PlatformPtr.IsValid())
			{
				continue;
			}

			const EShaderPlatform ShaderPlatformID = PlatformPtr->GetPlatformShaderType();

			for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
			{
				const FName TabName = MakeTabName(PlatformType, ShaderPlatformID, (EMaterialQualityLevel::Type)q, InstanceIndex);

				TabManager->UnregisterTabSpawner(TabName);
			}
		}
	}

	TabManager->UnregisterTabSpawner(StatsTabId);
	TabManager->UnregisterTabSpawner(OldStatsTabId);
	TabManager->UnregisterTabSpawner(HLSLCodeTabId);
}

void FMaterialStats::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Options);
}

void FMaterialStats::ComputeGridWarnings()
{
	auto GridPtr = GetGridStatsWidget();
	if (!GridPtr.IsValid())
	{
		return;
	}

	// no need to update warnings that often
	const double kMinimumTimeBetweenWarningsRefreshSeconds = 0.1;
	const double CurrentTime = FPlatformTime::Seconds();
	const bool bCooledDown = (CurrentTime - LastGridMessagesUpdate) > kMinimumTimeBetweenWarningsRefreshSeconds;

	if (!bCooledDown)
	{
		return;
	}

	LastGridMessagesUpdate = CurrentTime;

	TArray<FString> Messages;
	TArray<bool> IsErrorMessage;

	bool bAnyQualityPresent = false;
	for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
	{
		auto Quality = static_cast<EMaterialQualityLevel::Type>(q);
		bAnyQualityPresent |= GetStatsQualityFlag(Quality);
	}

	if (!bAnyQualityPresent)
	{
		Messages.Emplace("No material quality selected. Please use the 'Settings' button and choose the desired quality level to be analyzed.");
		IsErrorMessage.Add(false);
	}

	for (const auto& Entry : ShaderPlatformStatsDB)
	{
		auto& Platform = Entry.Value;
		if (Platform.IsValid())
		{
			for (int32 QualityLevel = 0; QualityLevel < EMaterialQualityLevel::Num; ++QualityLevel)
			{
				for (int32 InstanceIndex = 0; InstanceIndex < Platform->GetPlatformData(static_cast<EMaterialQualityLevel::Type>(QualityLevel)).Instances.Num(); ++InstanceIndex)
				{
					auto& Data = Platform->GetInstanceData(static_cast<EMaterialQualityLevel::Type>(QualityLevel), InstanceIndex);
					auto& Errors = Data.MaterialResourcesStats->GetCompileErrors();
					for (int32 ErrorIndex = 0; ErrorIndex < Errors.Num(); ++ErrorIndex)
					{
						const auto& PlatformName = Platform->GetPlatformName().ToString();
						const auto& AssetName = GetMaterialName(InstanceIndex);
						const FString QualityName = FMaterialStatsUtils::MaterialQualityToShortString((EMaterialQualityLevel::Type)QualityLevel);

						FString Message = FString::Printf(TEXT("[%s/%s:%s] %s"), *AssetName, *PlatformName, *QualityName, *Errors[ErrorIndex]);
						Messages.Add(Message);
						IsErrorMessage.Add(true);
					}
				}
			}
		}
	}

	bool bAnyPlatformPresent = false;
	bool bPlatformsNeedOfflineCompiler = false;

	const auto& PlatformList = GetPlatformsDB();
	for (auto Pair : PlatformList)
	{
		auto& PlatformPtr = Pair.Value;

		if (PlatformPtr->IsPresentInGrid())
		{
			bAnyPlatformPresent = true;

			auto ShaderPlatformType = PlatformPtr->GetPlatformShaderType();
			bool bNeedsOfflineCompiler = FMaterialStatsUtils::PlatformNeedsOfflineCompiler(ShaderPlatformType);
			if (bNeedsOfflineCompiler)
			{
				bool bCompilerAvailable = FMaterialStatsUtils::IsPlatformOfflineCompilerAvailable(ShaderPlatformType);

				if (!bCompilerAvailable)
				{
					auto WarningString = FString::Printf(TEXT("Platform %s needs an offline shader compiler to extract instruction count. Please check 'Editor Preferences' -> 'Content Editors' -> 'Material Editor' for additional settings."), *PlatformPtr->GetPlatformName().ToString());
					Messages.Add(MoveTemp(WarningString));
					IsErrorMessage.Add(false);
				}
			}
		}
	}

	if (!bAnyPlatformPresent)
	{
		Messages.Emplace("No platform selected. Please use the 'Settings' button and choose desired platform to be analyzed.");
		IsErrorMessage.Add(false);
	}

	if (Messages != LastGridMessages)
	{
		LastGridMessages = Messages;

		GridPtr->ClearMessages();

		check(Messages.Num() == IsErrorMessage.Num());
		for (int32 i = 0; i < Messages.Num(); ++i)
		{
			GridPtr->AddMessage(Messages[i], IsErrorMessage[i]);
		}
	}
}

void FMaterialStats::ExtractHLSLCode()
{
	HLSLCode = TEXT("");
	PreviousTranslatorHLSLCode = TEXT("");
	MaterialIR = TEXT("");

 	if (!HLSLTab.IsValid())
	{
		return;
	}

	FString FullSource;
	if (MaterialInterface != nullptr && MaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform)->GetMaterialExpressionSource(FullSource))
	{
		if (const FMaterialInsights* Insights = MaterialInterface->MaterialInsight.Get())
		{
			MaterialIR = Insights->IRString;

			TMap<FString, FString> SortedLegacy_ShaderString = Insights->Legacy_ShaderStringParameters; 
			SortedLegacy_ShaderString.KeySort([](const FString& A, const FString& B) { return A < B; });

			TMap<FString, FString> SortedNew_ShaderString = Insights->New_ShaderStringParameters; 
			SortedNew_ShaderString.KeySort([](const FString& A, const FString& B) { return A < B; });

			for (const TPair<FString, FString>& Pair : SortedLegacy_ShaderString)
			{
				const FString& Key = Pair.Key;
				const FString& Value = Pair.Value;
				PreviousTranslatorHLSLCode += FString::Printf(TEXT("// %s\n"), *Key);
				PreviousTranslatorHLSLCode += FString::Printf(TEXT("%s\n"), *Value);
			}

			for (const TPair<FString, FString>& Pair : SortedNew_ShaderString)
			{
				const FString& Key = Pair.Key;
				const FString& Value = Pair.Value;
				HLSLCode += FString::Printf(TEXT("// %s\n"), *Key);
				HLSLCode += FString::Printf(TEXT("%s\n"), *Value);
			}

			if (MaterialInterface->IsUsingNewHLSLGenerator())
			{
				HLSLCode += TEXT("// FULL SOURCE BELOW\n");
				HLSLCode += FullSource;
			}
			else
			{
				PreviousTranslatorHLSLCode += TEXT("// FULL SOURCE BELOW\n");
				PreviousTranslatorHLSLCode += FullSource;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

/*end FMaterialStats functions*/
/***********************************************************************************************************************/
