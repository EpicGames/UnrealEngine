// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/Material.h"
#include "MaterialStatsCommon.h"
#include "UObject/GCObject.h"
#include "Preferences/MaterialStatsOptions.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"

class FHLSLSyntaxHighlighterMarshaller;

/** structure used to store various statistics extracted from compiled shaders */
struct FShaderStatsInfo
{
	struct FContent
	{
		FString StrDescription;
		FString StrDescriptionLong;
	};

	TMap<ERepresentativeShader, FContent> ShaderInstructionCount;
	TMap<ERepresentativeShader, FContent> GenericShaderStatistics;
	FContent SamplersCount;
	FContent InterpolatorsCount;
	FContent TextureSampleCount;
	FContent VirtualTextureLookupCount;
	FContent VirtualTextureStackCount;
	FContent ShaderCount;
	FContent PreShaderCount;
	FContent LWCUsage;
	FString StrShaderErrors;

	void Reset()
	{
		ShaderInstructionCount.Empty();
		GenericShaderStatistics.Empty();

		SamplersCount.StrDescription = TEXT("Compiling...");
		SamplersCount.StrDescriptionLong = TEXT("Compiling...");

		InterpolatorsCount.StrDescription = TEXT("Compiling...");
		InterpolatorsCount.StrDescriptionLong = TEXT("Compiling...");

		TextureSampleCount.StrDescription = TEXT("Compiling...");
		TextureSampleCount.StrDescriptionLong = TEXT("Compiling...");

		VirtualTextureLookupCount.StrDescription = TEXT("Compiling...");
		VirtualTextureLookupCount.StrDescriptionLong = TEXT("Compiling...");

		VirtualTextureStackCount.StrDescription = TEXT("Compiling...");
		VirtualTextureStackCount.StrDescriptionLong = TEXT("Compiling...");

		ShaderCount.StrDescription = TEXT("Compiling...");
		ShaderCount.StrDescriptionLong = TEXT("Compiling...");

		PreShaderCount.StrDescription = TEXT("Compiling...");
		PreShaderCount.StrDescriptionLong = TEXT("Compiling...");

		LWCUsage.StrDescription = TEXT("Compiling...");
		LWCUsage.StrDescriptionLong = TEXT("Compiling...");

		StrShaderErrors.Empty();
	}

	void Empty()
	{
		ShaderInstructionCount.Empty();
		GenericShaderStatistics.Empty();

		SamplersCount.StrDescription.Empty();
		SamplersCount.StrDescriptionLong.Empty();

		InterpolatorsCount.StrDescription.Empty();
		InterpolatorsCount.StrDescriptionLong.Empty();
		
		TextureSampleCount.StrDescription.Empty();
		TextureSampleCount.StrDescriptionLong.Empty();

		VirtualTextureLookupCount.StrDescription.Empty();
		VirtualTextureLookupCount.StrDescriptionLong.Empty();

		VirtualTextureStackCount.StrDescription.Empty();
		VirtualTextureStackCount.StrDescriptionLong.Empty();

		ShaderCount.StrDescription.Empty();
		ShaderCount.StrDescriptionLong.Empty();

		PreShaderCount.StrDescription.Empty();
		PreShaderCount.StrDescriptionLong.Empty();

		StrShaderErrors.Empty();
	}

	bool HasErrors()
	{
		return !StrShaderErrors.IsEmpty();
	}
};

struct FMaterialShaderEntry
{
	FShaderId ShaderId;
	FString ShaderName;
	FString ShaderFullName;
};

/** structure used to manage shader compilation and source code extraction for a specified shader platform
*   used for building the material stats */
struct FShaderPlatformSettings
{
public:
	struct FInstanceData
	{
		/** pointer to the material resource created for this platform
		*  mainly used to compile the shaders and extract information from them */
		FMaterialResourceStats* MaterialResourcesStats;

		/** array of shader ids for this platform; needed to fill ComboBox in MaterialEditor's shader viewer
		* generated from ShaderID.ShaderType->GetFName() */
		TArray<TSharedPtr<FMaterialShaderEntry>> ArrShaderEntries;

		/** ComboBox current entry */
		FMaterialShaderEntry ComboBoxSelectedEntry;

		/** when true we should update the content of [FText ShaderCode] variable */
		bool bUpdateShaderCode = false;
		/** cached shader code computed by FShaderPlatformSettings::GetShaderCode() */
		FText ShaderCode;
		/** flag that marks an ongoing shader compilation */
		bool bCompilingShaders = false;
		/** flag suggests we needed to recompile shaders due to changes in the material */
		bool bNeedShaderRecompilation = false;
		/** flag suggests we only need to compile the most complex shader from available set */
		bool bOnlyCompileMostComplexShader = false;
		/** flag suggests we needed to warn the user due to compilation errors */
		bool bNeedToWarnAboutCompilationErrors = false;

		FShaderStatsInfo ShaderStatsInfo;
	};
	
	/////////////////////
	/** inner structure used to hold properties for a single material platform and with a specific quality level */
	struct FPlatformData
	{
		/* Array of material instances including base material and any material instances derived from it */
		TArray<FInstanceData> Instances;
		
		/** flag that marks the usage of this data structure */
		bool bExtractStats = false;

		/** true when code is listed in its own tab */
		bool bExtractCode = false;

		/** object used to display the content of [FText ShaderCode] */
		TSharedPtr<class SScrollBox> CodeScrollBox;
		/** weak pointer to the spawned shader code viewer tab */
		TWeakPtr<class SDockTab> CodeViewerTab;

		/** Time when the previous compilation was requested.*/
		double LastTimeCompilationRequested = 0.0;
	};
	/////////////////////

private:
	/** array of the above defined data structure, for each material quality setting */
	FPlatformData PlatformData[EMaterialQualityLevel::Num];

	/** type of platform for this material setting (Desktop, Android etc) */
	EPlatformCategoryType PlatformType;
	/** shader type used for this material setting (eg VULKAN_SM5) */
	EShaderPlatform PlatformShaderID;
	/** The name of the platform given at its creation time (ie the constructor) */
	FName PlatformName;
	/** The id of the platform computed from PlatformID */
	FName PlatformNameID;

	FString PlatformDescription;

	/** if true this will be visible in the material stats. grid */
	bool bPresentInGrid = false;

	/** if true this platform is always present in material stats grid */
	bool bAlwaysOn = false;

	/** if true this will be listed in the 'view code' menu */
	bool bAllowCodeView = false;

	/** if true this can be added in the stats grid widget to be analyzed */
	bool bAllowPresenceInGrid = false;

	/** pointer to the material whose stats are analyzed */
	UMaterial *Material = nullptr;
	/** pointer to the material instance whose stats are analyzed */
	UMaterialInstance *MaterialInstance = nullptr;
	/** array of derived material instances whose stats are analyzed */ 
	TArray<TObjectPtr<UMaterialInstance>> DerivedMaterialInstances;

private:
	/** function used to trigger shader rebuilding when needed */
	/** returns true if shaders are being recompiled, false otherwise */
	bool CheckShaders(bool bIgnoreCooldown);

	/** builds material resources needed to compile shaders */
	void AllocateMaterialResources();

	/** frees the material resources allocated by AllocateMaterialResources() */
	void ClearResources();

public:
	FShaderPlatformSettings(
		const EPlatformCategoryType _PlatformType,
		const EShaderPlatform _ShaderPlatformID,
		const FName _Name,
		const bool _bAllowPresenceInGrid,
		const bool _bAllowCodeView,
		const FString& _Description,
		const bool bAlwaysOn);
	~FShaderPlatformSettings();

	/** returns the name of this platform given in the constructor */
	FORCEINLINE FName GetPlatformName() const;

	/** retuns the id of this platform as a FName computed by FMaterialStatsUtils::ShaderPlatformTypeName() */
	FORCEINLINE FName GetPlatformID() const;

	/** returns the type of this platform (desktop, android, ios...) */
	FORCEINLINE EPlatformCategoryType GetCategoryType() const;

	FORCEINLINE FString GetPlatformDescription() const;

	/** returns the assigned scrollbox used to view shaders for a specific material quality level */
	FORCEINLINE TSharedPtr<class SScrollBox> GetShaderViewerScrollBox(const EMaterialQualityLevel::Type QualityLevel);

	/** stores a pointer to the spawned window tab that will contain the shader code */
	/** we need this to keep track of opened tabs */
	FORCEINLINE void SetCodeViewerTab(const EMaterialQualityLevel::Type QualityLevel, TSharedRef<SDockTab> Tab);

	/** retrives the pointer to the spawned window tab that contains the shader code */
	FORCEINLINE TWeakPtr<class SDockTab> GetCodeViewerTab(const EMaterialQualityLevel::Type QualityLevel);

	/** returns an array with the names of all the compiled shaders for this material with specified quality level, can return nullptr if no instance index is not available */
	FORCEINLINE const TArray<TSharedPtr<FMaterialShaderEntry>> *GetShaderEntries(const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);

	/** when set this flag will indicate the presence of this material with a particular quality level inside the stats widget */
	void SetExtractStatsFlag(const EMaterialQualityLevel::Type QualityType, const bool bValue);

	/** returns whether or not this material is allowed to display its shader code; set in the contructor */
	FORCEINLINE bool IsCodeViewAllowed() const;

	FORCEINLINE bool IsStatsGridPresenceAllowed() const;

	/** flag the need to extract and display the shaders generated by this material with a certain quality level */
	FORCEINLINE void SetCodeViewNeeded(const EMaterialQualityLevel::Type Quality, const bool bValue);

	/** returns whether or not this material was chosen to be displayed in the stats grid widget with any material quality level */
	FORCEINLINE bool IsPresentInGrid() const;

	/** returns whether or not this material is always presnet in the stats grid widget */
	FORCEINLINE bool IsAlwaysOn() const;

	/** used by the grid widget to enable or disable the presence of this material */
	FORCEINLINE bool FlipPresentInGrid();

	/** used  to enable or disable the presence of this material in the stats grid widget */
	FORCEINLINE void SetPresentInGrid(const bool bValue);

	/** flags shader compilation for a specific quality level */
	FORCEINLINE void SetNeedShaderCompilation(const EMaterialQualityLevel::Type QualityLevel, const bool bValue, const bool bOnlyCompileDerivedMI);

	/** returns the shader type used by this platform as set at construction time */
	FORCEINLINE EShaderPlatform GetPlatformShaderType() const;

	/** returns a reference to the platform settings used for a specified material quality level; see the definition of FPlatformData inner structure */
	FORCEINLINE FPlatformData& GetPlatformData(const EMaterialQualityLevel::Type QualityLevel);

	/** returns a reference to the platform instance data used for a specified material quality level and instance index; see the definition of FInstanceData inner structure */
	FORCEINLINE FInstanceData& GetInstanceData(const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);

	/** returns the shader name chosen in the shader viewer combo-box */
	FText GetSelectedShaderViewComboText(EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) const;

	/** callback function called when we change the content of the shader viewer combo-box, used to select a different shader to be displayed */
	void OnShaderViewComboSelectionChanged(TSharedPtr<FMaterialShaderEntry> Item, EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex);

	/** returns the actual shader source selected by the shaders viewer's combo-box */
	FText GetShaderCodeText(const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex);

	/** returns the number of shaders in this material w/o compiling shaders */
	FText GetShaderCountText(const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex);

	/** returns all shaders' stats concatenated together*/
	FString GetShadersStats() const;

	/** call this whenever the analyzed material or material instance is changed */
	void SetMaterial(UMaterial *InBaseMaterial, UMaterialInstance *InBaseMaterialInstance, const TArray<TObjectPtr<UMaterialInstance>>& InDerivedMaterialInstances);

	/** main function used to update the state of this platform; will be called from FMaterialStats::Update() */
	/** returns true if something changed for this the update call, false otherwise */
	bool Update();

	bool CachePendingShaders();
};

/** name alias used bellow in FMaterialStats */
using TMapPlatformSettings = TMap<EPlatformCategoryType, TArray<TSharedPtr<FShaderPlatformSettings>>>;
using TMapPlatformTypeSettings = TMap<EShaderPlatform, TSharedPtr<FShaderPlatformSettings>>;

/** structure used as a collection of the above FShaderPlatformSettings, for each needed shader platform */
/** also manages material stats extraction and stats grid content */
class FMaterialStats : public FGCObject, public TSharedFromThis<FMaterialStats>
{
	/** friendship established because FMaterialStatsUtils does the actual instantiation of this class */
	friend class FMaterialStatsUtils;

	/** maps that contain the collection of all platforms from which we can extract statistics */
	TMapPlatformTypeSettings ShaderPlatformStatsDB; //sorted by shader platform (gl_sm5, d3d_sm4, etc)
	TMapPlatformSettings PlatformTypeDB; // sorted by platform type (desktop, android, ios etc)

	/** pointer to the widget that will display the collected data from all the above platforms */
	TSharedPtr<class SMaterialEditorStatsWidget> GridStatsWidget;
	TSharedPtr<class SWidget> OldStatsWidget;
	TSharedPtr<class IMessageLogListing> OldStatsListing;

	/** pointer to the logical grid that will prepare data to be displayed by the above GridStatsWidget */
	TSharedPtr<class FMaterialStatsGrid> StatsGrid;

	/** array of bools that flag a specific global material quality setting of the stats grid widget */
	bool bArrStatsQualitySelector[EMaterialQualityLevel::Num] = { false };
	bool bArrStatsQualitySelectorAlwaysOn[EMaterialQualityLevel::Num] = { false };

	/** inspect all derived material instances also */
	EMaterialStatsDerivedMIOption MaterialStatsDerivedMIOption = EMaterialStatsDerivedMIOption::CompileOnly;

	/** names of the analyzed materials */
	TArray<FString> MaterialNames;

	/** the id of the stats grid widget tab */
	static const FName StatsTabId;
	static const FName OldStatsTabId;
	static const FName HLSLCodeTabId;

	/** If true, show material stats like number of shader instructions. */
	bool bShowStats = false;
	bool bShowOldStats = false;

	/** Tracks whether the code tab is open, so we don't have to update it when closed. */
	TWeakPtr<SDockTab> StatsTab;
	TWeakPtr<SDockTab> OldStatsTab;
	TWeakPtr<SDockTab> HLSLTab;

	/** Cached HLSL code for analyzed material */
	FString HLSLCode;

	/** If the new translator is enabled this will contain the source for the old translator.  Useful for comparing. */
	FString PreviousTranslatorHLSLCode;

	/** Cached IR from the new translator if enabled */
	FString MaterialIR;

	TObjectPtr<class UMaterialStatsOptions >Options = nullptr;

	/** Pointer to Material Editor or to Material Instance Editor set by Initialize() function */
	class IMaterialEditor *MaterialEditor = nullptr;

	/** pointer to the material interface whose stats are analyzed */
	class UMaterialInterface *MaterialInterface = nullptr;

	/** array of pointers to derived material instances whose stats are analyzed */
	TArray<TObjectPtr<UMaterialInstance>> DerivedMaterialInstances;

	/** grid warning messages */
	TArray<FString> LastGridMessages;
	double LastGridMessagesUpdate = 0.0;

	bool bNeedsGridRefresh = false;

	TMulticastDelegate<void()> RefreshDependentTabs;

	TSharedPtr<FHLSLSyntaxHighlighterMarshaller> SyntaxHighlighter;

private:
	/** adds a specified platform in the grid widget for analysis; usually called from BuildShaderPlatformDB() */
	TSharedPtr<FShaderPlatformSettings> AddShaderPlatform(const EPlatformCategoryType PlatformType, const EShaderPlatform PlatformID, const FName PlatformName, const bool bAllowCodeView, const FString& Description, const bool bAlwaysOn = false);

	/** build a collection of available shader platform for which we can extract various statistics */
	void BuildShaderPlatformDB(const bool bAllowIgnoringCompilationErrors);

	/** this will spawn the window that will display the a specific set of shaders from the analyzed material */
	TSharedRef<class SDockTab> SpawnTab_ShaderCode(const class FSpawnTabArgs& Args, const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex);
	TSharedRef<class SDockTab> SpawnTab_HLSLCode(const class FSpawnTabArgs& Args);

	TSharedRef<SWidget> BuildShaderCodeWidget(FText TextLabel, TFunction<FText(void)>&& InShaderCodeCallback, TFunction<FReply(void)>&& InCopyCodeCallback, TAttribute<EVisibility> InVisibilityAttribute);

	/** utility function used to build names for the shader viewing tabs  */
	static FName MakeTabName(const EPlatformCategoryType PlatformType, const EShaderPlatform ShaderPlatformType, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);

	/** functions responsible for construction and spawning of the window that will display the stats grid widget */
	void BuildStatsTab();
	TSharedRef<SDockTab> SpawnTab_Stats(const FSpawnTabArgs& Args);

	void BuildOldStatsTab();
	TSharedRef<SDockTab> SpawnTab_OldStats(const FSpawnTabArgs& Args);

	/** adds shader view menus for the given IMaterialEditor */
	void BuildViewShaderCodeMenus();

	/** displays or hides the window that displays the stats grid widget */
	void ToggleStats();
	void ToggleOldStats();
	void SetShowStats(const bool bValue);
	void SetShowOldStats(const bool bValue);

	/** wrapper function that calls SetCodeViewNeeded on the specified platform to signal the need (or not) to extract compile its shaders and extract the source code */
	void SetShaderPlatformUseCodeView(const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type Quality, const bool bValue);

	/** checks if there are any opened windows that display shader code */
	bool IsCodeViewWindowActive() const;

	/** true when we are displaying any content in the stats grid widget */
	FORCEINLINE bool IsShowingStats() const;
	FORCEINLINE bool IsShowingOldStats() const;

	/** triggers and refresh call for the stats grid widget */
	void RefreshStatsGrid();
	/** this will display/hide the stats grid widget */
	void DisplayStatsGrid(const bool bShow);
	void DisplayOldStats(const bool bShow);

	void LoadSettings(const bool bAllowIgnoringCompilationErrors);
	void SaveSettings();

	/** function that will collect (eventual) warning messages when the stats grid is not properly configured */
	/** the actual display is handled by the SMaterialEditorStatsWidget class */
	void ComputeGridWarnings();

	/** extracts the HLSL code for analyzed material */
	void ExtractHLSLCode();

	/** use FMaterialStatsUtils::CreateMaterialStats() to create an instance of this class */
	FMaterialStats() {}
public:
	~FMaterialStats();

	///////////////////////////////////////////
	// Utilities Functions
	/** return the name of the window that displays the grid stats widget */
	static FORCEINLINE FName GetGridStatsTabName();
	static FORCEINLINE FName GetGridOldStatsTabName();

	/** returns the material name we're analyzing */
	FORCEINLINE FString GetMaterialName(const int32 InstanceIndex) const;

	/** returns a pointer to the material stats grid widget */
	FORCEINLINE TSharedPtr<class SMaterialEditorStatsWidget> GetGridStatsWidget();

	/** functions that return references to the collection of stored platforms that analyze the current material */
	FORCEINLINE const TMapPlatformTypeSettings& GetPlatformsDB() const;
	FORCEINLINE const TMapPlatformSettings& GetPlatformsTypeDB() const;

	FORCEINLINE TSharedPtr<FMaterialStatsGrid> GetStatsGrid();

	FORCEINLINE TSharedPtr<class IMessageLogListing> GetOldStatsListing();

	/** functions used to query or set a material quality level that is to be analyzed in the material stats grid widget */
	bool SwitchStatsQualityFlag(const EMaterialQualityLevel::Type Quality);
	void SetStatusQualityFlag(const EMaterialQualityLevel::Type Quality, const bool bValue);
	FORCEINLINE bool GetStatsQualityFlag(const EMaterialQualityLevel::Type Quality);
	FORCEINLINE bool GetStatsQualityFlagAlwaysOn(const EMaterialQualityLevel::Type Quality);

	void SetMaterialStatsDerivedMIOption(const EMaterialStatsDerivedMIOption value);
	FORCEINLINE EMaterialStatsDerivedMIOption GetMaterialStatsDerivedMIOption() const;
	FORCEINLINE bool GetProvideDerivedMIFlag() const;

	/** switches on or off the presence of a specified shader platform inside the stats grid widget for this material */
	bool SwitchShaderPlatformUseStats(const EShaderPlatform PlatformID);

	/** Sets the name of the material that will be displayed in the stats grid widget */
	void SetMaterialsDisplayNames(const TArray<FString>& Names);

	/** returns a platform name given at its construction time */
	FName GetPlatformName(const EShaderPlatform InEnumValue) const;

	/** returns the shader type used by a platform with the specified name */
	EShaderPlatform GetShaderPlatformID(const FName InName) const;

	/** utility functions that return pointers to the requested platforms */
	TSharedPtr<FShaderPlatformSettings> GetPlatformSettings(const EShaderPlatform PlatformID);
	TSharedPtr<FShaderPlatformSettings> GetPlatformSettings(const FName PlatformName);

	/** returns the shader code computed by the specified platform with some quality level */
	FText GetShaderCodeText(const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex);

	/** returns the number of shaders in the material with the specified platform with some quality level.  Does this without compiling shaders. */
	FText GetShaderCountText(const EShaderPlatform PlatformID, const EMaterialQualityLevel::Type QualityType, const int32 InstanceIndex);

	/** returns all shaders' stats concatenated together*/
	FString GetShadersStats() const;

	/** call this whenever some material property is changed, as it will trigger shader recompilation */
	void SignalMaterialChanged();

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMaterialStats");
	}
	// end Utilities Functions
	///////////////////////////////////////////

	///////////////////////////////////////////
	// Setup Functions
	/** call this from chosen IMaterialEditor, when all other tabs are registered, to add the material stats grid tab and shader view menus */
	void RegisterTabs();

	/** call this from chosen IMaterialEditor, when all other tabs are unregistered, to remove material stats grid tab and shader view menus */
	void UnregisterTabs();

private:
	/** this function will do the setup procedure and its called from FMaterialStatsUtils::CreateMaterialStats()  */
	void Initialize(IMaterialEditor *MaterialEditor, const bool bShowMaterialInstancesMenu, const bool bAllowIgnoringCompilationErrors);
public:
	//end Setup Functions
	///////////////////////////////////////////

	///////////////////////////////////////////
	// Update Functions
	/** call this from chosen IMaterialEditor whenever an update to this analysis tool in appropriate */
	void Update();

	/** call this to request caching required shaders regardless of cooldown */
	void CacheAndCompilePendingShaders();

	/** this will set the material or material instance to be analyzed by this class */
	void SetMaterial(UMaterial *InMaterial, const TArray<TObjectPtr<UMaterialInstance>>& InDerivedMaterialInstances);
	void SetMaterial(UMaterialInstance *InMaterialInstance);

	// end Update Functions
	///////////////////////////////////////////

	/** returns true if any new compilation errors are discovered */
	bool AnyNewCompilationErrors(const int32 StartingFromInstanceIndex = 0);
};


//////////////////////////////////////////////////////////////////////////////////////////////////
// FMaterialStats implementation

FORCEINLINE const TMapPlatformTypeSettings& FMaterialStats::GetPlatformsDB() const
{
	return ShaderPlatformStatsDB;
}

FORCEINLINE const TMapPlatformSettings& FMaterialStats::GetPlatformsTypeDB() const
{
	return PlatformTypeDB;
}

FORCEINLINE TSharedPtr<FMaterialStatsGrid> FMaterialStats::GetStatsGrid()
{
	return StatsGrid;
}

FORCEINLINE TSharedPtr<class IMessageLogListing> FMaterialStats::GetOldStatsListing()
{
	return OldStatsListing;
}

FORCEINLINE bool FMaterialStats::IsShowingStats() const
{
	return bShowStats;
}

FORCEINLINE bool FMaterialStats::IsShowingOldStats() const
{
	return bShowOldStats;
}

FORCEINLINE void FMaterialStats::SetMaterialsDisplayNames(const TArray<FString>& Names)
{
	MaterialNames = Names;
}

FORCEINLINE FString FMaterialStats::GetMaterialName(const int32 InstanceIndex) const
{
	if (InstanceIndex < MaterialNames.Num())
	{
		return MaterialNames[InstanceIndex];
	}
	else
	{
		return TEXT("");
	}
}

FORCEINLINE FName FMaterialStats::GetGridStatsTabName()
{
	return StatsTabId;
}

FORCEINLINE FName FMaterialStats::GetGridOldStatsTabName()
{
	return OldStatsTabId;
}

FORCEINLINE bool FMaterialStats::GetStatsQualityFlag(const EMaterialQualityLevel::Type Quality)
{
	check(Quality < EMaterialQualityLevel::Num);
	return bArrStatsQualitySelector[(int32)Quality];
}

FORCEINLINE bool FMaterialStats::GetStatsQualityFlagAlwaysOn(const EMaterialQualityLevel::Type Quality)
{
	check(Quality < EMaterialQualityLevel::Num);
	return bArrStatsQualitySelectorAlwaysOn[(int32)Quality];
}

FORCEINLINE EMaterialStatsDerivedMIOption FMaterialStats::GetMaterialStatsDerivedMIOption() const
{
	return MaterialStatsDerivedMIOption;
}

FORCEINLINE bool FMaterialStats::GetProvideDerivedMIFlag() const
{
	return MaterialStatsDerivedMIOption != EMaterialStatsDerivedMIOption::Ignore;
}

FORCEINLINE TSharedPtr<class SMaterialEditorStatsWidget> FMaterialStats::GetGridStatsWidget()
{
	return GridStatsWidget;
}

// end FMaterialStats implementation
//////////////////////////////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////////////////////////////
// FShaderPlatformSettings implementation
FORCEINLINE FShaderPlatformSettings::~FShaderPlatformSettings()
{
	ClearResources();
}

FORCEINLINE FName FShaderPlatformSettings::GetPlatformName() const
{
	return PlatformName;
}

FORCEINLINE FName FShaderPlatformSettings::GetPlatformID() const
{
	return PlatformNameID;
}

FORCEINLINE EPlatformCategoryType FShaderPlatformSettings::GetCategoryType() const
{
	return PlatformType;
}

FORCEINLINE FString FShaderPlatformSettings::GetPlatformDescription() const
{
	return PlatformDescription;
}

FORCEINLINE TSharedPtr<class SScrollBox> FShaderPlatformSettings::GetShaderViewerScrollBox(const EMaterialQualityLevel::Type QualityLevel)
{
	FPlatformData& SomePlatformData = GetPlatformData(QualityLevel);
	return SomePlatformData.CodeScrollBox;
}

FORCEINLINE void FShaderPlatformSettings::SetCodeViewerTab(const EMaterialQualityLevel::Type QualityLevel, TSharedRef<SDockTab> Tab)
{
	FPlatformData& SomePlatformData = GetPlatformData(QualityLevel);
	SomePlatformData.CodeViewerTab = Tab;
}

FORCEINLINE TWeakPtr<class SDockTab> FShaderPlatformSettings::GetCodeViewerTab(const EMaterialQualityLevel::Type QualityLevel)
{
	FPlatformData& SomePlatformData = GetPlatformData(QualityLevel);
	return SomePlatformData.CodeViewerTab;
}

FORCEINLINE const TArray<TSharedPtr<FMaterialShaderEntry>> *FShaderPlatformSettings::GetShaderEntries(const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	check(QualityLevel < EMaterialQualityLevel::Num);
	if (InstanceIndex < PlatformData[QualityLevel].Instances.Num())
	{
		auto& InstanceData = GetInstanceData(QualityLevel, InstanceIndex);
		return &InstanceData.ArrShaderEntries;
	}
	else
	{
		return nullptr;
	}
}

FORCEINLINE void FShaderPlatformSettings::SetExtractStatsFlag(const EMaterialQualityLevel::Type QualityType, const bool bValue)
{
	check(QualityType != EMaterialQualityLevel::Num);
	PlatformData[QualityType].bExtractStats = bValue;
}

FORCEINLINE bool FShaderPlatformSettings::IsCodeViewAllowed() const
{
	return bAllowCodeView;
}

FORCEINLINE bool FShaderPlatformSettings::IsStatsGridPresenceAllowed() const
{
	return bAllowPresenceInGrid;
}

FORCEINLINE void FShaderPlatformSettings::SetCodeViewNeeded(const EMaterialQualityLevel::Type Quality, const bool bValue)
{
	PlatformData[Quality].bExtractCode = bValue;
}

FORCEINLINE bool FShaderPlatformSettings::IsPresentInGrid() const
{
	return bPresentInGrid;
}

FORCEINLINE bool FShaderPlatformSettings::IsAlwaysOn() const
{
	return bAlwaysOn;
}

FORCEINLINE bool FShaderPlatformSettings::FlipPresentInGrid()
{
	SetPresentInGrid(!IsPresentInGrid());

	return IsPresentInGrid();
}

FORCEINLINE void FShaderPlatformSettings::SetPresentInGrid(const bool bValue)
{
	if (bAllowPresenceInGrid) // if we're allowed to be present in the stats grid
	{
		if (!bAlwaysOn) // and bAlwaysOn is false - then any bValue is fine
		{
			bPresentInGrid = bValue;
		}
		else if (bValue == true) // otherwise only enabling is fine, as we shouldn't be able to disable this platform
		{
			bPresentInGrid = bValue;
		}
	}
	else
	{
		if (bValue == false) // if we're not allowed to be present, only disabling this platform is fine
		{
			bPresentInGrid = bValue;
		}
	}
}

FORCEINLINE void FShaderPlatformSettings::SetNeedShaderCompilation(const EMaterialQualityLevel::Type QualityLevel, const bool bValue, const bool bOnlyCompileDerivedMI)
{
	check(QualityLevel < EMaterialQualityLevel::Num);
	for (int32 InstancesIndex = 0; InstancesIndex < PlatformData[QualityLevel].Instances.Num(); ++InstancesIndex)
	{
		auto& Instance = GetInstanceData(QualityLevel, InstancesIndex);
		Instance.bNeedShaderRecompilation = bValue;
		Instance.bOnlyCompileMostComplexShader = (InstancesIndex > 0) && bOnlyCompileDerivedMI;
	}
}

FORCEINLINE EShaderPlatform FShaderPlatformSettings::GetPlatformShaderType() const
{
	return PlatformShaderID;
}

FORCEINLINE FShaderPlatformSettings::FPlatformData& FShaderPlatformSettings::GetPlatformData(const EMaterialQualityLevel::Type QualityLevel)
{
	check(QualityLevel < EMaterialQualityLevel::Num);
	return PlatformData[QualityLevel];
}

FORCEINLINE FShaderPlatformSettings::FInstanceData& FShaderPlatformSettings::GetInstanceData(const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	check(QualityLevel < EMaterialQualityLevel::Num);
	return PlatformData[QualityLevel].Instances[InstanceIndex];
}

// end FShaderPlatformSettings implementation
//////////////////////////////////////////////////////////////////////////////////////////////////
