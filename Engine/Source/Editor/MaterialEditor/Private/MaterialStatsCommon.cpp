// Copyright Epic Games, Inc. All Rights Reserved.
//
#include "MaterialStatsCommon.h"
#include "EngineGlobals.h"
#include "MaterialStats.h"
#include "LocalVertexFactory.h"
#include "GPUSkinVertexFactory.h"
#include "RenderUtils.h"
#include "MaterialEditorSettings.h"
#include "RHIShaderFormatDefinitions.inl"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"
#include "Styling/StyleColors.h"

/***********************************************************************************************************************/
/*begin FMaterialResourceStats functions*/

void FMaterialResourceStats::SetupExtraCompilationSettings(FExtraShaderCompilerSettings& Settings) const
{
	Settings.bExtractShaderSource = true;
	FMaterialStatsUtils::GetPlatformOfflineCompilerSettings(GetShaderPlatform(), Settings);
}

/*end FMaterialResourceStats functions*/
/***********************************************************************************************************************/

/***********************************************************************************************************************/
/*begin FMaterialStatsUtils */

TSharedPtr<FMaterialStats> FMaterialStatsUtils::CreateMaterialStats(class IMaterialEditor* MaterialEditor, const bool bShowMaterialInstancesMenu, const bool bAllowIgnoringCompilationErrors)
{
	TSharedPtr<FMaterialStats> MaterialStats = MakeShareable(new FMaterialStats());
	MaterialStats->Initialize(MaterialEditor, bShowMaterialInstancesMenu, bAllowIgnoringCompilationErrors);

	return MaterialStats;
}

FString FMaterialStatsUtils::MaterialQualityToString(const EMaterialQualityLevel::Type Quality)
{
	FString StrQuality;

	switch (Quality)
	{
		case EMaterialQualityLevel::High:
			StrQuality = TEXT("High Quality");
		break;
		case EMaterialQualityLevel::Medium:
			StrQuality = TEXT("Medium Quality");
		break;
		case EMaterialQualityLevel::Low:
			StrQuality = TEXT("Low Quality");
		break;
		case EMaterialQualityLevel::Epic:
			StrQuality = TEXT("Epic Quality");
		break;
	}

	return StrQuality;
}

FString FMaterialStatsUtils::MaterialQualityToShortString(const EMaterialQualityLevel::Type Quality)
{
	FString StrQuality;

	switch (Quality)
	{
		case EMaterialQualityLevel::High:
			StrQuality = TEXT("High");
		break;
		case EMaterialQualityLevel::Medium:
			StrQuality = TEXT("Medium");
		break;
		case EMaterialQualityLevel::Low:
			StrQuality = TEXT("Low");
		break;
		case EMaterialQualityLevel::Epic:
			StrQuality = TEXT("Epic");
		break;
	}

	return StrQuality;
}

EMaterialQualityLevel::Type FMaterialStatsUtils::StringToMaterialQuality(const FString& StrQuality)
{
	if (StrQuality.Equals(TEXT("High Quality")))
	{
		return EMaterialQualityLevel::High;
	}		
	else if (StrQuality.Equals(TEXT("Medium Quality")))
	{
		return EMaterialQualityLevel::Medium;
	}
	else if (StrQuality.Equals(TEXT("Low Quality")))
	{
		return EMaterialQualityLevel::Low;
	}
	else if (StrQuality.Equals(TEXT("Epic Quality")))
	{
		return EMaterialQualityLevel::Epic;
	}

	return EMaterialQualityLevel::Num;
}

FString FMaterialStatsUtils::GetPlatformTypeName(const EPlatformCategoryType InEnumValue)
{
	FString PlatformName;

	switch (InEnumValue)
	{
		case EPlatformCategoryType::Desktop:
			PlatformName = FString("Desktop");
		break;
		case EPlatformCategoryType::Android:
			PlatformName = FString("Android");
		break;
		case EPlatformCategoryType::IOS:
			PlatformName = FString("IOS");
		break;
		case EPlatformCategoryType::Console:
			PlatformName = FString("Console");
		break;
	}

	return PlatformName;
}

FString FMaterialStatsUtils::ShaderPlatformTypeName(const EShaderPlatform PlatformID)
{
	FString FormatName = LexToString(PlatformID);
	if (FormatName.StartsWith(TEXT("SF_")))
	{
		FormatName.MidInline(3, MAX_int32, EAllowShrinking::No);
	}
	return FormatName;
}

FString FMaterialStatsUtils::GetPlatformOfflineCompilerPath(const EShaderPlatform ShaderPlatform)
{
	FExtraShaderCompilerSettings SCSettings;
	GetPlatformOfflineCompilerSettings(ShaderPlatform, SCSettings);
	return SCSettings.OfflineCompilerPath;
}

void FMaterialStatsUtils::GetPlatformOfflineCompilerSettings(const EShaderPlatform ShaderPlatform, FExtraShaderCompilerSettings& SCSettings)
{
	auto GetSCType = [](EOfflineShaderCompiler SC)
	{
		switch (SC)
		{
		case EOfflineShaderCompiler::Mali:
			return EOfflineShaderCompilerType::Mali;
		case EOfflineShaderCompiler::Adreno:
			return EOfflineShaderCompilerType::Adreno;
		default:
			return EOfflineShaderCompilerType::Num;
		}
	};
	if (FDataDrivenShaderPlatformInfo::GetNeedsOfflineCompiler(ShaderPlatform))
	{
		if (FDataDrivenShaderPlatformInfo::GetIsAndroidOpenGLES(ShaderPlatform)
			|| (FDataDrivenShaderPlatformInfo::GetIsLanguageVulkan(ShaderPlatform) && FDataDrivenShaderPlatformInfo::GetIsMobile(ShaderPlatform)))
		{
			SCSettings.OfflineCompiler = GetSCType(GetDefault<UMaterialEditorSettings>()->OfflineCompiler);
			SCSettings.OfflineCompilerPath = FPaths::ConvertRelativePathToFull(GetDefault<UMaterialEditorSettings>()->OfflineCompilerPath.FilePath);
			SCSettings.GPUTarget = GetDefault<UMaterialEditorSettings>()->GPUTarget;
			SCSettings.bDumpAll = GetDefault<UMaterialEditorSettings>()->bDumpAll;
			SCSettings.bSaveCompilerStatsFiles = GetDefault<UMaterialEditorSettings>()->bSaveCompilerStatsFiles;
			static const auto CVarMobileMultiView = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.MobileMultiView"));
			SCSettings.bMobileMultiView = (CVarMobileMultiView && CVarMobileMultiView->GetValueOnAnyThread() != 0);
			return;
		}
	}
	SCSettings.OfflineCompiler = EOfflineShaderCompilerType::Num;
	return;
}

bool FMaterialStatsUtils::IsPlatformOfflineCompilerAvailable(const EShaderPlatform ShaderPlatform)
{
	FExtraShaderCompilerSettings Settings;
	GetPlatformOfflineCompilerSettings(ShaderPlatform, Settings);

	bool bCompilerExists = FPaths::FileExists(Settings.OfflineCompilerPath);

	return bCompilerExists;
}

bool FMaterialStatsUtils::PlatformNeedsOfflineCompiler(const EShaderPlatform ShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetNeedsOfflineCompiler(ShaderPlatform);
}

FString FMaterialStatsUtils::RepresentativeShaderTypeToString(const ERepresentativeShader ShaderType)
{
	switch (ShaderType)
	{
		case ERepresentativeShader::StationarySurface:
			return TEXT("Stationary surface");

		case ERepresentativeShader::StationarySurfaceCSM:
			return TEXT("Stationary surface + CSM");

		case ERepresentativeShader::StationarySurfaceNPointLights:
			return TEXT("Stationary surface + Point Lights");

		case ERepresentativeShader::DynamicallyLitObject:
			return TEXT("Dynamically lit object");

		case ERepresentativeShader::StaticMesh:
			return TEXT("Static Mesh");

		case ERepresentativeShader::SkeletalMesh:
			return TEXT("Skeletal Mesh");

		case ERepresentativeShader::SkinnedCloth:
			return TEXT("Skinned Cloth");

		case ERepresentativeShader::NaniteMesh:
			return TEXT("Nanite Mesh");

		case ERepresentativeShader::UIDefaultFragmentShader:
			return TEXT("UI Pixel Shader");

		case ERepresentativeShader::UIDynamicFragmentShader:
			return TEXT("UI Pixel Shader (Dynamic)");

		case ERepresentativeShader::UIDefaultVertexShader:
			return TEXT("UI Vertex Shader");

		case ERepresentativeShader::UIInstancedVertexShader:
			return TEXT("UI Instanced Vertex Shader");

		case ERepresentativeShader::RuntimeVirtualTextureOutput:
			return TEXT("Runtime Virtual Texture Output");

		default:
			return TEXT("Unknown shader name");
	}
}

FSlateColor FMaterialStatsUtils::PlatformTypeColor(EPlatformCategoryType PlatformType)
{
	FSlateColor Color(FStyleColors::Foreground);

	switch (PlatformType)
	{
		case EPlatformCategoryType::Desktop:
			Color = FStyleColors::AccentBlue;
		break;
		case EPlatformCategoryType::Android:
			Color = FStyleColors::AccentGreen;
		break;
		case EPlatformCategoryType::IOS:
			Color = FStyleColors::AccentYellow;
		break;
		case EPlatformCategoryType::Console:
			Color = FStyleColors::AccentPurple;
		break;

		default:
			return Color;
	}

	return Color;
}

FSlateColor FMaterialStatsUtils::QualitySettingColor(const EMaterialQualityLevel::Type QualityType)
{
	switch (QualityType)
	{
		case EMaterialQualityLevel::Low:
			return FStyleColors::AccentGreen;
		case EMaterialQualityLevel::High:
			return FStyleColors::AccentOrange;
		case EMaterialQualityLevel::Medium:
			return FStyleColors::Warning;
		case EMaterialQualityLevel::Epic:
			return FStyleColors::Error;
		default:
			return FStyleColors::Foreground;
	}
}

void FMaterialStatsUtils::GetRepresentativeShaderTypesAndDescriptions(TMap<FName, TArray<FRepresentativeShaderInfo>>& ShaderTypeNamesAndDescriptions, const FMaterial* TargetMaterial)
{
	static const FName FLocalVertexFactoryName = FLocalVertexFactory::StaticType.GetFName();
	static const FName FGPUFactoryName = TEXT("TGPUSkinVertexFactoryDefault");
	static const FName FClothVertexFactoryName = TEXT("TGPUSkinAPEXClothVertexFactoryDefault");
	static const FName FNaniteVertexFactoryName = TEXT("FNaniteVertexFactory");

	if (TargetMaterial->IsUIMaterial())
	{
		static FName TSlateMaterialShaderPSDefaultName = TEXT("TSlateMaterialShaderPSDefault");
		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
			.Add(FRepresentativeShaderInfo(ERepresentativeShader::UIDefaultFragmentShader, TSlateMaterialShaderPSDefaultName, TEXT("Default UI Pixel Shader")));

		static FName TSlateMaterialShaderPSDynamicName = TEXT("TSlateMaterialShaderPSDynamic");
		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
		.Add(FRepresentativeShaderInfo(ERepresentativeShader::UIDynamicFragmentShader, TSlateMaterialShaderPSDynamicName, TEXT("Dynamic UI Pixel Shader")));

		static FName TSlateMaterialShaderVSfalseName = TEXT("TSlateMaterialShaderVSfalse");
		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
			.Add(FRepresentativeShaderInfo(ERepresentativeShader::UIDefaultVertexShader, TSlateMaterialShaderVSfalseName, TEXT("Default UI Vertex Shader")));

		static FName TSlateMaterialShaderVStrueName = TEXT("TSlateMaterialShaderVStrue");
		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
			.Add(FRepresentativeShaderInfo(ERepresentativeShader::UIInstancedVertexShader, TSlateMaterialShaderVStrueName, TEXT("Instanced UI Vertex Shader")));
	}
	else if (TargetMaterial->GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		static FName TBasePassPSFNoLightMapPolicyName = TEXT("TBasePassPSFNoLightMapPolicy");

		const bool bTranslucent = IsTranslucentBlendMode(*TargetMaterial);
		if (TargetMaterial->GetShadingModels().IsUnlit())
		{
			ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
				.Add(FRepresentativeShaderInfo(
						ERepresentativeShader::StationarySurface, 
						TBasePassPSFNoLightMapPolicyName, 
						TBasePassPSFNoLightMapPolicyName.ToString()));
		}
		else
		{
			ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
				.Add(FRepresentativeShaderInfo(
						ERepresentativeShader::DynamicallyLitObject, 
						TBasePassPSFNoLightMapPolicyName, 
						TBasePassPSFNoLightMapPolicyName.ToString()));

			if (IsStaticLightingAllowed())
			{
				if (TargetMaterial->IsUsedWithStaticLighting())
				{
					static FName TBasePassPSTLightMapPolicyName = TEXT("TBasePassPSTDistanceFieldShadowsAndLightMapPolicyHQ");
					ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
						.Add(FRepresentativeShaderInfo(ERepresentativeShader::StationarySurface, TBasePassPSTLightMapPolicyName, TEXT("Base pass shader with Surface Lightmap")));
				}

				static FName TBasePassPSFPrecomputedVolumetricLightmapLightingPolicyName = TEXT("TBasePassPSFPrecomputedVolumetricLightmapLightingPolicy");
				ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
					.Add(FRepresentativeShaderInfo(ERepresentativeShader::DynamicallyLitObject, TBasePassPSFPrecomputedVolumetricLightmapLightingPolicyName, TEXT("Base pass shader with Volumetric Lightmap")));
			}
		}

		static FName TBasePassVSFNoLightMapPolicyName = TEXT("TBasePassVSFNoLightMapPolicy");
		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
			.Add(FRepresentativeShaderInfo(ERepresentativeShader::StaticMesh, TBasePassVSFNoLightMapPolicyName, TEXT("Base pass vertex shader")));

		if (TargetMaterial->IsUsedWithSkeletalMesh() || TargetMaterial->IsUsedWithMorphTargets())
		{	
			ShaderTypeNamesAndDescriptions.FindOrAdd(FGPUFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::SkeletalMesh, TBasePassVSFNoLightMapPolicyName, TEXT("Base pass vertex shader")));
		}
		if (TargetMaterial->IsUsedWithAPEXCloth())
		{
			ShaderTypeNamesAndDescriptions.FindOrAdd(FClothVertexFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::SkinnedCloth, TBasePassVSFNoLightMapPolicyName, TEXT("Base pass vertex shader")));
		}

		if (TargetMaterial->GetFeatureLevel() >= ERHIFeatureLevel::SM6)
		{
			if (TargetMaterial->IsUsedWithNanite())
			{
				static FName TBasePassCSFNoLightMapPolicyName = TEXT("TBasePassCSFNoLightMapPolicy");
				ShaderTypeNamesAndDescriptions.FindOrAdd(FNaniteVertexFactoryName)
					.Add(FRepresentativeShaderInfo(ERepresentativeShader::NaniteMesh, TBasePassCSFNoLightMapPolicyName, TEXT("Nanite Compute Shader")));
			}
		}

		// Add the shader type with the most sampler usages so we can accurately report the worst case scenario.
		// This is ad-hoc, and ideally we have a better way for finding this shader type in the future.
		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
			.Add(FRepresentativeShaderInfo(ERepresentativeShader::StationarySurface, FName(TEXT("TBasePassPSFCachedVolumeIndirectLightingPolicy")), TEXT("MaxSampler")));

		// For materials that write to a runtime virtual texture add a pixel shader stat.
		if (TargetMaterial->HasRuntimeVirtualTextureOutput())
		{
			static const FName LandscapeFactoryName = TEXT("FLandscapeVertexFactory");
			static const FName RuntimeVirtualTexturePSName = TEXT("TVirtualTexturePSBaseColorNormalSpecular");

			ShaderTypeNamesAndDescriptions.FindOrAdd(LandscapeFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::RuntimeVirtualTextureOutput, FName(RuntimeVirtualTexturePSName), TEXT("Runtime Virtual Texture Output")));
		}
	}
	else
	{
		static const FName TMobileBasePassVSFNoLightMapPolicyName = TEXT("TMobileBasePassVSFNoLightMapPolicy");
		static const FName TMobileBasePassPSFNoLightMapPolicyName = TEXT("TMobileBasePassPSFNoLightMapPolicy");
		
		if (TargetMaterial->GetShadingModels().IsUnlit())
		{
			ShaderTypeNamesAndDescriptions.Add(FLocalVertexFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::StationarySurface, TMobileBasePassPSFNoLightMapPolicyName,
					TEXT("Mobile base pass shader without light map")));
		}
		else
		{			
			if (IsStaticLightingAllowed() && TargetMaterial->IsUsedWithStaticLighting())
			{
				static auto* CVarAllowDistanceFieldShadows = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AllowDistanceFieldShadows"));
				const bool bAllowDistanceFieldShadows = CVarAllowDistanceFieldShadows->GetValueOnAnyThread() != 0;
				if (bAllowDistanceFieldShadows)
				{
					// distance field shadows only shaders
					static const FName TMobileBasePassPSFMobileDistanceFieldShadowsAndLQLightMapPolicyName =
						TEXT("TMobileBasePassPSFMobileDistanceFieldShadowsAndLQLightMapPolicy");
					
					ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
							.Add(FRepresentativeShaderInfo(ERepresentativeShader::StationarySurface, TMobileBasePassPSFMobileDistanceFieldShadowsAndLQLightMapPolicyName,
								TEXT("Mobile base pass shader with distance field shadows")));
				}
				else //no shadows & lightmapped
				{
					static const FName TMobileBasePassPSTLightMapPolicyLQName =
						TEXT("TMobileBasePassPSTLightMapPolicyLQ");
										
					ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
						.Add(FRepresentativeShaderInfo(ERepresentativeShader::StationarySurface, TMobileBasePassPSTLightMapPolicyLQName,
							TEXT("Mobile base pass shader with static lighting")));
				}
			}
			// dynamically lit shader NoLightmapPolicy
			ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::DynamicallyLitObject, TMobileBasePassPSFNoLightMapPolicyName,
					TEXT("Mobile base pass shader with only dynamic lighting")));
		}

		ShaderTypeNamesAndDescriptions.FindOrAdd(FLocalVertexFactoryName)
			.Add(FRepresentativeShaderInfo(ERepresentativeShader::StaticMesh, TMobileBasePassVSFNoLightMapPolicyName,
				TEXT("Mobile base pass vertex shader")));

		if (TargetMaterial->IsUsedWithSkeletalMesh() || TargetMaterial->IsUsedWithMorphTargets())
		{
			ShaderTypeNamesAndDescriptions.FindOrAdd(FGPUFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::SkeletalMesh, TMobileBasePassVSFNoLightMapPolicyName,
					TEXT("Mobile base pass vertex shader")));
		}
		if (TargetMaterial->IsUsedWithAPEXCloth())
		{
			ShaderTypeNamesAndDescriptions.FindOrAdd(FClothVertexFactoryName)
				.Add(FRepresentativeShaderInfo(ERepresentativeShader::SkinnedCloth, TMobileBasePassVSFNoLightMapPolicyName,
					TEXT("Mobile base pass vertex shader")));
		}
	}
}

#if WITH_EDITORONLY_DATA
// Returns the specified array of shader statistics as string. Each entry begins with a newline character '\n'.
static FString GenericShaderStatsToString(const TArray<FGenericShaderStat>& Statistics)
{
	TStringBuilder<2048> StatisticsStrBuilder;
	for (const FGenericShaderStat& Stat : Statistics)
	{
		StatisticsStrBuilder << "\n" << Stat.StatName << ": ";
		Visit([&StatisticsStrBuilder](auto& StoredValue)
 		{
			StatisticsStrBuilder << StoredValue;
		}, Stat.Value);
	}

	return StatisticsStrBuilder.ToString();
}
#endif // WITH_EDITORONLY_DATA

/**
* Gets instruction counts that best represent the likely usage of this material based on shading model and other factors.
* @param Results - an array of descriptions to be populated
*/
void FMaterialStatsUtils::GetRepresentativeInstructionCounts(TArray<FShaderInstructionsInfo>& Results, const FMaterialResource* Target)
{
	TMap<FName, TArray<FRepresentativeShaderInfo>> ShaderTypeNamesAndDescriptions;
	Results.Empty();

	//when adding a shader type here be sure to update FPreviewMaterial::ShouldCache()
	//so the shader type will get compiled with preview materials
	const FMaterialShaderMap* MaterialShaderMap = Target->GetGameThreadShaderMap();
	if (MaterialShaderMap)
	{
		GetRepresentativeShaderTypesAndDescriptions(ShaderTypeNamesAndDescriptions, Target);
		TStaticArray<bool, (int32)ERepresentativeShader::Num> bShaderTypeAdded(InPlace, false);
		
		if (Target->IsUIMaterial())
		{
			//for (const TPair<FName, FRepresentativeShaderInfo>& ShaderTypePair : ShaderTypeNamesAndDescriptions)
			for (auto DescriptionPair : ShaderTypeNamesAndDescriptions)
			{
				auto& DescriptionArray = DescriptionPair.Value;
				for (int32 i = 0; i < DescriptionArray.Num(); ++i)
				{
					const FRepresentativeShaderInfo& ShaderInfo = DescriptionArray[i];
					if (!bShaderTypeAdded[(int32)ShaderInfo.ShaderType])
					{
						FShaderType* ShaderType = FindShaderTypeByName(ShaderInfo.ShaderName);
						check(ShaderType);
						const int32 NumInstructions = MaterialShaderMap->GetMaxNumInstructionsForShader(ShaderType);

						FShaderInstructionsInfo Info;
						Info.ShaderType = ShaderInfo.ShaderType;
						Info.ShaderDescription = ShaderInfo.ShaderDescription;
						Info.InstructionCount = NumInstructions;
						Info.ShaderStatisticsString = GenericShaderStatsToString(MaterialShaderMap->GetShaderStatistics(ShaderType));
						if (Info.ShaderStatisticsString.Len() == 0)
						{
							Info.ShaderStatisticsString = TEXT("n/a");
						}

						Results.Push(Info);

						bShaderTypeAdded[(int32)ShaderInfo.ShaderType] = true;
					}
				}
			}
		}
		else
		{
			for (auto DescriptionPair : ShaderTypeNamesAndDescriptions)
			{
				FVertexFactoryType* FactoryType = FindVertexFactoryType(DescriptionPair.Key);
				const FMeshMaterialShaderMap* MeshShaderMap = MaterialShaderMap->GetMeshShaderMap(FactoryType);
				if (MeshShaderMap)
				{
					TMap<FHashedName, TShaderRef<FShader>> ShaderMap;
					MeshShaderMap->GetShaderList(*MaterialShaderMap, ShaderMap);

					auto& DescriptionArray = DescriptionPair.Value;

					for (int32 i = 0; i < DescriptionArray.Num(); ++i)
					{
						const FRepresentativeShaderInfo& ShaderInfo = DescriptionArray[i];
						if (!bShaderTypeAdded[(int32)ShaderInfo.ShaderType])
						{
							TShaderRef<FShader>* ShaderEntry = ShaderMap.Find(ShaderInfo.ShaderName);
							if (ShaderEntry != nullptr)
							{
								FShaderType* ShaderType = (*ShaderEntry).GetType();
								{
									const int32 NumInstructions = MeshShaderMap->GetMaxNumInstructionsForShader(*MaterialShaderMap, ShaderType);

									FShaderInstructionsInfo Info;
									Info.ShaderType = ShaderInfo.ShaderType;
									Info.ShaderDescription = ShaderInfo.ShaderDescription;
									Info.InstructionCount = NumInstructions;
#if WITH_EDITORONLY_DATA
									Info.ShaderStatisticsString = GenericShaderStatsToString(MeshShaderMap->GetShaderStatistics(*MaterialShaderMap, ShaderType));
#endif // WITH_EDITORONLY_DATA
									if (Info.ShaderStatisticsString.Len() == 0)
									{
										Info.ShaderStatisticsString = TEXT("n/a");
									}

									Results.Push(Info);

									bShaderTypeAdded[(int32)ShaderInfo.ShaderType] = true;
								}
							}
						}
					}
				}
			}
		}
	}
}

void FMaterialStatsUtils::ExtractMatertialStatsInfo(EShaderPlatform ShaderPlatform, FShaderStatsInfo& OutInfo, const FMaterialResource* MaterialResource)
{
	// extract potential errors
	const ERHIFeatureLevel::Type MaterialFeatureLevel = MaterialResource->GetFeatureLevel();
	FString FeatureLevelName;
	GetFeatureLevelName(MaterialFeatureLevel, FeatureLevelName);

	OutInfo.Empty();
	TArray<FString> CompileErrors = MaterialResource->GetCompileErrors();
	for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
	{
		OutInfo.StrShaderErrors += FString::Printf(TEXT("[%s] %s\n"), *FeatureLevelName, *CompileErrors[ErrorIndex]);
	}

	bool bNoErrors = OutInfo.StrShaderErrors.Len() == 0;

	const FMaterialShaderMap* MaterialShaderMap = MaterialResource->GetGameThreadShaderMap();
	if (!bNoErrors || !MaterialShaderMap)
	{
		return;
	}

	// extract instructions info
	TArray<FMaterialStatsUtils::FShaderInstructionsInfo> ShaderInstructionInfo;
	GetRepresentativeInstructionCounts(ShaderInstructionInfo, MaterialResource);

	for (int32 InstructionIndex = 0; InstructionIndex < ShaderInstructionInfo.Num(); InstructionIndex++)
	{
		FShaderStatsInfo::FContent Content;

		Content.StrDescription = ShaderInstructionInfo[InstructionIndex].InstructionCount > 0 ? FString::Printf(TEXT("%u"), ShaderInstructionInfo[InstructionIndex].InstructionCount) : TEXT("n/a");
		Content.StrDescriptionLong = ShaderInstructionInfo[InstructionIndex].InstructionCount > 0 ?
			FString::Printf(TEXT("%s: %u instructions\nStats: %s"), *ShaderInstructionInfo[InstructionIndex].ShaderDescription, ShaderInstructionInfo[InstructionIndex].InstructionCount, *ShaderInstructionInfo[InstructionIndex].ShaderStatisticsString) :
			TEXT("Offline shader compiler not available or an error was encountered!");

		OutInfo.ShaderInstructionCount.Add(ShaderInstructionInfo[InstructionIndex].ShaderType, Content);

		FString Description = ShaderInstructionInfo[InstructionIndex].ShaderStatisticsString;
		FShaderStatsInfo::FContent GenericContent;
		GenericContent.StrDescription = Description;
		GenericContent.StrDescriptionLong = Description;
		OutInfo.GenericShaderStatistics.Add(ShaderInstructionInfo[InstructionIndex].ShaderType, GenericContent);
	}

	// extract samplers info
	const int32 SamplersUsed = FMath::Max(MaterialResource->GetSamplerUsage(), 0);
	const int32 MaxSamplers = GetExpectedFeatureLevelMaxTextureSamplers(MaterialResource->GetFeatureLevel());
	OutInfo.SamplersCount.StrDescription = FString::Printf(TEXT("%u/%u"), SamplersUsed, MaxSamplers);
	OutInfo.SamplersCount.StrDescriptionLong = FString::Printf(TEXT("%s samplers: %u/%u"), TEXT("Texture"), SamplersUsed, MaxSamplers);

	// extract estimated sample info
	uint32 NumVSTextureSamples = 0, NumPSTextureSamples = 0;
	MaterialResource->GetEstimatedNumTextureSamples(NumVSTextureSamples, NumPSTextureSamples);

	OutInfo.TextureSampleCount.StrDescription = FString::Printf(TEXT("VS(%u), PS(%u)"), NumVSTextureSamples, NumPSTextureSamples);
	OutInfo.TextureSampleCount.StrDescriptionLong = FString::Printf(TEXT("Texture Lookups (Est.): Vertex(%u), Pixel(%u)"), NumVSTextureSamples, NumPSTextureSamples);

	// extract estimated VT info
	const uint32 NumVirtualTextureLookups = MaterialResource->GetEstimatedNumVirtualTextureLookups();
	OutInfo.VirtualTextureLookupCount.StrDescription = FString::Printf(TEXT("%u"), NumVirtualTextureLookups);
	OutInfo.VirtualTextureLookupCount.StrDescriptionLong = FString::Printf(TEXT("Virtual Texture Lookups (Est.): %u"), NumVirtualTextureLookups);

	const uint32 NumVirtualTextureStacks = MaterialResource->GetNumVirtualTextureStacks();
	OutInfo.VirtualTextureStackCount.StrDescription = FString::Printf(TEXT("%u"), NumVirtualTextureStacks);
	OutInfo.VirtualTextureStackCount.StrDescriptionLong = FString::Printf(TEXT("Virtual Texture Stacks (Est.): %u"), NumVirtualTextureStacks);

	// extract interpolators info
	uint32 UVScalarsUsed, CustomInterpolatorScalarsUsed;
	MaterialResource->GetUserInterpolatorUsage(UVScalarsUsed, CustomInterpolatorScalarsUsed);

	const uint32 TotalScalars = UVScalarsUsed + CustomInterpolatorScalarsUsed;
	const uint32 MaxScalars = FMath::DivideAndRoundUp(TotalScalars, 4u) * 4;

	OutInfo.InterpolatorsCount.StrDescription = FString::Printf(TEXT("%u/%u"), TotalScalars, MaxScalars);
	OutInfo.InterpolatorsCount.StrDescriptionLong = FString::Printf(TEXT("User interpolators: %u/%u Scalars (%u/4 Vectors) (TexCoords: %i, Custom: %i)"),
		TotalScalars, MaxScalars, MaxScalars / 4, UVScalarsUsed, CustomInterpolatorScalarsUsed);

	// Extract total shader count w/o having to compile shaders.
	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(nullptr);
		

	const FMaterialCompilationOutput& CompilationOutput = MaterialShaderMap->GetCompilationOutput();
	TArray<FDebugShaderTypeInfo> OutShaderInfo;
	MaterialResource->GetShaderTypes(LayoutParams, OutShaderInfo, FCompilationShaderParameters(MaterialResource, CompilationOutput));

	int TotalShadersForMaterial = 0;
	for (const FDebugShaderTypeInfo& ShaderInfo : OutShaderInfo)
	{
		TotalShadersForMaterial += ShaderInfo.ShaderTypes.Num();

		for (const FDebugShaderPipelineInfo& PipelineInfo : ShaderInfo.Pipelines)
		{
			TotalShadersForMaterial += PipelineInfo.ShaderTypes.Num();
		}
	}

	OutInfo.ShaderCount.StrDescription = FString::Printf(TEXT("%u"), TotalShadersForMaterial);
	OutInfo.ShaderCount.StrDescriptionLong = FString::Printf(TEXT("Total Shaders: %u"), TotalShadersForMaterial);

	FString LWCMessage;
	FMaterialResource::FLWCUsagesArray LWCFuncUsagesVS;
	FMaterialResource::FLWCUsagesArray LWCFuncUsagesPS;
	FMaterialResource::FLWCUsagesArray LWCFuncUsagesCS;
	MaterialResource->GetEstimatedLWCFuncUsages(LWCFuncUsagesVS, LWCFuncUsagesPS, LWCFuncUsagesCS);
	for (int KindIndex = 0; KindIndex < (int)ELWCFunctionKind::Max; ++KindIndex)
	{
		int Usages = LWCFuncUsagesVS[KindIndex] + LWCFuncUsagesPS[KindIndex] + LWCFuncUsagesCS[KindIndex];
		if (Usages > 0)
		{
			LWCMessage += FString::Printf(
				TEXT("%s: %u (VS), %u (PS), %u (CS)\n"), 
				*UEnum::GetDisplayValueAsText((ELWCFunctionKind)KindIndex).ToString(), 
				LWCFuncUsagesVS[KindIndex],
				LWCFuncUsagesPS[KindIndex],
				LWCFuncUsagesCS[KindIndex]);
		}
	}

	OutInfo.LWCUsage.StrDescription = LWCMessage;
	OutInfo.LWCUsage.StrDescriptionLong = LWCMessage;

	if (FMaterialShaderMap* ShaderMap = MaterialResource->GetGameThreadShaderMap())
	{
		// Add number of preshaders and stats
		uint32 TotalParams, TotalOps;
		MaterialResource->GetPreshaderStats(TotalParams, TotalOps);
		OutInfo.PreShaderCount.StrDescription = FString::Printf(TEXT("%u outputs\n%u params\n%u ops"), ShaderMap->GetNumPreshaders(), TotalParams, TotalOps);
		OutInfo.PreShaderCount.StrDescriptionLong = FString::Printf(TEXT("%u outputs, %u parameter fetches, %u total operations"), ShaderMap->GetNumPreshaders(), TotalParams, TotalOps);
	}
}

/*end FMaterialStatsUtils */
/***********************************************************************************************************************/
