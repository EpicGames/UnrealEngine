// Copyright Epic Games, Inc. All Rights Reserved.
//

#include "MaterialStatsGrid.h"
#include "MaterialStats.h"
#include "MaterialStatsCommon.h"

/*==============================================================================================================*/
/* FGridRow functions*/

void FStatsGridRow::AddCell(FName ColumnName, TSharedPtr<FGridCell> Cell)
{
	RowCells.Add(ColumnName, Cell);
}

void FStatsGridRow::RemoveCell(FName ColumnName)
{
	RowCells.Remove(ColumnName);
}

TSharedPtr<FGridCell> FStatsGridRow::GetCell(const FName ColumnName)
{
	TSharedPtr<FGridCell> RetCell = nullptr;

	auto* CellPtr = RowCells.Find(ColumnName);

	if (CellPtr != nullptr)
	{
		RetCell = *CellPtr;
	}

	return RetCell;
}

void FStatsGridRow::RemovePlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	RemoveCell(ColumnName);
}

void FStatsGridRow::FillPlatformCellsHelper(TSharedPtr<FMaterialStats> StatsManager)
{
	auto& PlatformDB = StatsManager->GetPlatformsDB();
	for (auto Pair : PlatformDB)
	{
		auto Platform = Pair.Value;
		if (!Platform->IsPresentInGrid())
		{
			continue;
		}

		for (int32 q = 0; q < (int32)EMaterialQualityLevel::Num; ++q)
		{
			EMaterialQualityLevel::Type QualityLevel = (EMaterialQualityLevel::Type)q;

			if (StatsManager->GetStatsQualityFlag(QualityLevel))
			{
				for (int32 InstanceIndex = 0; InstanceIndex < Platform->GetPlatformData(QualityLevel).Instances.Num(); ++InstanceIndex)
				{
					// call implementation specific function to build the needed cell
					AddPlatform(StatsManager, Platform, QualityLevel, InstanceIndex);
				}
			}
		}
	}
}

/* end FGridRow functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_Empty functions*/

void FStatsGridRow_Empty::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// just an array of empty cells
	AddCell(FMaterialStatsGrid::DescriptorColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_Empty::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	TSharedPtr<FGridCell_Empty> Cell = MakeShareable(new FGridCell_Empty());

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/* end FStatsGridRow_Empty functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_Name functions*/

void FStatsGridRow_Name::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// we don't use a descriptor for this row
	AddCell(FMaterialStatsGrid::DescriptorColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_Name::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// add a cell that will query any available errors for this platform
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::Name, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));
	Cell->SetHorizontalAlignment(HAlign_Fill);

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_Name functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_Quality functions*/

void FStatsGridRow_Quality::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// we don't use a descriptor for this row
	AddCell(FMaterialStatsGrid::DescriptorColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_Quality::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// translate material quality to string and store it inside a StaticString cell
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_StaticString> Cell = MakeShareable(new FGridCell_StaticString(CellContent, CellContent));

	Cell->SetContentBold(true);
	FSlateColor CellColor = FMaterialStatsUtils::QualitySettingColor(QualityLevel);
	Cell->SetColor(CellColor);

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_Quality functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_Shaders functions*/

FStatsGridRow_Shaders::FStatsGridRow_Shaders(ERepresentativeShader RepresentativeShader, bool bHeader, bool bInstructionRow)
	: bIsHeaderRow(bHeader)
	, bInstructionRow(bInstructionRow)
	, ShaderType(RepresentativeShader)
{
}

FStatsGridRow_Shaders::EShaderClass FStatsGridRow_Shaders::GetShaderClass(const ERepresentativeShader Shader)
{
	switch (Shader)
	{
		case ERepresentativeShader::StationarySurface:
		case ERepresentativeShader::StationarySurfaceCSM:
		case ERepresentativeShader::StationarySurfaceNPointLights:
		case ERepresentativeShader::DynamicallyLitObject:
		case ERepresentativeShader::UIDefaultFragmentShader:
		case ERepresentativeShader::RuntimeVirtualTextureOutput:
			return EShaderClass::FragmentShader;

		case ERepresentativeShader::StaticMesh:
		case ERepresentativeShader::SkeletalMesh:
		case ERepresentativeShader::UIDefaultVertexShader:
		case ERepresentativeShader::UIInstancedVertexShader:
			return EShaderClass::VertexShader;

		case ERepresentativeShader::NaniteMesh:
			return EShaderClass::ComputeShader;
	}

	return EShaderClass::VertexShader;
}

void FStatsGridRow_Shaders::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	TSharedPtr<FGridCell> HeaderCell;

	// in the first row of this type add a "Vertex/Fragment Shader" static text
	if (bIsHeaderRow)
	{
		EShaderClass ShaderClass = GetShaderClass(ShaderType);

		FString HeaderContent;
		if (ShaderClass == EShaderClass::VertexShader)
		{
			HeaderContent = TEXT("Vertex Shader");
		}
		else if (ShaderClass == EShaderClass::FragmentShader)
		{
			HeaderContent = TEXT("Pixel Shader");
		}
		else if (ShaderClass == EShaderClass::ComputeShader)
		{
			HeaderContent = TEXT("Compute Shader");
		}
		
		HeaderCell = MakeShareable(new FGridCell_StaticString(HeaderContent, HeaderContent));
		HeaderCell->SetContentBold(true);
		HeaderCell->SetColor(FStyleColors::Foreground);
	}
	else
	{
		HeaderCell = MakeShareable(new FGridCell_Empty());
	}

	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	// now add a cell that can display the name of this shader's class
	FString ShaderColumnContent = FMaterialStatsUtils::RepresentativeShaderTypeToString(ShaderType);
	TSharedPtr<FGridCell> ShaderNameCell = MakeShareable(new FGridCell_StaticString(ShaderColumnContent, ShaderColumnContent));
	ShaderNameCell->SetHorizontalAlignment(HAlign_Fill);
	ShaderNameCell->SetContentBold(true);
	ShaderNameCell->SetColor(FStyleColors::Foreground);
	AddCell(FMaterialStatsGrid::ShaderColumnName, ShaderNameCell);

	// now add a cell that can display the what statistic we are describing.
	FString ColumnContent = bInstructionRow ? TEXT("Instruction Count") : TEXT("Platform Statistics");
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_StaticString(ColumnContent, ColumnContent)));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_Shaders::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// add a cell that display the instruction count for this platform
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, bInstructionRow ? EShaderInfoType::InstructionsCount : EShaderInfoType::GenericShaderStatistics, ShaderType, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_Shaders functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_Samplers functions*/

void FStatsGridRow_Samplers::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static text in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("Samplers"), TEXT("Texture Samplers")));
	HeaderCell->SetColor(FLinearColor::Gray);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_Samplers::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the total number of samplers in this material
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::SamplersCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_Samplers functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_Interpolators functions*/

void FStatsGridRow_Interpolators::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("Interpolators"), TEXT("Interpolators")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_Interpolators::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the total number of interpolators in this material
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::InterpolatorsCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_Interpolators functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_NumTextureSamples functions*/

void FStatsGridRow_NumTextureSamples::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("Texture Lookups (Est.)"), TEXT("Texture Lookups (Est.)")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_NumTextureSamples::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the total number of texture samples in this material
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::TextureSampleCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_NumTextureSamples functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_NumVirtualTextureLookups functions*/

void FStatsGridRow_NumVirtualTextureLookups::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("VT Lookups (Est.)"), TEXT("Virtual Texture Lookups (Est.)")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_NumVirtualTextureLookups::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the total number of texture samples in this material
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::VirtualTextureLookupCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_NumVirtualTextureLookups functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_NumVirtualTextureStacks functions*/

void FStatsGridRow_NumVirtualTextureStacks::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("VT Stacks (Est.)"), TEXT("Virtual Texture Stacks (Est.)")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_NumVirtualTextureStacks::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the total number of texture samples in this material
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::VirtualTextureStackCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_NumVirtualTextureStacks functions*/

/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_NumShaders functions*/

void FStatsGridRow_NumShaders::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("Shader Count"), TEXT("Number of shaders compiled for this platform and quality level.")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_NumShaders::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the number of shaders
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::ShaderCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_NumTotalShaders functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_NumPreshaders functions*/

void FStatsGridRow_NumPreshaders::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("Preshader Count"), TEXT("Number of preshader instructions that will be evaluated on the CPU for this material.")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_NumPreshaders::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the number of pre shaders
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::PreShaderCount, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_NumPreshaders functions*/
/*==============================================================================================================*/

/*==============================================================================================================*/
/* FStatsGridRow_LWCUsage functions*/

void FStatsGridRow_LWCUsage::CreateRow(TSharedPtr<FMaterialStats> StatsManager)
{
	// static string in the descriptor column
	TSharedPtr<FGridCell> HeaderCell = MakeShareable(new FGridCell_StaticString(TEXT("Large World Coordinate Usages (Est.)"), TEXT("Estimates for the number of large world coordinate operations in the material.")));
	HeaderCell->SetColor(FStyleColors::Foreground);
	HeaderCell->SetContentBold(true);
	AddCell(FMaterialStatsGrid::DescriptorColumnName, HeaderCell);

	AddCell(FMaterialStatsGrid::ShaderColumnName, MakeShareable(new FGridCell_Empty()));
	AddCell(FMaterialStatsGrid::ShaderStatisticColumnName, MakeShareable(new FGridCell_Empty()));

	FillPlatformCellsHelper(StatsManager);
}

void FStatsGridRow_LWCUsage::AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	// cell that will enumerate the number of pre shaders
	const FString CellContent = FMaterialStatsUtils::MaterialQualityToShortString(QualityLevel);
	TSharedPtr<FGridCell_ShaderValue> Cell = MakeShareable(new FGridCell_ShaderValue(StatsManager, EShaderInfoType::LWCUsage, ERepresentativeShader::Num, QualityLevel, Platform->GetPlatformShaderType(), InstanceIndex));

	const FName ColumnName = FMaterialStatsGrid::MakePlatformColumnName(Platform, QualityLevel, InstanceIndex);
	AddCell(ColumnName, Cell);
}

/*end FStatsGridRow_LWCUsage functions*/
/*==============================================================================================================*/

/***********************************************************************************************************************/
/*FShaderStatsGrid functions*/

const FName FMaterialStatsGrid::DescriptorColumnName = TEXT("Descriptor");
const FName FMaterialStatsGrid::ShaderColumnName = TEXT("ShaderList");
const FName FMaterialStatsGrid::ShaderStatisticColumnName = TEXT("ShaderStatistics");

FMaterialStatsGrid::FMaterialStatsGrid(TWeakPtr<FMaterialStats> _StatsManager)
{
	StatsManagerWPtr = _StatsManager;
}

FMaterialStatsGrid::~FMaterialStatsGrid()
{
	StaticRows.Empty();
	VertexShaderRows.Empty();
	FragmentShaderRows.Empty();
	GridColumnContent.Empty();
}

TSharedPtr<FGridCell> FMaterialStatsGrid::GetCell(int32 RowID, FName ColumnName)
{
	// if there is no such row return a newly created cell with empty content
	ERowType RowType;
	int32 Index;
	DissasambleRowKey(RowType, Index, RowID);

	if (RowType == ERowType::FragmentShader && Index >= 0 && Index < FragmentShaderRows.Num())
	{
		return FragmentShaderRows[Index]->GetCell(ColumnName);
	}
	else if (RowType == ERowType::VertexShader && Index >= 0 && Index < VertexShaderRows.Num())
	{
		return VertexShaderRows[Index]->GetCell(ColumnName);
	}
	else
	{
		auto* RowPtr = StaticRows.Find(RowType);
		if (RowPtr != nullptr)
		{
			return (*RowPtr)->GetCell(ColumnName);
		}
	}

	return MakeShareable(new FGridCell_Empty());
}

void FMaterialStatsGrid::CollectShaderInfo(const TSharedPtr<FShaderPlatformSettings>& PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	const FShaderPlatformSettings::FPlatformData& PlatformData = PlatformPtr->GetPlatformData(QualityLevel);

	for (int32 i = 0; i < (int32)ERepresentativeShader::Num; ++i)
	{
		UsedShaders[i] |= PlatformData.Instances[InstanceIndex].ShaderStatsInfo.ShaderInstructionCount.Contains((ERepresentativeShader)i);
	}
}

void FMaterialStatsGrid::CollectShaderInfo()
{
	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
	{
		return;
	}

	for (int32 i = 0; i < UsedShaders.Num(); ++i)
	{
		UsedShaders[i] = false;
	}

	auto& PlatformDB = StatsManager->GetPlatformsDB();
	for (auto Pair : PlatformDB)
	{
		auto Platform = Pair.Value;

		if (!Platform->IsPresentInGrid())
		{
			continue;
		}

		for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
		{
			EMaterialQualityLevel::Type QualitySetting = static_cast<EMaterialQualityLevel::Type>(q);

			for (int32 InstanceIndex = 0; InstanceIndex < Platform->GetPlatformData(QualitySetting).Instances.Num(); ++InstanceIndex)
			{
				CollectShaderInfo(Platform, QualitySetting, InstanceIndex);
			}
		}
	}
}

FString FMaterialStatsGrid::GetColumnContent(const FName ColumnName) const
{
	auto* Ptr = GridColumnContent.Find(ColumnName);

	return Ptr != nullptr ? Ptr->Content : TEXT("");
}

FString FMaterialStatsGrid::GetColumnContentLong(const FName ColumnName) const
{
	auto* Ptr = GridColumnContent.Find(ColumnName);

	return Ptr != nullptr ? Ptr->ContentLong : TEXT("");
}

FSlateColor FMaterialStatsGrid::GetColumnColor(const FName ColumnName) const
{
	auto* Ptr = GridColumnContent.Find(ColumnName);

	return Ptr != nullptr ? Ptr->Color : FLinearColor::Gray;
}

void FMaterialStatsGrid::BuildKeyAndInsert(const ERowType RowType, int16 Index /*= 0*/)
{
	int32 Key = AssembleRowKey(RowType, Index);
	RowIDs.Add(MakeShareable(new int32(Key)));
}

void FMaterialStatsGrid::CheckForErrors()
{
	PlatformErrorsType = EGlobalErrorsType::NoErrors;

	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (StatsManager.IsValid())
	{
		auto& PlatformDB = StatsManager->GetPlatformsDB();
		for (auto Pair : PlatformDB)
		{
			auto Platform = Pair.Value;

			if (!Platform->IsPresentInGrid())
			{
				continue;
			}

			for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
			{
				EMaterialQualityLevel::Type QualitySetting = static_cast<EMaterialQualityLevel::Type>(q);
				if (StatsManager->GetStatsQualityFlag(QualitySetting))
				{
					for (int32 InstanceIndex = 0; InstanceIndex < Platform->GetPlatformData(QualitySetting).Instances.Num(); ++InstanceIndex)
					{
						auto &Data = Platform->GetInstanceData(QualitySetting, InstanceIndex);

						if (Data.ShaderStatsInfo.HasErrors())
						{
							PlatformErrorsType = PlatformErrorsType == EGlobalErrorsType::SpecificPlatformErrors ? EGlobalErrorsType::SpecificPlatformErrors : EGlobalErrorsType::GlobalPlatformErrors;
						}
						else
						{
							PlatformErrorsType = PlatformErrorsType == EGlobalErrorsType::NoErrors ? EGlobalErrorsType::NoErrors : EGlobalErrorsType::SpecificPlatformErrors;
						}
					}
				}
			}
		}
	}
}

void FMaterialStatsGrid::BuildRowIds()
{
	RowIDs.Reset();

	BuildKeyAndInsert(ERowType::Name);
	BuildKeyAndInsert(ERowType::Quality);

	// add the rest of the rows only if there's at least one error free platform
	if (PlatformErrorsType != EGlobalErrorsType::GlobalPlatformErrors)
	{
		for (int32 i = 0; i < FragmentShaderRows.Num(); ++i)
		{
			BuildKeyAndInsert(ERowType::FragmentShader, (int16)i);
		}

		if (FragmentShaderRows.Num())
		{
			BuildKeyAndInsert(ERowType::Empty);
		}

		for (int32 i = 0; i < VertexShaderRows.Num(); ++i)
		{
			BuildKeyAndInsert(ERowType::VertexShader, (int16)i);
		}

		if (VertexShaderRows.Num())
		{
			BuildKeyAndInsert(ERowType::Empty);
		}

		BuildKeyAndInsert(ERowType::Samplers);
		BuildKeyAndInsert(ERowType::TextureSamples);
		BuildKeyAndInsert(ERowType::VirtualTextureLookups);
		BuildKeyAndInsert(ERowType::VirtualTextureStacks);
		BuildKeyAndInsert(ERowType::Interpolators);
		BuildKeyAndInsert(ERowType::Shaders);
		BuildKeyAndInsert(ERowType::PreShaders);
		BuildKeyAndInsert(ERowType::LWCUsage);
	}
}

void FMaterialStatsGrid::OnShaderChanged()
{
	CollectShaderInfo();

	BuildShaderRows();
	BuildColumnInfo();
	CheckForErrors();

	BuildRowIds();
}

void FMaterialStatsGrid::OnColumnNumChanged()
{
	// reset all rows to rebuild columns
	BuildStaticRows();
	OnShaderChanged();
}

void FMaterialStatsGrid::BuildStaticRows()
{
	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
	{
		return;
	}

	StaticRows.Reset();
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_Empty());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::Empty, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_Name());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::Name, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_Quality());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::Quality, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_Samplers());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::Samplers, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_NumTextureSamples());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::TextureSamples, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_NumVirtualTextureLookups());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::VirtualTextureLookups, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_NumVirtualTextureStacks());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::VirtualTextureStacks, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_Interpolators());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::Interpolators, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_NumShaders());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::Shaders, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_NumPreshaders());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::PreShaders, Row);
	}
	{
		TSharedPtr<FStatsGridRow> Row = MakeShareable(new FStatsGridRow_LWCUsage());
		Row->CreateRow(StatsManager);
		StaticRows.Add(ERowType::LWCUsage, Row);
	}
}

void FMaterialStatsGrid::BuildShaderRows()
{
	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
	{
		return;
	}

	VertexShaderRows.Empty();
	FragmentShaderRows.Empty();

	// fragment shaders
	for (auto i = (int32)ERepresentativeShader::FirstFragmentShader; i <= (int32)ERepresentativeShader::LastFragmentShader; ++i)
	{
		if (UsedShaders[i])
		{
			bool bFirstShader = !FragmentShaderRows.Num();

			TSharedPtr<FStatsGridRow> FragShaderRow = MakeShareable(new FStatsGridRow_Shaders((ERepresentativeShader)i, bFirstShader, true));
			FragShaderRow->CreateRow(StatsManager);

			FragmentShaderRows.Add(FragShaderRow);

			TSharedPtr<FStatsGridRow> FragShaderRow2 = MakeShareable(new FStatsGridRow_Shaders((ERepresentativeShader)i, false, false));
			FragShaderRow2->CreateRow(StatsManager);

			FragmentShaderRows.Add(FragShaderRow2);
		}
	}

	// vertex shaders
	for (auto i = (int32)ERepresentativeShader::FirstVertexShader; i <= (int32)ERepresentativeShader::LastVertexShader; ++i)
	{
		if (UsedShaders[i])
		{
			bool bFirstShader = !VertexShaderRows.Num();

			TSharedPtr<FStatsGridRow> VertShaderRow = MakeShareable(new FStatsGridRow_Shaders((ERepresentativeShader)i, bFirstShader, true));
			VertShaderRow->CreateRow(StatsManager);

			VertexShaderRows.Add(VertShaderRow);

			TSharedPtr<FStatsGridRow> VertShaderRow2 = MakeShareable(new FStatsGridRow_Shaders((ERepresentativeShader)i, false, false));
			VertShaderRow2->CreateRow(StatsManager);

			VertexShaderRows.Add(VertShaderRow2);			
		}
	}
}

void FMaterialStatsGrid::BuildColumnInfo()
{
	GridColumnContent.Empty();
	GridColumnContent.Add(DescriptorColumnName, FColumnInfo());
	GridColumnContent.Add(ShaderColumnName, FColumnInfo());
	GridColumnContent.Add(ShaderStatisticColumnName, FColumnInfo());

	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
	{
		return;
	}

	const auto& PlatformDB = StatsManager->GetPlatformsDB();
	for (auto PlatformPair : PlatformDB)
	{
		if (!PlatformPair.Value->IsPresentInGrid())
		{
			continue;
		}

		for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
		{
			auto QualityLevel = static_cast<EMaterialQualityLevel::Type>(q);

			if (!StatsManager->GetStatsQualityFlag(QualityLevel))
			{
				continue;
			}

			const int32 NumInstances = PlatformPair.Value->GetPlatformData(QualityLevel).Instances.Num();
			// skip all derived MI's if not asked to show stats as errors are shown separately
			const int32 IterateForNum = StatsManager->GetMaterialStatsDerivedMIOption() == EMaterialStatsDerivedMIOption::ShowStats ? NumInstances : FMath::Min(1, NumInstances);  

			for (int32 InstanceIndex = 0; InstanceIndex < IterateForNum; ++InstanceIndex)
			{
				AddColumnInfo(PlatformPair.Value, QualityLevel, InstanceIndex);
			}
		}
	}
}

void FMaterialStatsGrid::BuildGrid()
{
	CollectShaderInfo();

	BuildStaticRows();
	BuildShaderRows();
	BuildColumnInfo();
	CheckForErrors();

	BuildRowIds();
}

void FMaterialStatsGrid::AddColumnInfo(TSharedPtr<FShaderPlatformSettings> PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	FColumnInfo Info;

	Info.Color = FMaterialStatsUtils::PlatformTypeColor(PlatformPtr->GetCategoryType());
	Info.Content = PlatformPtr->GetPlatformName().ToString();
	Info.ContentLong = PlatformPtr->GetPlatformDescription();

	const FName ColumnName = MakePlatformColumnName(PlatformPtr, QualityLevel, InstanceIndex);
	GridColumnContent.Add(ColumnName, Info);
}

void FMaterialStatsGrid::RemoveColumnInfo(TSharedPtr<FShaderPlatformSettings> PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	const FName ColumnName = MakePlatformColumnName(PlatformPtr, QualityLevel, InstanceIndex);
	GridColumnContent.Remove(ColumnName);
}

void FMaterialStatsGrid::AddOrRemovePlatform(TSharedPtr<FShaderPlatformSettings> PlatformPtr, const bool bAdd, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex)
{
	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
	{
		return;
	}

	// update column record
	if (bAdd)
	{
		AddColumnInfo(PlatformPtr, QualityLevel, InstanceIndex);
	}
	else
	{
		RemoveColumnInfo(PlatformPtr, QualityLevel, InstanceIndex);
	}

	for (auto RowPair : StaticRows)
	{
		if (bAdd)
		{
			RowPair.Value->AddPlatform(StatsManager, PlatformPtr, QualityLevel, InstanceIndex);
		}
		else
		{
			RowPair.Value->RemovePlatform(StatsManager, PlatformPtr, QualityLevel, InstanceIndex);
		}
	}

	for (int32 i = 0; i < VertexShaderRows.Num(); ++i)
	{
		if (bAdd)
		{
			VertexShaderRows[i]->AddPlatform(StatsManager, PlatformPtr, QualityLevel, InstanceIndex);
		}
		else
		{
			VertexShaderRows[i]->RemovePlatform(StatsManager, PlatformPtr, QualityLevel, InstanceIndex);
		}
	}

	for (int32 i = 0; i < FragmentShaderRows.Num(); ++i)
	{
		if (bAdd)
		{
			FragmentShaderRows[i]->AddPlatform(StatsManager, PlatformPtr, QualityLevel, InstanceIndex);
		}
		else
		{
			FragmentShaderRows[i]->RemovePlatform(StatsManager, PlatformPtr, QualityLevel, InstanceIndex);
		}
	}
}

void FMaterialStatsGrid::OnAddOrRemovePlatform(TSharedPtr<FShaderPlatformSettings> PlatformPtr)
{
	bool bAdded = PlatformPtr->IsPresentInGrid();

	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
	{
		return;
	}

	for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
	{
		auto QualityLevel = static_cast<EMaterialQualityLevel::Type>(q);

		if (!StatsManager->GetStatsQualityFlag(QualityLevel))
		{
			continue;
		}

		for (int32 InstanceIndex = 0; InstanceIndex < PlatformPtr->GetPlatformData(QualityLevel).Instances.Num(); ++InstanceIndex)
		{
			AddOrRemovePlatform(PlatformPtr, bAdded, QualityLevel, InstanceIndex);
		}
	}

	// recheck shader rows in case something changed
	OnShaderChanged();
}

void FMaterialStatsGrid::OnQualitySettingChanged(const EMaterialQualityLevel::Type QualityLevel)
{
	TSharedPtr<FMaterialStats> StatsManager = StatsManagerWPtr.Pin();
	if (!StatsManager.IsValid())
		return;

	const bool bQualityOn = StatsManager->GetStatsQualityFlag(QualityLevel);

	const auto& PlatformDB = StatsManager->GetPlatformsDB();
	for (auto PlatformPair : PlatformDB)
	{
		if (PlatformPair.Value->IsPresentInGrid())
		{
			for (int32 InstanceIndex = 0; InstanceIndex < PlatformPair.Value->GetPlatformData(QualityLevel).Instances.Num(); ++InstanceIndex)
			{
				AddOrRemovePlatform(PlatformPair.Value, bQualityOn, QualityLevel, InstanceIndex);
			}
		}
	}

	// recheck shader rows in case something changed
	OnShaderChanged();
}

FName FMaterialStatsGrid::MakePlatformColumnName(const TSharedPtr<FShaderPlatformSettings>& Platform, const EMaterialQualityLevel::Type Quality, const int32 InstanceIndex)
{
	FName RetName = *(Platform->GetPlatformID().ToString() + "_" + FMaterialStatsUtils::MaterialQualityToString(Quality) + "_" + FString::FromInt(InstanceIndex));
	return RetName;
}

/*end FShaderStatsGrid functions*/
/***********************************************************************************************************************/

/***********************************************************************************************************************/
/* FGridCell functions */

FGridCell::FGridCell()
{
	CellColor = FStyleColors::Foreground;
}

FString FGridCell_Empty::GetCellContent()
{
	return TEXT("");
}

FString FGridCell_Empty::GetCellContentLong()
{
	return TEXT("");
}

FGridCell_StaticString::FGridCell_StaticString(const FString& _Content, const FString& _ContentLong)
{
	Content = _Content;
	ContentLong = _ContentLong;
}

FString FGridCell_StaticString::GetCellContent()
{
	return Content;
}

FString FGridCell_StaticString::GetCellContentLong()
{
	return ContentLong;
}

FGridCell_ShaderValue::FGridCell_ShaderValue(const TWeakPtr<FMaterialStats>& _MaterialStatsWPtr, const EShaderInfoType _InfoType, const ERepresentativeShader _ShaderType,
	const EMaterialQualityLevel::Type _QualityLevel, const EShaderPlatform _PlatformType, const int32 _InstanceIndex)
{
	MaterialStatsWPtr = _MaterialStatsWPtr;

	InfoType = _InfoType;
	ShaderType = _ShaderType;
	QualityLevel = _QualityLevel;
	PlatformType = _PlatformType;
	InstanceIndex = _InstanceIndex;
}

FString FGridCell_ShaderValue::InternalGetContent(bool bLongContent)
{
	auto MaterialStats = MaterialStatsWPtr.Pin();
	if (!MaterialStats.IsValid())
	{
		return TEXT("");
	}

	auto Platform = MaterialStats->GetPlatformSettings(PlatformType);
	if (!Platform.IsValid() || Platform->GetPlatformData(QualityLevel).Instances.Num() <= InstanceIndex)
	{
		return TEXT("");
	}

	const auto& InstanceData = Platform->GetInstanceData(QualityLevel, InstanceIndex);

	switch (InfoType)
	{
		case EShaderInfoType::Name:
			return MaterialStats->GetMaterialName(InstanceIndex);

		case EShaderInfoType::InstructionsCount:
		{
			auto* Count = InstanceData.ShaderStatsInfo.ShaderInstructionCount.Find(ShaderType);
			if (Count)
			{
				return bLongContent ? Count->StrDescriptionLong : Count->StrDescription;
			}
		}
		break;

		case EShaderInfoType::GenericShaderStatistics:
		{
			auto* Count = InstanceData.ShaderStatsInfo.GenericShaderStatistics.Find(ShaderType);
			if (Count)
			{
				return bLongContent ? Count->StrDescriptionLong : Count->StrDescription;
			}
		}
		break;

		case EShaderInfoType::InterpolatorsCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.InterpolatorsCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.InterpolatorsCount.StrDescription;

		case EShaderInfoType::TextureSampleCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.TextureSampleCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.TextureSampleCount.StrDescription;

		case EShaderInfoType::SamplersCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.SamplersCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.SamplersCount.StrDescription;

		case EShaderInfoType::VirtualTextureLookupCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.VirtualTextureLookupCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.VirtualTextureLookupCount.StrDescription;

		case EShaderInfoType::VirtualTextureStackCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.VirtualTextureStackCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.VirtualTextureStackCount.StrDescription;

		case EShaderInfoType::ShaderCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.ShaderCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.ShaderCount.StrDescription;

		case EShaderInfoType::PreShaderCount:
			return bLongContent ? InstanceData.ShaderStatsInfo.PreShaderCount.StrDescriptionLong : InstanceData.ShaderStatsInfo.PreShaderCount.StrDescription;

		case EShaderInfoType::LWCUsage:
			return bLongContent ? InstanceData.ShaderStatsInfo.LWCUsage.StrDescriptionLong : InstanceData.ShaderStatsInfo.LWCUsage.StrDescription;
	}

	return TEXT("");
}

FString FGridCell_ShaderValue::GetCellContent()
{
	return InternalGetContent(false);
}

FString FGridCell_ShaderValue::GetCellContentLong()
{
	return InternalGetContent(true);
}

FGridCell::EIcon FGridCell_ShaderValue::GetIcon() const
{
	auto MaterialStats = MaterialStatsWPtr.Pin();
	if (!MaterialStats.IsValid())
	{
		return EIcon::None;
	}

	auto Platform = MaterialStats->GetPlatformSettings(PlatformType);
	if (!Platform.IsValid() || Platform->GetPlatformData(QualityLevel).Instances.Num() <= InstanceIndex)
	{
		return EIcon::None;
	}

	const auto& InstanceData = Platform->GetInstanceData(QualityLevel, InstanceIndex);
	switch (InfoType)
	{
		case EShaderInfoType::Name:
			return InstanceData.MaterialResourcesStats->GetCompileErrors().Num() > 0 ? EIcon::Error : EIcon::None;
		default:
			return EIcon::None;
	}
}
