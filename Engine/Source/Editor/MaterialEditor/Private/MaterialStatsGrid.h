// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineGlobals.h"
#include "SceneTypes.h"
#include "Types/SlateEnums.h"
#include "RHIDefinitions.h"
#include "MaterialStatsCommon.h"
#include "Containers/StaticArray.h"
#include "MaterialStats.h"
#include "Styling/StyleColors.h"

/** class to represent a single cell inside the material stats grid */
class FGridCell
{
public:
	enum class EIcon
	{
		None,
		Error
	};

protected:
	/** attributes used at display time to configure widgets */
	FSlateColor CellColor;
	bool bBoldContent = false;

	EHorizontalAlignment HAlignment = EHorizontalAlignment::HAlign_Center;
	EVerticalAlignment VAlignment = EVerticalAlignment::VAlign_Center;

public:
	FGridCell();
	virtual ~FGridCell() = default;

	/** returns the main content of this cell */
	virtual FString GetCellContent() = 0;
	/** this can be used for tool tips or other detailed descriptions */
	virtual FString GetCellContentLong() = 0;

	FORCEINLINE FSlateColor GetColor() const;
	FORCEINLINE void SetColor(const FSlateColor& Color);

	FORCEINLINE bool IsContentBold() const;
	FORCEINLINE void SetContentBold(bool bValue);

	virtual EHorizontalAlignment GetHorizontalAlignment();
	FORCEINLINE void SetHorizontalAlignment(EHorizontalAlignment Align);

	FORCEINLINE EVerticalAlignment GetVerticalAlignment() const;
	FORCEINLINE void SetVerticalAlignment(EVerticalAlignment Align);

	virtual EIcon GetIcon() const
	{
		return EIcon::None;
	}
};

/** this time of cell with just return an empty string and its mainly used to separate rows */
class FGridCell_Empty : public FGridCell
{
public:
	FString GetCellContent() override;
	FString GetCellContentLong() override;
};

/** cell that stores & returns a simple static string */
class FGridCell_StaticString : public FGridCell
{
	FString Content;
	FString ContentLong;

public:
	FGridCell_StaticString(const FString& _Content, const FString& _ContentLong);

	FString GetCellContent() override;
	FString GetCellContentLong() override;
};

/** enumeration used to classify arguments for FGridCell_ShaderValue */
enum class EShaderInfoType
{
	Name,
	InstructionsCount,
	SamplersCount,
	InterpolatorsCount,
	TextureSampleCount,
	VirtualTextureLookupCount,
	VirtualTextureStackCount,
	ShaderCount,
	PreShaderCount,
	LWCUsage,
	GenericShaderStatistics,
};

/** this type of cell will query certain type of informations from the material */
class FGridCell_ShaderValue : public FGridCell
{
private:
	TWeakPtr<class FMaterialStats> MaterialStatsWPtr;

	EShaderInfoType InfoType;
	ERepresentativeShader ShaderType;
	EMaterialQualityLevel::Type QualityLevel;
	EShaderPlatform PlatformType;
	int32 InstanceIndex;

	FString InternalGetContent(bool bLongContent);

public:
	FGridCell_ShaderValue(const TWeakPtr<FMaterialStats>& _MaterialStatsWPtr, const EShaderInfoType _InfoType, const ERepresentativeShader _ShaderType,
		const EMaterialQualityLevel::Type _QualityLevel, const EShaderPlatform _PlatformType, const int32 _InstanceIndex);

	FString GetCellContent() override;
	FString GetCellContentLong() override;
	EIcon GetIcon() const override;
};

/** virtual class to model grid row generation */
class FStatsGridRow
{
protected:
	/** key is column name */
	TMap<FName, TSharedPtr<FGridCell>> RowCells;

protected:
	void AddCell(FName ColumnName, TSharedPtr<FGridCell> Cell);
	void RemoveCell(FName ColumnName);

	/** helper function that will loop through all platforms present in the grid and attempt to build their columns by calling AddPlatform() function */
	void FillPlatformCellsHelper(TSharedPtr<FMaterialStats> StatsManager);

public:
	FStatsGridRow() {}
	virtual ~FStatsGridRow() = default;

	/** this function should generate all needed cells  */
	virtual void CreateRow(TSharedPtr<FMaterialStats> StatsManager) = 0;

	/** Add/RemovePlatforms should be called when a platform is added or removed from the grid */
	virtual void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) = 0;
	virtual void RemovePlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);

	TSharedPtr<FGridCell> GetCell(const FName ColumnName);

	void RemoveAll()
	{
		RowCells.Empty();
	}
};

/** separator row */
class FStatsGridRow_Empty : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** material/instance name */
class FStatsGridRow_Name : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** row that will produce static string from EMaterialQualityLevel::Type */
class FStatsGridRow_Quality : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** row that will extract and the number of instructions for each used shader */
class FStatsGridRow_Shaders : public FStatsGridRow
{
public:
	enum class EShaderClass
	{
		VertexShader,
		FragmentShader,
		ComputeShader
	};

private:
	// if this is true it will add a text in the 'description' column with 'fragment/vertex shader' text
	bool bIsHeaderRow = false;

	bool bInstructionRow = true;

	//EShaderType ShaderType;
	ERepresentativeShader ShaderType;
private:
	EShaderClass GetShaderClass(const ERepresentativeShader Shader);

public:
	FStatsGridRow_Shaders(ERepresentativeShader RepresentativeShader, bool bHeader, bool bInstructionRow);

	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the global number of samplers present in the material for a specified platform */
class FStatsGridRow_Samplers : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the global number of interpolators present in the material for a specified platform */
class FStatsGridRow_Interpolators : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the global number of texture samples present in the material for a specified platform */
class FStatsGridRow_NumTextureSamples : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the global number of virtual texture lookups present in the material for a specified platform */
class FStatsGridRow_NumVirtualTextureLookups : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the global number of virtual texture stacks present in the material for a specified platform */
class FStatsGridRow_NumVirtualTextureStacks : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the total number of shaders present in the material for a specified platform */
class FStatsGridRow_NumShaders : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the total number of shaders present in the material for a specified platform */
class FStatsGridRow_NumPreshaders : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** this row will display the LWC usage in the material for a specified platform */
class FStatsGridRow_LWCUsage : public FStatsGridRow
{
public:
	void CreateRow(TSharedPtr<FMaterialStats> StatsManager) override;

	void AddPlatform(TSharedPtr<FMaterialStats> StatsManager, const TSharedPtr<FShaderPlatformSettings> Platform, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex) override;
};

/** class that models the logical material stats grid */
class FMaterialStatsGrid
{
	/** enum used whether any shader platform reported errors */
	enum class EGlobalErrorsType
	{
		// no errors at all
		NoErrors,
		// there are some platform specific errors
		SpecificPlatformErrors,
		// all platforms have errors
		GlobalPlatformErrors
	};

	/** enum used to differentiate between various row types */
	enum class ERowType
	{
		Empty,
		Name,
		Quality,
		Samplers,
		Interpolators,
		TextureSamples,
		VirtualTextureLookups,
		VirtualTextureStacks,
		Shaders,
		PreShaders,
		LWCUsage,

		VertexShader,
		FragmentShader,
	};

	/** collection of row object that will not change as shader platforms are added or removed */
	TMap<ERowType, TSharedPtr<FStatsGridRow>> StaticRows;
	/** array of shader columns that vary with the number of shaders present in each analyzed material */
	TArray<TSharedPtr<FStatsGridRow>> VertexShaderRows;
	TArray<TSharedPtr<FStatsGridRow>> FragmentShaderRows;

	/** this structure will held additional informations about the columns of this grid, needed at display time */
	struct FColumnInfo
	{
		FString Content = TEXT("");
		FString ContentLong = TEXT("");
		FSlateColor Color = FStyleColors::Foreground;
	};

	/** collection of column information sorted by their names */
	TMap<FName, FColumnInfo> GridColumnContent;

	/** array feed into a SListView used by GridStatsWidget
	*  each entry is a pointer to the id of each row inside the grid
	*  the ids are assembled/disassabled with AssembleRowKey()/DissasambleRowKey()
	*/
	TArray<TSharedPtr<int32>> RowIDs;

	/** pointer to stats manager who will actually create an instance of this class */
	TWeakPtr<class FMaterialStats> StatsManagerWPtr;

	/** helper array that mark the presence or absence of each type of shader (ERepresentativeShader) in the analyzed material */
	TStaticArray<bool, (int32)ERepresentativeShader::Num> UsedShaders;

	/** variable used to indicate the presence of errors in any of the analyzed shader platforms */
	/** this will be updated every time a shader is compiled */
	EGlobalErrorsType PlatformErrorsType;

public:
	const static FName DescriptorColumnName;
	const static FName ShaderColumnName;
	const static FName ShaderStatisticColumnName;

private:
	void AddColumnInfo(TSharedPtr<FShaderPlatformSettings> PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);
	void RemoveColumnInfo(TSharedPtr<FShaderPlatformSettings> PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);

	/** this function will go through all available shader platforms and will call build the content of GridColumnContent */
	void BuildColumnInfo();

	/** setup functions that will perform the actual row building */
	/** BuildShaderRows will be called after each shader compilation */
	void BuildShaderRows();
	void BuildStaticRows();

	void CheckForErrors();

	/** this functions will build the content of UsedShaders array */
	void CollectShaderInfo(const TSharedPtr<FShaderPlatformSettings>& PlatformPtr, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);
	void CollectShaderInfo();

	/** functions that build the row ids array used by GridStatsWidget to identify available rows inside this logical grid */
	void BuildKeyAndInsert(const ERowType RowType, int16 Index = 0);
	void BuildRowIds();

	/** helper functions to create an ID for each row */
	FORCEINLINE int32 AssembleRowKey(const ERowType RowType, const int16 Index);
	FORCEINLINE void DissasambleRowKey(ERowType& RowType, int32& Index, const int32 Key);

	void AddOrRemovePlatform(TSharedPtr<FShaderPlatformSettings> PlatformPtr, const bool bAdd, const EMaterialQualityLevel::Type QualityLevel, const int32 InstanceIndex);

public:
	FMaterialStatsGrid(TWeakPtr<FMaterialStats> _StatsManager);
	~FMaterialStatsGrid();

	TSharedPtr<FGridCell> GetCell(int32 RowID, FName ColumnName);

	FORCEINLINE const TArray<TSharedPtr<int32>>* GetGridRowIDs() const;

	FORCEINLINE TArray<FName> GetVisibleColumnNames() const;

	void OnShaderChanged();
	void OnColumnNumChanged();

	void OnAddOrRemovePlatform(TSharedPtr<FShaderPlatformSettings> PlatformPtr);
	void OnQualitySettingChanged(const EMaterialQualityLevel::Type QualityLevel);

	/** call to build the content of this grid */
	void BuildGrid();

	FString GetColumnContent(const FName ColumnName) const;
	FString GetColumnContentLong(const FName ColumnName) const;
	FSlateColor GetColumnColor(const FName ColumnName) const;

	/** helper function that will assemble a column name from the given arguments */
	static FName MakePlatformColumnName(const TSharedPtr<FShaderPlatformSettings>& Platform, const EMaterialQualityLevel::Type Quality, const int32 InstanceIndex);
};

//////////////////////////////////////////////////////////////////////////////////////////////////
// FShaderStatsGrid implementation
FORCEINLINE const TArray<TSharedPtr<int32>> *FMaterialStatsGrid::GetGridRowIDs() const
{
	return &RowIDs;
}

FORCEINLINE int32 FMaterialStatsGrid::AssembleRowKey(const ERowType RowType, const int16 Index)
{
	int32 Key = ((int32)Index << 16) | (int32)RowType;
	return Key;
}

FORCEINLINE void FMaterialStatsGrid::DissasambleRowKey(ERowType& RowType, int32& Index, const int32 Key)
{
	RowType = (ERowType)(Key & 0xffff);
	Index = Key >> 16;
}

FORCEINLINE TArray<FName> FMaterialStatsGrid::GetVisibleColumnNames() const
{
	TArray<FName> ColumnList;
	GridColumnContent.GenerateKeyArray(ColumnList);

	return ColumnList;
}
//////////////////////////////////////////////////////////////////////////////////////////////////

FORCEINLINE FSlateColor FGridCell::GetColor() const
{
	return CellColor;
}

FORCEINLINE void FGridCell::SetColor(const FSlateColor& Color)
{
	CellColor = Color;
}

FORCEINLINE bool FGridCell::IsContentBold() const
{
	return bBoldContent;
}

FORCEINLINE void FGridCell::SetContentBold(bool bValue)
{
	bBoldContent = bValue;
}

FORCEINLINE EHorizontalAlignment FGridCell::GetHorizontalAlignment()
{
	return HAlignment;
}

FORCEINLINE void FGridCell::SetHorizontalAlignment(EHorizontalAlignment Align)
{
	HAlignment = Align;
}

FORCEINLINE EVerticalAlignment FGridCell::GetVerticalAlignment() const
{
	return VAlignment;
}

FORCEINLINE void FGridCell::SetVerticalAlignment(EVerticalAlignment Align)
{
	VAlignment = Align;
}
