// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialEditorCustomPrimitiveDataWidget.h"
#include "Widgets/Layout/SScrollBox.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraph.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SToolTip.h"
#include "IMaterialEditor.h"

#define LOCTEXT_NAMESPACE "SMaterialEditorCustomPrimitiveDataWidget"

class SCustomPrimitiveDataRow : public SMultiColumnTableRow<TSharedPtr<FCustomPrimitiveDataRowData>>
{
public:
	SLATE_BEGIN_ARGS(SCustomPrimitiveDataRow) {}
		SLATE_ARGUMENT(TSharedPtr<FCustomPrimitiveDataRowData>, Entry)
		SLATE_EVENT(FSimpleDelegate, OnValueChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		RowData = Args._Entry;
		OnValueChangedDelegate = Args._OnValueChanged;

		FSuperRowType::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedRef<STextBlock> TextBlock = SNew(STextBlock);
		if (RowData->bIsDuplicate)
		{
			TextBlock->SetToolTip(SNew(SToolTip).Text(FText::FromString("This slot is potentially incorrectly overlapping")).BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground")));
			TextBlock->SetColorAndOpacity(FSlateColor(FLinearColor::Red));
		}

		if (ColumnName == "Slot")
		{
			TextBlock->SetText(FText::FromString(FString::FormatAsNumber(RowData->Slot)));
			return TextBlock;
		}
		else if (ColumnName == "Name")
		{
			TextBlock->SetText(FText::FromString(RowData->Name));
			return TextBlock;
		}
		else if (ColumnName == "PreviewValue")
		{
			TSharedRef<SSpinBox<float>> SpinBox = SNew(SSpinBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.IsEnabled(RowData->bIsDuplicate == false)
				.Value_Lambda([this]() { return RowData->PreviewValue; })
				.OnValueChanged_Lambda([this](float InValue)
				{
					RowData->PreviewValue = InValue;
					OnValueChangedDelegate.ExecuteIfBound();
				}
			);

			return SpinBox;
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FCustomPrimitiveDataRowData> RowData;
	FSimpleDelegate OnValueChangedDelegate;
};

void SMaterialCustomPrimitiveDataPanel::Refresh()
{
	if (MaterialEditorInstance && MaterialEditorInstance->PreviewMaterial)
	{
		const TArray<float> CustomFloats = MakeCustomPrimitiveFloatData();

		Items.Empty();

		TArray<UMaterialExpressionScalarParameter*> Scalars;
		MaterialEditorInstance->PreviewMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionScalarParameter>(Scalars);

		for (const UMaterialExpressionScalarParameter* Expr : Scalars)
		{
			if (Expr->bUseCustomPrimitiveData)
			{
				const int32 SlotIndex = (int32)Expr->PrimitiveDataIndex;
				const float PreviewValue = CustomFloats.IsValidIndex(SlotIndex) ? CustomFloats[SlotIndex] : 0.0f;
				Items.Emplace(MakeShared<FCustomPrimitiveDataRowData>(SlotIndex, Expr->GetParameterName().ToString(), PreviewValue));
			}
		}

		TArray<UMaterialExpressionVectorParameter*> Vectors;
		MaterialEditorInstance->PreviewMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionVectorParameter>(Vectors);

		for (const UMaterialExpressionVectorParameter* Expr : Vectors)
		{
			if (Expr->bUseCustomPrimitiveData)
			{
				TArray<FString> Elements({".r", ".g" , ".b",  ".a" });
				for (int32 i = 0; i < 4; i++)
				{
					const int32 SlotIndex = (int32)Expr->PrimitiveDataIndex + i;
					const float PreviewValue = CustomFloats.IsValidIndex(SlotIndex) ? CustomFloats[SlotIndex] : 0.0f;
					Items.Emplace(MakeShared<FCustomPrimitiveDataRowData>(SlotIndex, Expr->GetParameterName().ToString() + Elements[i], PreviewValue));
				}
			}
		}

		// Sort the items
		Items.StableSort([](const TSharedPtr<FCustomPrimitiveDataRowData>& A, const TSharedPtr<FCustomPrimitiveDataRowData>& B) {
			int32 SlotA = A->Slot;
			int32 SlotB = B->Slot;
			if (SlotA == SlotB)
			{
				return A->Name < B->Name;
			}
			return SlotA < SlotB;
			});

		// Mark items that are in duplicate slots
		for ( int32 i=0; i < Items.Num() - 1; ++i )
		{
			if (Items[i]->Slot == Items[i + 1]->Slot)
			{
				Items[i]->bIsDuplicate = true;
				Items[i + 1]->bIsDuplicate = true;
			}
		}

		RefreshCustomPrimitiveData();
	}

	ListViewWidget->RequestListRefresh();
}

void SMaterialCustomPrimitiveDataPanel::RefreshCustomPrimitiveData()
{
	if (MaterialEditor)
	{
		TArray<float> CustomFloats;
		if (bUsePreviewValues)
		{
			CustomFloats = MakeCustomPrimitiveFloatData();
		}

		MaterialEditor->SetCustomPrimitiveData(CustomFloats);
	}
}

TArray<float> SMaterialCustomPrimitiveDataPanel::MakeCustomPrimitiveFloatData() const
{
	TArray<float> CustomFloats;

	int32 SlotMax = 0;
	for (const TSharedPtr<FCustomPrimitiveDataRowData>& Item : Items)
	{
		SlotMax = FMath::Max(SlotMax, Item->Slot);
	}

	CustomFloats.AddZeroed(SlotMax + 1);
	for (const TSharedPtr<FCustomPrimitiveDataRowData>& Item : Items)
	{
		CustomFloats[Item->Slot] = Item->PreviewValue;
	}
	return CustomFloats;
}

void SMaterialCustomPrimitiveDataPanel::Construct(const FArguments& InArgs, IMaterialEditor* InMaterialEditor, UMaterialEditorPreviewParameters* InMaterialEditorInstance)
{	
	this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered"))
				.Padding(FMargin(4.0f))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(FMargin(3.0f, 4.0f))
						.HAlign(HAlign_Left)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Custom Primitive Data Parameters"))
							.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							.ShadowOffset(FVector2D(1.0f, 1.0f))
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.Padding(FMargin(8.0f, 4.0f, 3.0f, 4.0f))
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([this]() { return bUsePreviewValues ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
							{
								bUsePreviewValues = (NewState == ECheckBoxState::Checked);
								RefreshCustomPrimitiveData();
							})
							[
								SNew(STextBlock)
								.Text(FText::FromString("Use Preview Values"))
								.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							]
						]
					]
					+ SVerticalBox::Slot()
					.Padding(FMargin(3.0f, 2.0f, 3.0f, 3.0f))
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.OnMouseButtonDown(this, &SMaterialCustomPrimitiveDataPanel::OnMouseButtonDown)
						[
							SNew(SScrollBox)
							+ SScrollBox::Slot()
							[
								SAssignNew(ListViewWidget, SListView<TSharedPtr<FCustomPrimitiveDataRowData>>)
								.ListItemsSource(&Items)
								.OnGenerateRow(this, &SMaterialCustomPrimitiveDataPanel::OnGenerateRowForList)
								.SelectionMode(ESelectionMode::None)
								.HeaderRow
								(
									SAssignNew(HeaderRow, SHeaderRow)
									+ SHeaderRow::Column("Slot").DefaultLabel(FText::FromString("Slot")).ManualWidth(48.0f)
									+ SHeaderRow::Column("Name").DefaultLabel(FText::FromString("Name"))
									+ SHeaderRow::Column("PreviewValue").DefaultLabel(FText::FromString("Preview Value"))
								)
							]
						]
					]
				]
			]
		];

	MaterialEditor = InMaterialEditor;
	MaterialEditorInstance = InMaterialEditorInstance;
	Refresh();
}

TSharedRef<ITableRow> SMaterialCustomPrimitiveDataPanel::OnGenerateRowForList(TSharedPtr<FCustomPrimitiveDataRowData> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCustomPrimitiveDataRow, OwnerTable)
		.Entry(Item)
		.OnValueChanged(FSimpleDelegate::CreateSP(this, &SMaterialCustomPrimitiveDataPanel::RefreshCustomPrimitiveData));
}

FReply SMaterialCustomPrimitiveDataPanel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("PreviewValues", "Preview Values"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PreviewValues_Clear", "Clear Values"),
				LOCTEXT("PreviewValues_ClearTooltip", "Clears all the preview values back to 0.0"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SMaterialCustomPrimitiveDataPanel::ClearPreviewValues))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("PreviewValues_CopyDefaults", "Copy Defaults to Values"),
				LOCTEXT("PreviewValues_CopyDefaultsTooltip", "Copies the default values from the material nodes to the preview values"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SMaterialCustomPrimitiveDataPanel::CopyDefaultToPreviewValues))
			);
		}
		MenuBuilder.EndSection();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			*MouseEvent.GetEventPath(),
			MenuBuilder.MakeWidget(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SMaterialCustomPrimitiveDataPanel::ClearPreviewValues()
{
	for (const TSharedPtr<FCustomPrimitiveDataRowData>& Item : Items)
	{
		Item->PreviewValue = 0.0f;
	}

	RefreshCustomPrimitiveData();
}

void SMaterialCustomPrimitiveDataPanel::CopyDefaultToPreviewValues()
{
	TMap<int32, float> DefaultValues;

	if (MaterialEditorInstance && MaterialEditorInstance->PreviewMaterial)
	{
		// Scalars
		{
			TArray<UMaterialExpressionScalarParameter*> Scalars;
			MaterialEditorInstance->PreviewMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionScalarParameter>(Scalars);

			for (const UMaterialExpressionScalarParameter* Expr : Scalars)
			{
				if (Expr->bUseCustomPrimitiveData)
				{
					DefaultValues.FindOrAdd((int32)Expr->PrimitiveDataIndex, Expr->DefaultValue);
				}
			}
		}

		// Vectors
		{
			TArray<UMaterialExpressionVectorParameter*> Vectors;
			MaterialEditorInstance->PreviewMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionVectorParameter>(Vectors);

			for (const UMaterialExpressionVectorParameter* Expr : Vectors)
			{
				if (Expr->bUseCustomPrimitiveData)
				{
					for (int32 i = 0; i < 4; i++)
					{
						DefaultValues.FindOrAdd((int32)Expr->PrimitiveDataIndex + i, Expr->DefaultValue.Component(i));
					}
				}
			}
		}
	}

	for (const TSharedPtr<FCustomPrimitiveDataRowData>& Item : Items)
	{
		Item->PreviewValue = DefaultValues.FindRef(Item->Slot);
	}

	RefreshCustomPrimitiveData();
}

#undef LOCTEXT_NAMESPACE
