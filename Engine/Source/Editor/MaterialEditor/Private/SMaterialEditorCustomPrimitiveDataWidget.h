// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Materials/Material.h"
#include "IDetailTreeNode.h"
#include "IDetailPropertyRow.h"
#include "MaterialPropertyHelpers.h"

class IMaterialEditor;
class UMaterialEditorPreviewParameters;

/** Data for a row in the list of custom primitive data entries*/
struct FCustomPrimitiveDataRowData
{
	FCustomPrimitiveDataRowData(int32 InSlot, FString InName, float InPreviewValue, bool bInIsDuplicate = false) : Slot(InSlot), Name(InName), PreviewValue(InPreviewValue), bIsDuplicate(bInIsDuplicate) {}

	int32 Slot = 0;
	FString Name = "";
	float PreviewValue = 0.0f;
	bool bIsDuplicate = false;
};

class SMaterialCustomPrimitiveDataPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialCustomPrimitiveDataPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, IMaterialEditor* MaterialEditor, UMaterialEditorPreviewParameters* InMaterialEditorInstance);

	void SetEditorInstance(UMaterialEditorPreviewParameters* InMaterialEditorInstance) { MaterialEditorInstance = InMaterialEditorInstance; Refresh(); }

private:

	void Refresh();

	void RefreshCustomPrimitiveData();

	TArray<float> MakeCustomPrimitiveFloatData() const;

	/** Adds a new textbox with the string to the list */
	TSharedRef<ITableRow> OnGenerateRowForList(TSharedPtr<FCustomPrimitiveDataRowData> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called when a mouse button is down in the table area */
	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Clears all the preview values */
	void ClearPreviewValues();

	/** Copies all the default node values to the preview values. */
	void CopyDefaultToPreviewValues();

	/** The list of strings */
	TArray<TSharedPtr<FCustomPrimitiveDataRowData>> Items;

	/** The actual UI list */
	TSharedPtr<SListView<TSharedPtr<FCustomPrimitiveDataRowData>>> ListViewWidget;

	TSharedPtr<SHeaderRow> HeaderRow;

	/** When enabled the preview values will be set as custom primitive data. */
	bool bUsePreviewValues = true;

	/** Pointer to the material editor interface */
	IMaterialEditor* MaterialEditor = nullptr;

	/** The set of material parameters this is associated with */
	UMaterialEditorPreviewParameters* MaterialEditorInstance;
};