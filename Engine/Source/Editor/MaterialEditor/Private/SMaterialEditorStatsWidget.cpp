// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialEditorStatsWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Fonts/FontMeasure.h"
#include "Styling/AppStyle.h"
#include "MaterialStatsGrid.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "MaterialEditorStatsWidget"

const float SMaterialEditorStatsWidget::ColumnSizeSmall = 100.0f;
const float SMaterialEditorStatsWidget::ColumnSizeMedium = 150.0f;
const float SMaterialEditorStatsWidget::ColumnSizeLarge = 200.0f;
const float SMaterialEditorStatsWidget::ColumnSizeExtraLarge = 400.0f;

const FName SMaterialEditorStatsWidget::RegularFontStyle = TEXT("DataTableEditor.CellText");
const FName SMaterialEditorStatsWidget::BoldFontStyle = TEXT("RichTextBlock.Bold");

class SMaterialStatsViewRow : public SMultiColumnTableRow<TSharedPtr<int32>>
{
public:
	SLATE_BEGIN_ARGS(SMaterialStatsViewRow) {}
		/** The list item for this row */
		SLATE_ARGUMENT(TSharedPtr<int32>, PtrRowID)
		SLATE_ARGUMENT(TWeakPtr<FMaterialStats>, MaterialStatsWPtr)
	SLATE_END_ARGS()

	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		PtrRowID = InArgs._PtrRowID;
		MaterialStatsWPtr = InArgs._MaterialStatsWPtr;

		SMultiColumnTableRow<TSharedPtr<int32>>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "TableView.Row"),
			InOwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FText GetTextForCell(const FName Name, const bool bToolTip) const;
	static const FSlateBrush* GetIconForCell(const FGridCell::EIcon Icon);
	const FSlateBrush* GetIconForCell(const FName Name) const;
	FSlateColor GetColorForCell(const FName Name) const;
	EHorizontalAlignment GetHAlignForCell(const FName Name) const;
	EVerticalAlignment GetVAlignForCell(const FName Name) const;

private:
	/** The item associated with this row of data */
	TSharedPtr<int32> PtrRowID;

	TWeakPtr<FMaterialStats> MaterialStatsWPtr;
};

TSharedRef<SWidget> SMaterialStatsViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	EHorizontalAlignment HAlign = EHorizontalAlignment::HAlign_Fill;
	EVerticalAlignment VALign = EVerticalAlignment::VAlign_Top;

	FName UsedFontStyle = SMaterialEditorStatsWidget::GetRegularFontStyleName();

	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid() && PtrRowID.IsValid())
	{
		const int32 RowID = *PtrRowID;

		auto Cell = StatsPtr->GetStatsGrid()->GetCell(RowID, ColumnName);

		UsedFontStyle = Cell->IsContentBold() ? SMaterialEditorStatsWidget::GetBoldFontStyleName() : SMaterialEditorStatsWidget::GetRegularFontStyleName();
		HAlign = Cell->GetHorizontalAlignment();
		VALign = Cell->GetVerticalAlignment();
	}

	return SNew(SBox)
		.Padding(FMargin(4, 2, 4, 2))
		.HAlign(HAlign)
		.VAlign(VALign)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SMaterialStatsViewRow::GetIconForCell, ColumnName)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), UsedFontStyle)
				.Text(this, &SMaterialStatsViewRow::GetTextForCell, ColumnName, false)
				.ToolTipText(this, &SMaterialStatsViewRow::GetTextForCell, ColumnName, true)
				.AutoWrapText(true)
			]
		];
}

FText SMaterialStatsViewRow::GetTextForCell(const FName Name, const bool bToolTip) const
{
	FString CellContent;

	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid() && PtrRowID.IsValid())
	{
		const int32 RowID = *PtrRowID;

		auto Cell = StatsPtr->GetStatsGrid()->GetCell(RowID, Name);
		if (Cell.IsValid())
		{
			CellContent = bToolTip ? Cell->GetCellContentLong() : Cell->GetCellContent();
		}
	}

	FText FinalText = CellContent.Len() > 0 ? FText::FromString(CellContent) : FText::FromString(TEXT(""));
	return FinalText;
}

const FSlateBrush* SMaterialStatsViewRow::GetIconForCell(const FGridCell::EIcon Icon)
{
	switch(Icon)
	{
	case FGridCell::EIcon::Error:
		return FAppStyle::GetBrush(TEXT("MessageLog.Error"));
	default:
		return nullptr;
	}
}

const FSlateBrush* SMaterialStatsViewRow::GetIconForCell(const FName Name) const
{
	FGridCell::EIcon CellContent = FGridCell::EIcon::None;

	const auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid() && PtrRowID.IsValid())
	{
		const int32 RowID = *PtrRowID;

		const auto Cell = StatsPtr->GetStatsGrid()->GetCell(RowID, Name);
		if (Cell.IsValid())
		{
			CellContent = Cell->GetIcon();
		}
	}

	return GetIconForCell(CellContent);
}

FSlateColor SMaterialStatsViewRow::GetColorForCell(const FName Name) const
{
	return FStyleColors::Foreground;
}

EHorizontalAlignment SMaterialStatsViewRow::GetHAlignForCell(const FName Name) const
{
	EHorizontalAlignment Align = HAlign_Center;

	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid() && PtrRowID.IsValid())
	{
		const int32 RowID = *PtrRowID;

		const auto Cell = StatsPtr->GetStatsGrid()->GetCell(RowID, Name);

		Align = Cell->GetHorizontalAlignment();
	}

	return Align;
}

EVerticalAlignment SMaterialStatsViewRow::GetVAlignForCell(const FName Name) const
{
	EVerticalAlignment Align = VAlign_Center;

	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid() && PtrRowID.IsValid())
	{
		const int32 RowID = *PtrRowID;

		const auto Cell = StatsPtr->GetStatsGrid()->GetCell(RowID, Name);

		Align = Cell->GetVerticalAlignment();
	}

	return Align;
}

float SMaterialEditorStatsWidget::GetColumnSize(const FName ColumnName) const
{
	float ColumnSize = ColumnSizeSmall;

	if (ColumnName == FMaterialStatsGrid::DescriptorColumnName)
	{
		ColumnSize = ColumnSizeMedium;
	}
	else if (ColumnName == FMaterialStatsGrid::ShaderColumnName)
	{
		ColumnSize = ColumnSizeLarge;
	}
	else if (ColumnName == FMaterialStatsGrid::ShaderStatisticColumnName)
	{
		ColumnSize = ColumnSizeMedium;
	}
	else
	{
		auto StatsPtr = MaterialStatsWPtr.Pin();
		if (StatsPtr.IsValid())
		{
			TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FSlateFontInfo FontInfo = FAppStyle::GetFontStyle(BoldFontStyle);

			auto ArrRowIds = *StatsPtr->GetStatsGrid()->GetGridRowIDs();

			for (int32 i = 0; i < ArrRowIds.Num(); ++i)
			{
				if (ArrRowIds[i].IsValid())
				{
					const auto Cell = StatsPtr->GetStatsGrid()->GetCell(*ArrRowIds[i], ColumnName);

					const FString Content = Cell->GetCellContent();
					const FSlateBrush* Icon = SMaterialStatsViewRow::GetIconForCell(Cell->GetIcon());

					FVector2D FontMeasure = FontMeasureService->Measure(Content, FontInfo);
					const float IconSize = Icon != nullptr ? Icon->GetImageSize().X : 0.0f;

					ColumnSize = FMath::Clamp(FontMeasure.X + IconSize, ColumnSize, ColumnSizeExtraLarge);
				}
			}
		}
	}

	return ColumnSize;
}

SHeaderRow::FColumn::FArguments SMaterialEditorStatsWidget::CreateColumnArgs(const FName ColumnName)
{
	FSlateColor Color = FStyleColors::AccentGreen;
	FString Content;
	FString ContentLong;
	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid())
	{
		Content = StatsPtr->GetStatsGrid()->GetColumnContent(ColumnName);
		ContentLong = StatsPtr->GetStatsGrid()->GetColumnContentLong(ColumnName);
		Color = StatsPtr->GetStatsGrid()->GetColumnColor(ColumnName);
	}

	const auto ColumnArgs = SHeaderRow::Column(ColumnName)
		.DefaultLabel(FText::FromString(Content))
		.HAlignHeader(EHorizontalAlignment::HAlign_Center)
		.ManualWidth(this, &SMaterialEditorStatsWidget::GetColumnSize, ColumnName)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Margin(4.0f)
			.ColorAndOpacity(Color)
			.Text(FText::FromString(Content))
			.ToolTipText(FText::FromString(ContentLong))
		];

	return ColumnArgs;
}

void SMaterialEditorStatsWidget::InsertColumnAfter(const FName ColumnName, const FName PreviousColumn)
{
	int32 InsertIndex = -1;

	auto& ColumnArray = PlatformColumnHeader->GetColumns();
	auto Iterator = ColumnArray.CreateConstIterator();
	while (Iterator)
	{
		if (Iterator->ColumnId == PreviousColumn)
		{
			InsertIndex = Iterator.GetIndex() + 1;
			break;
		}

		++Iterator;
	}

	if (InsertIndex != -1)
	{
		const auto Column = CreateColumnArgs(ColumnName);
		PlatformColumnHeader->InsertColumn(Column, InsertIndex);
	}
}

void SMaterialEditorStatsWidget::AddColumn(const FName ColumnName)
{
	const auto Column = CreateColumnArgs(ColumnName);
	PlatformColumnHeader->AddColumn(Column);
}

void SMaterialEditorStatsWidget::RemoveColumn(const FName ColumnName)
{
	PlatformColumnHeader->RemoveColumn(ColumnName);
}

void SMaterialEditorStatsWidget::RebuildColumns()
{
	PlatformColumnHeader->ClearColumns();

	auto StatsManager = MaterialStatsWPtr.Pin();
	if (StatsManager.IsValid())
	{
		auto Grid = StatsManager->GetStatsGrid();

		if (Grid.IsValid())
		{
			auto ColumnNames = Grid->GetVisibleColumnNames();

			for (int32 i = 0; i < ColumnNames.Num(); ++i)
			{
				AddColumn(ColumnNames[i]);
			}
		}
	}
}

uint32 SMaterialEditorStatsWidget::CountSubPlatforms(EPlatformCategoryType Category) const
{
	uint32 PlatformCount = 0;
	const auto StatsPtr = MaterialStatsWPtr.Pin();

	if (!StatsPtr.IsValid())
	{
		return 0;
	}

	auto& PlatformsDB = StatsPtr->GetPlatformsTypeDB();
	auto* ArrPlaformsPtr = PlatformsDB.Find(Category);
	if (ArrPlaformsPtr == nullptr)
	{
		return 0;
	}

	auto& ArrPlatforms = *ArrPlaformsPtr;

	for (int32 i = 0; i < ArrPlatforms.Num(); ++i)
	{
		auto PlatformPtr = ArrPlatforms[i];
		if (!PlatformPtr.IsValid())
		{
			continue;
		}

		PlatformCount++;
	}

	return PlatformCount;

}

void SMaterialEditorStatsWidget::CreatePlatformMenus(FMenuBuilder& Builder, EPlatformCategoryType Category)
{
	const auto StatsPtr = MaterialStatsWPtr.Pin();

	if (!StatsPtr.IsValid())
	{
		return;
	}

	auto& PlatformsDB = StatsPtr->GetPlatformsTypeDB();
	auto* ArrPlaformsPtr = PlatformsDB.Find(Category);
	if (ArrPlaformsPtr == nullptr)
	{
		return;
	}

	auto& ArrPlatforms = *ArrPlaformsPtr;

	for (int32 i = 0; i < ArrPlatforms.Num(); ++i)
	{
		auto PlatformPtr = ArrPlatforms[i];
		if (!PlatformPtr.IsValid())
		{
			continue;
		}

		const FName PlatformName = PlatformPtr->GetPlatformName();

		// lambda function used to determine the check-state for the shader platform checkboxes
		auto Lamda_PlatformCheckState = [PlatformPtr = PlatformPtr]()
		{
			if (PlatformPtr.IsValid())
			{
				return PlatformPtr->IsPresentInGrid() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		};

		// lambda function used to determine the enable state for the shader platform checkboxes
		auto Lamda_PlatformEnableState = [PlatformPtr = PlatformPtr]()
		{
			return PlatformPtr.IsValid() && !PlatformPtr->IsAlwaysOn() && PlatformPtr->IsStatsGridPresenceAllowed();
		};

		// lambda used with shader platform checkboxes to add or remove selected shader platforms
		auto Lamda_PlatformFlipState = [WidgetPtr = this, PlatformPtr = PlatformPtr](const ECheckBoxState NewState)
		{
			auto MaterialStats = WidgetPtr->MaterialStatsWPtr.Pin();
			if (PlatformPtr.IsValid() && MaterialStats.IsValid())
			{
				MaterialStats->SwitchShaderPlatformUseStats(PlatformPtr->GetPlatformShaderType());
				WidgetPtr->OnColumnNumChanged();
			}

			WidgetPtr->RequestRefresh();
		};

		auto PlatformWidget = SNew(SCheckBox)
			.OnCheckStateChanged_Lambda(Lamda_PlatformFlipState)
			.IsChecked_Lambda(Lamda_PlatformCheckState)
			.IsEnabled_Lambda(Lamda_PlatformEnableState)
			.Content()
			[
				SNew(STextBlock)
				.Text(FText::FromName(PlatformName))
				.Margin(FMargin(2.0f, 2.0f, 4.0f, 2.0f))
			];

		Builder.AddMenuEntry(FUIAction(), PlatformWidget);
	}
}

void SMaterialEditorStatsWidget::CreatePlatformCategoryMenus(FMenuBuilder& Builder)
{
	for (int32 i = 0; i < (int32)EPlatformCategoryType::Num; ++i)
	{
		const EPlatformCategoryType PlatformType = static_cast<EPlatformCategoryType>(i);

		if (CountSubPlatforms(PlatformType) > 0)
		{
			auto PlatformTypeWidget = SNew(STextBlock)
				.Text(FText::FromString(FMaterialStatsUtils::GetPlatformTypeName(PlatformType)));

			Builder.AddSubMenu(PlatformTypeWidget, FNewMenuDelegate::CreateSP(this, &SMaterialEditorStatsWidget::CreatePlatformMenus, PlatformType));
		}
	}
}

void SMaterialEditorStatsWidget::OnFlipQualityState(const ECheckBoxState NewState, const EMaterialQualityLevel::Type QualityLevel)
{
	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid())
	{
		StatsPtr->SwitchStatsQualityFlag(QualityLevel);

		StatsPtr->GetStatsGrid()->OnQualitySettingChanged(QualityLevel);

		OnColumnNumChanged();

		RequestRefresh();
	}
}

void SMaterialEditorStatsWidget::CreateQualityMenus(FMenuBuilder& Builder)
{
	for (int32 i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		auto QualityLevel = static_cast<EMaterialQualityLevel::Type>(i);

		auto Lamda_QualityCheckState = [QualityType = (EMaterialQualityLevel::Type)i, MaterialStatsWPtr = MaterialStatsWPtr]()
		{
			const auto StatsPtr = MaterialStatsWPtr.Pin();
			if (StatsPtr.IsValid())
			{
				return StatsPtr->GetStatsQualityFlag(QualityType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		};

		auto Lamba_QualityCheckAlwaysOn = [QualityType = (EMaterialQualityLevel::Type)i, MaterialStatsWPtr = MaterialStatsWPtr]()
		{
			const auto StatsPtr = MaterialStatsWPtr.Pin();
			return StatsPtr.IsValid() && !StatsPtr->GetStatsQualityFlagAlwaysOn(QualityType);
		};

		const FText QualitySettingName = FText::FromString(FMaterialStatsUtils::MaterialQualityToString(QualityLevel));

		auto QualityWidget = SNew(SCheckBox)
			.OnCheckStateChanged(this, &SMaterialEditorStatsWidget::OnFlipQualityState, QualityLevel)
			.IsChecked_Lambda(Lamda_QualityCheckState)
			.IsEnabled_Lambda(Lamba_QualityCheckAlwaysOn)
			.Content()
			[
				SNew(STextBlock)
				.Text(QualitySettingName)
			];

		Builder.AddMenuEntry(FUIAction(), QualityWidget);
	}
}

FText SMaterialEditorStatsWidget::MaterialStatsDerivedMIOptionToDescription(const EMaterialStatsDerivedMIOption Option)
{
	static_assert(static_cast<int32>(EMaterialStatsDerivedMIOption::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (Option)
	{
	case EMaterialStatsDerivedMIOption::Ignore:			return LOCTEXT("MaterialPlatformStats_IgnoreMIs", "Ignore derived material instances");
	case EMaterialStatsDerivedMIOption::CompileOnly:	return LOCTEXT("MaterialPlatformStats_CompileMIs", "Compile derived material instances");
	case EMaterialStatsDerivedMIOption::ShowStats:		return LOCTEXT("MaterialPlatformStats_ShowMIs", "Show stats for derived material instances");
	default:											return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

void SMaterialEditorStatsWidget::CreateDerivedMaterialsMenu(class FMenuBuilder& Builder)
{
	DerivedMaterialInstancesComboBoxItems.SetNum(static_cast<int32>(EMaterialStatsDerivedMIOption::InvalidOrMax));
	auto AddOption = [&](EMaterialStatsDerivedMIOption Option)
	{
		DerivedMaterialInstancesComboBoxItems[static_cast<int32>(Option)] = MakeShared<EMaterialStatsDerivedMIOption>(Option);
	};
	if (bAllowIgnoringCompilationErrors)
	{
		AddOption(EMaterialStatsDerivedMIOption::Ignore);
	}
	AddOption(EMaterialStatsDerivedMIOption::CompileOnly);
	AddOption(EMaterialStatsDerivedMIOption::ShowStats);

	auto PlatformWidget = SNew(SComboBox<TSharedPtr<EMaterialStatsDerivedMIOption>>)
		.OptionsSource(&DerivedMaterialInstancesComboBoxItems)
		.OnGenerateWidget_Lambda([](TSharedPtr<EMaterialStatsDerivedMIOption> Option)
		{
			return SNew(STextBlock)
				.Text(Option.IsValid() ? MaterialStatsDerivedMIOptionToDescription(*Option) : MaterialStatsDerivedMIOptionToDescription(EMaterialStatsDerivedMIOption::InvalidOrMax));
		})
		.OnSelectionChanged_Lambda([WidgetPtr = this](TSharedPtr<EMaterialStatsDerivedMIOption> Option, ESelectInfo::Type InSelectType)
		{
			auto MaterialStats = WidgetPtr->MaterialStatsWPtr.Pin();
			if (MaterialStats.IsValid() && Option.IsValid())
			{
				MaterialStats->SetMaterialStatsDerivedMIOption(*Option);
			}
		})
	[
		SNew(STextBlock)
		.Text_Lambda([WidgetPtr = this]()
		{
			auto MaterialStats = WidgetPtr->MaterialStatsWPtr.Pin();
			if (MaterialStats.IsValid())
			{
				return MaterialStatsDerivedMIOptionToDescription(MaterialStats->GetMaterialStatsDerivedMIOption());
			}
			else
			{
				return MaterialStatsDerivedMIOptionToDescription(EMaterialStatsDerivedMIOption::InvalidOrMax);
			}
		})
	]
	;

	Builder.AddMenuEntry(FUIAction(), PlatformWidget);
}

void SMaterialEditorStatsWidget::CreateGlobalQualityMenu(class FMenuBuilder& Builder)
{
	auto GlobalQualityWidget = SNew(STextBlock)
		.Text(LOCTEXT("GlobalQualitySettings", "Global Quality Settings"));

	Builder.AddSubMenu(GlobalQualityWidget, FNewMenuDelegate::CreateSP(this, &SMaterialEditorStatsWidget::CreateQualityMenus));
}

TSharedRef<SWidget> SMaterialEditorStatsWidget::GetSettingsButtonContent()
{
	FMenuBuilder Builder(false, nullptr);

	CreatePlatformCategoryMenus(Builder);
	if (bShowMaterialInstancesMenu)
	{
		Builder.AddMenuSeparator();
		CreateDerivedMaterialsMenu(Builder);
	}
	Builder.AddMenuSeparator();
	CreateGlobalQualityMenu(Builder);

	return Builder.MakeWidget();
}

TSharedPtr<SWidget> SMaterialEditorStatsWidget::BuildMessageArea()
{
	MessageBoxWidget = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight();

	return MessageBoxWidget;
}

void SMaterialEditorStatsWidget::AddMessage(const FString& Message, const bool bIsError)
{
	MessageBoxWidget->AddSlot()
		.AutoHeight()
		.Padding(2.5, 2.5)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush(TEXT("Brushes.Recessed")))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(bIsError ? "MessageLog.Error" : "MessageLog.Warning"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SEditableText)
					.Style(FAppStyle::Get(), TEXT("RichTextBlock.Bold"))
					.ColorAndOpacity(bIsError ? FStyleColors::Error : FStyleColors::Warning)
					.Text(FText::FromString(Message))
					.ToolTipText(FText::FromString(Message))
					.IsReadOnly(true)
				]
			]
		];
}

void SMaterialEditorStatsWidget::ClearMessages()
{
	MessageBoxWidget->ClearChildren();
}

FReply SMaterialEditorStatsWidget::OnButtonClick(EAppReturnType::Type ButtonID)
{
	auto MaterialStats = MaterialStatsWPtr.Pin();
	if (!MaterialStats.IsValid())
	{
		return FReply::Handled();
	}

	FString ShadersStats = MaterialStats->GetShadersStats();
	FPlatformApplicationMisc::ClipboardCopy(*ShadersStats);

	return FReply::Handled();
}

void SMaterialEditorStatsWidget::Construct(const FArguments& InArgs)
{
	MaterialStatsWPtr = InArgs._MaterialStatsWPtr;
	bShowMaterialInstancesMenu = InArgs._ShowMaterialInstancesMenu;
	bAllowIgnoringCompilationErrors = InArgs._AllowIgnoringCompilationErrors;

	const auto StatsPtr = MaterialStatsWPtr.Pin();

	if (!StatsPtr.IsValid())
	{
		return;
	}

	auto MessageArea = BuildMessageArea();

	SetVisibility(EVisibility::SelfHitTestInvisible);

	// construct default column headers
	PlatformColumnHeader = SNew(SHeaderRow);
	RebuildColumns();

	const auto VerticalScrollbar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(11.0f, 11.0f));

	const auto HorizontalScrollbar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(11.0f, 11.0f));

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot() // this will contain the tool bar
		.Padding(0.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0)
			[
				SNullWidget::NullWidget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(0)
				.Text(LOCTEXT("CopyStats", "CopyStats"))
				.ToolTipText(LOCTEXT("CopyStatsTooltip", "Copy all stats to the clipboard"))
				.OnClicked(this, &SMaterialEditorStatsWidget::OnButtonClick, EAppReturnType::Cancel)
			] 
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
				.ForegroundColor(FSlateColor::UseStyle())
				.ContentPadding(0)
				.OnGetMenuContent(this, &SMaterialEditorStatsWidget::GetSettingsButtonContent)
				.HasDownArrow(true)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					// Icon
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("LevelEditor.GameSettings"))
					]
					// Text
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 0, 2, 0)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SettingsButton", "Settings"))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush(TEXT("Brushes.Recessed")))
				.Padding(0.0f)
				.HAlign(HAlign_Fill)
				[
					SNew(SScrollBox)
					.Orientation(Orient_Vertical)
					.ExternalScrollbar(VerticalScrollbar)
					+ SScrollBox::Slot()
					[
						SNew(SBorder)
						.Padding(0.0f)
						.HAlign(HAlign_Fill)
						.BorderImage(FAppStyle::Get().GetBrush(TEXT("Brushes.Recessed")))
						[
							MessageArea.ToSharedRef()
						]
					]
					+ SScrollBox::Slot()
					[
						SNew(SScrollBox)
						.Orientation(Orient_Horizontal)
						.ExternalScrollbar(HorizontalScrollbar)
						+ SScrollBox::Slot()
						[
							SAssignNew(MaterialInfoList, SListView<TSharedPtr<int32>>)
							.ExternalScrollbar(VerticalScrollbar)
							.ListItemsSource(StatsPtr->GetStatsGrid()->GetGridRowIDs())
							.OnGenerateRow(this, &SMaterialEditorStatsWidget::MakeMaterialInfoWidget)
							.Visibility(EVisibility::Visible)
							.SelectionMode(ESelectionMode::Single)
							.HeaderRow(PlatformColumnHeader)
						]
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollbar
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			HorizontalScrollbar
		]
];
}

TSharedRef<ITableRow> SMaterialEditorStatsWidget::MakeMaterialInfoWidget(const TSharedPtr<int32> PtrRowID, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SMaterialStatsViewRow, OwnerTable)
		.PtrRowID(PtrRowID)
		.MaterialStatsWPtr(MaterialStatsWPtr);
}

void SMaterialEditorStatsWidget::RequestRefresh()
{
	MaterialInfoList->RequestListRefresh();
}

FName SMaterialEditorStatsWidget::GetRegularFontStyleName()
{
	return RegularFontStyle;
}

FName SMaterialEditorStatsWidget::GetBoldFontStyleName()
{
	return BoldFontStyle;
}

void SMaterialEditorStatsWidget::OnColumnNumChanged()
{
	auto StatsPtr = MaterialStatsWPtr.Pin();
	if (StatsPtr.IsValid() && StatsPtr->GetStatsGrid().IsValid())
	{
		StatsPtr->GetStatsGrid()->OnColumnNumChanged();
		RebuildColumns();
	}
}

#undef LOCTEXT_NAMESPACE