// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "MaterialStats.h"
#include "MaterialStatsGrid.h"

//////////////////////////////////////////////////////////////////////////
// SMaterialEditorStatsWidget

/** Widget class used to display stats extracted from materials; used in the material editor */
class SMaterialEditorStatsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialEditorStatsWidget)
		: _MaterialStatsWPtr(nullptr)
		, _ShowMaterialInstancesMenu(true)
		, _AllowIgnoringCompilationErrors(true)
	{}

		SLATE_ARGUMENT(TWeakPtr<FMaterialStats>, MaterialStatsWPtr)
		SLATE_ARGUMENT(bool, ShowMaterialInstancesMenu)
		SLATE_ARGUMENT(bool, AllowIgnoringCompilationErrors)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback used to populate SListView */
	TSharedRef<ITableRow> MakeMaterialInfoWidget(const TSharedPtr<int32> Item, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Request a refresh of the list view */
	void RequestRefresh();

	void AddMessage(const FString& Message, const bool bIsError);
	void ClearMessages();

	void FillWarningMessages();

	static FName GetRegularFontStyleName();
	static FName GetBoldFontStyleName();

	void OnColumnNumChanged();

private:
	void RebuildColumns();
	uint32 CountSubPlatforms(EPlatformCategoryType Category) const;
	void AddColumn(const FName ColumnName);
	void InsertColumnAfter(const FName ColumnName, const FName PreviousColumn);
	void RemoveColumn(const FName ColumnName);

	SHeaderRow::FColumn::FArguments CreateColumnArgs(const FName ColumnName);

	float GetColumnSize(const FName ColumnName) const;

	TSharedRef<SWidget> GetSettingsButtonContent();

	void CreatePlatformMenus(class FMenuBuilder& Builder, EPlatformCategoryType Category);
	void CreatePlatformCategoryMenus(class FMenuBuilder& Builder);

	static FText MaterialStatsDerivedMIOptionToDescription(const EMaterialStatsDerivedMIOption Option);

	void CreateQualityMenus(class FMenuBuilder& Builder);
	void CreateDerivedMaterialsMenu(class FMenuBuilder& Builder);
	void CreateGlobalQualityMenu(class FMenuBuilder& Builder);

	void OnFlipQualityState(const ECheckBoxState NewState, const EMaterialQualityLevel::Type QualitySetting);

	TSharedPtr<SWidget> BuildMessageArea();

	FReply OnButtonClick(EAppReturnType::Type ButtonID);

protected:
	static const float ColumnSizeSmall;
	static const float ColumnSizeMedium;
	static const float ColumnSizeLarge;
	static const float ColumnSizeExtraLarge;

	static const FName RegularFontStyle;
	static const FName BoldFontStyle;

	TSharedPtr<SVerticalBox> MessageBoxWidget;

	TWeakPtr<FMaterialStats> MaterialStatsWPtr;
	bool bShowMaterialInstancesMenu;
	bool bAllowIgnoringCompilationErrors;

	TSharedPtr<SListView<TSharedPtr<int32>>> MaterialInfoList;

	TSharedPtr<SHeaderRow> PlatformColumnHeader;

	TArray<TSharedPtr<EMaterialStatsDerivedMIOption>> DerivedMaterialInstancesComboBoxItems;
};