// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialEditorSubstrateWidget.h"

#include "EditorWidgetsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MaterialEditor.h"
#include "SubstrateDefinitions.h"
#include <functional>

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Notifications/SErrorText.h"
#include "MaterialEditor/SGraphSubstrateMaterial.h"
#include "RHIShaderPlatform.h"

#define LOCTEXT_NAMESPACE "SMaterialEditorSubstrateWidget"

DEFINE_LOG_CATEGORY_STATIC(LogMaterialEditorSubstrateWidget, Log, All);

void SMaterialEditorSubstrateWidget::Construct(const FArguments& InArgs, TWeakPtr<IMaterialEditor> InMaterialEditorPtr)
{
	MaterialEditorPtr = InMaterialEditorPtr;

	ButtonApplyToPreview = SNew(SButton)
		.HAlign(HAlign_Center)
		.OnClicked(this, &SMaterialEditorSubstrateWidget::OnButtonApplyToPreview)
		.Text(LOCTEXT("ButtonApplyToPreview", "Apply to preview"));

	CheckBoxForceFullSimplification = SNew(SCheckBox)
		.Padding(5.0f)
		.ToolTipText(LOCTEXT("CheckBoxForceFullSimplificationToolTip", "This will force full simplification of the material."));

	MaterialBudgetTextBlock = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "Log.Normal")
		.ColorAndOpacity(FLinearColor::White)
		.ShadowColorAndOpacity(FLinearColor::Black)
		.ShadowOffset(FVector2D::UnitVector)
		.Text(LOCTEXT("DescriptionTextBlock_Default", "Shader is compiling"));

	MaterialFeatures0TextBlock = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "Log.Normal")
		.ColorAndOpacity(FLinearColor::White)
		.ShadowColorAndOpacity(FLinearColor::Black)
		.ShadowOffset(FVector2D::UnitVector)
		.Text(LOCTEXT("DescriptionTextBlock_Default", "Shader is compiling"));

	MaterialFeatures1TextBlock = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "Log.Normal")
		.ColorAndOpacity(FLinearColor::White)
		.ShadowColorAndOpacity(FLinearColor::Black)
		.ShadowOffset(FVector2D::UnitVector)
		.Text(LOCTEXT("DescriptionTextBlock_Default", "Shader is compiling"));

	MaterialFeatures2TextBlock = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "Log.Normal")
		.ColorAndOpacity(FLinearColor::White)
		.ShadowColorAndOpacity(FLinearColor::Black)
		.ShadowOffset(FVector2D::UnitVector)
		.Text(LOCTEXT("DescriptionTextBlock_Default", "Shader is compiling"));

	DescriptionTextBlock = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "Log.Normal")
		.ColorAndOpacity(FLinearColor::White)
		.ShadowColorAndOpacity(FLinearColor::Black)
		.ShadowOffset(FVector2D::UnitVector)
		.Text(LOCTEXT("DescriptionTextBlock_Default", "Shader is compiling"));

	BytesPerPixelOverride = Substrate::GetBytePerPixel(SP_PCD3D_SM5);
	BytesPerPixelOverrideInput = SNew(SNumericEntryBox<uint32>)
		.MinDesiredValueWidth(150.0f)
		.MinValue(12)
		.MaxValue(BytesPerPixelOverride)
		.MinSliderValue(12)
		.MaxSliderValue(BytesPerPixelOverride)
		.OnBeginSliderMovement(this, &SMaterialEditorSubstrateWidget::OnBeginBytesPerPixelSliderMovement)
		.OnEndSliderMovement(this, &SMaterialEditorSubstrateWidget::OnEndBytesPerPixelSliderMovement)
		.AllowSpin(true)
		.OnValueChanged(this, &SMaterialEditorSubstrateWidget::OnBytesPerPixelChanged)
		.OnValueCommitted(this, &SMaterialEditorSubstrateWidget::OnBytesPerPixelCommitted)
		.Value(this, &SMaterialEditorSubstrateWidget::GetBytesPerPixelValue)
		.IsEnabled(false);

	CheckBoxBytesPerPixelOverride = SNew(SCheckBox)
		.Padding(5.0f)
		.ToolTipText(LOCTEXT("CheckBoxBytesPerPixelOverride", "This will force the byte per pixel count for the preview material. It cannot go higher than the current project setting."))
		.OnCheckStateChanged(this, &SMaterialEditorSubstrateWidget::OnCheckBoxBytesPerPixelChanged);

	ClosuresPerPixelOverride = Substrate::GetClosurePerPixel(SP_PCD3D_SM5, Substrate::IgnoreMaterialQualityLevelsForClosurePerPixel);
	ClosuresPerPixelOverrideInput = SNew(SNumericEntryBox<uint32>)
		.MinDesiredValueWidth(150.0f)
		.MinValue(1)
		.MaxValue(ClosuresPerPixelOverride)
		.MinSliderValue(1)
		.MaxSliderValue(ClosuresPerPixelOverride)
		.OnBeginSliderMovement(this, &SMaterialEditorSubstrateWidget::OnBeginClosuresPerPixelSliderMovement)
		.OnEndSliderMovement(this, &SMaterialEditorSubstrateWidget::OnEndClosuresPerPixelSliderMovement)
		.AllowSpin(true)
		.OnValueChanged(this, &SMaterialEditorSubstrateWidget::OnClosuresPerPixelChanged)
		.OnValueCommitted(this, &SMaterialEditorSubstrateWidget::OnClosuresPerPixelCommitted)
		.Value(this, &SMaterialEditorSubstrateWidget::GetClosuresPerPixelValue)
		.IsEnabled(false);

	CheckBoxClosuresPerPixelOverride = SNew(SCheckBox)
		.Padding(5.0f)
		.ToolTipText(LOCTEXT("CheckBoxClosuresPerPixelOverride", "This will force the byte per pixel count for the preview material. It cannot go higher than the current project setting."))
		.OnCheckStateChanged(this, &SMaterialEditorSubstrateWidget::OnCheckBoxClosuresPerPixelChanged);

	if (Substrate::IsSubstrateEnabled())
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			//.AutoHeight()			// Cannot use that otherwise scrollbars disapear.
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				.ScrollBarAlwaysVisible(false)
				+ SScrollBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 5.0f, 0.0f, 0.0f)
					[
						SNew(SWrapBox)
						.UseAllottedSize(true)
						+ SWrapBox::Slot()
						.Padding(0.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(LOCTEXT("MaterialSimplificationPreview", "Material Simplification Preview"))
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						//.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						//.BorderBackgroundColor(FLinearColor(0.9f, 0.6f, 0.6f, 1.0f))
						.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SWrapBox)
								.UseAllottedSize(true)
								+SWrapBox::Slot()
								.Padding(20.0, 0.0, 0.0, 0.0) // Padding(float Left, float Top, float Right, float Bottom)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									CheckBoxForceFullSimplification->AsShared()
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FLinearColor::White)
								.ShadowColorAndOpacity(FLinearColor::Black)
								.ShadowOffset(FVector2D::UnitVector)
								.Text(LOCTEXT("FullsimplificationLabel", "Full simplification"))
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(20.0, 0.0, 0.0, 0.0)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SWrapBox)
								.UseAllottedSize(true)
								+SWrapBox::Slot()
								.Padding(5.0f)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									CheckBoxBytesPerPixelOverride->AsShared()
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FLinearColor::White)
								.ShadowColorAndOpacity(FLinearColor::Black)
								.ShadowOffset(FVector2D::UnitVector)
								.Text(LOCTEXT("OverrideBytesPerPixel", "Override bytes per pixel"))
							]
							+SHorizontalBox::Slot()
							.MaxWidth(200)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SWrapBox)
								.UseAllottedSize(true)
								+SWrapBox::Slot()
								.Padding(0.0, 0.0, 0.0, 0.0)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									BytesPerPixelOverrideInput->AsShared()
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(20.0, 0.0, 0.0, 0.0)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SWrapBox)
								.UseAllottedSize(true)
								+SWrapBox::Slot()
								.Padding(5.0f)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									CheckBoxClosuresPerPixelOverride->AsShared()
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FLinearColor::White)
								.ShadowColorAndOpacity(FLinearColor::Black)
								.ShadowOffset(FVector2D::UnitVector)
								.Text(LOCTEXT("OverrideClosuresPerPixel", "Override closures per pixel"))
							]
							+SHorizontalBox::Slot()
							.MaxWidth(200)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SWrapBox)
								.UseAllottedSize(true)
								+SWrapBox::Slot()
								.Padding(0.0, 0.0, 0.0, 0.0)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									ClosuresPerPixelOverrideInput->AsShared()
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SWrapBox)
								//.UseAllottedSize(true)
								+ SWrapBox::Slot()
								.Padding(20.0, 0.0, 0.0, 0.0)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									ButtonApplyToPreview->AsShared()
								]
							]
						] // Border
					] // VerticalBox
					


					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 15.0f, 0.0f, 0.0f)
					[
						SNew(SWrapBox)
						.UseAllottedSize(true)
						+ SWrapBox::Slot()
						.Padding(0.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(LOCTEXT("MaterialTopologyPreview", "Material Topology Preview"))
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
						//.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						//.BorderBackgroundColor(FLinearColor(0.9f, 0.6f, 0.6f, 1.0f))
						.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
						[
							SNew(SWrapBox)
							.UseAllottedSize(true)
							+ SWrapBox::Slot()
							.Padding(10.0f)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SAssignNew(MaterialBox, SBox)
							]
						]
					]


					
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 15.0f, 0.0f, 0.0f)
					[
						SNew(SWrapBox)
						.UseAllottedSize(true)
						+ SWrapBox::Slot()
						.Padding(0.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(LOCTEXT("MaterialBudgetTextBlock", "Material Budget"))
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
						.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
						[
							SNew(SWrapBox)
							.UseAllottedSize(true)
							+SWrapBox::Slot()
							.Padding(5.0f)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								MaterialBudgetTextBlock->AsShared()
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 15.0f, 0.0f, 0.0f)
					[
						SNew(SWrapBox)
						.UseAllottedSize(true)
						+ SWrapBox::Slot()
						.Padding(0.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(LOCTEXT("MaterialFeaturesTextBlock", "Material Features"))
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
						.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.0f))
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Top)
							[
								MaterialFeatures0TextBlock->AsShared()
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.0f))
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Top)
							[
								MaterialFeatures1TextBlock->AsShared()
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.0f))
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Top)
							[
								MaterialFeatures2TextBlock->AsShared()
							]
						]
					]
					
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 15.0f, 0.0f, 0.0f)
					[
						SNew(SWrapBox)
						.UseAllottedSize(true)
						+ SWrapBox::Slot()
						.Padding(0.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(LOCTEXT("MaterialAdvancedDetails", "Material Advanced Details"))
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
						//.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						//.BorderBackgroundColor(FLinearColor(0.9f, 0.6f, 0.6f, 1.0f))
						.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
						[
							SNew(SWrapBox)
							.UseAllottedSize(true)
							+SWrapBox::Slot()
							.Padding(5.0f)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								DescriptionTextBlock->AsShared()
							]
						]
					]
				]
			]
		];
	}
	else
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)
				+SWrapBox::Slot()
				.Padding(5.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FLinearColor::Yellow)
					.ShadowColorAndOpacity(FLinearColor::Black)
					.ShadowOffset(FVector2D::UnitVector)
					.Text(LOCTEXT("SubstrateWidgetNotEnable", "Details cannot be shown: Substrate is not enabled for this project (See the project settings window, rendering settings section)."))
				]
			]
		];
	}
}

TSharedRef<SWidget> SMaterialEditorSubstrateWidget::GetContent()
{
	return SharedThis(this);
}

SMaterialEditorSubstrateWidget::~SMaterialEditorSubstrateWidget()
{
}

static FString SubstrateMaterialTypeToString(uint8 InMaterialType)
{
	switch (InMaterialType)
	{
		case SUBSTRATE_MATERIAL_TYPE_SIMPLE:  			return FString::Printf(TEXT("SIMPLE  (Diffuse, albedo, roughness)\r\n"));
		case SUBSTRATE_MATERIAL_TYPE_SINGLE:  			return FString::Printf(TEXT("SINGLE  (BSDF all features except anisotropy)\r\n"));
		case SUBSTRATE_MATERIAL_TYPE_COMPLEX:  			return FString::Printf(TEXT("COMPLEX (Anisotropy, multi-slabs, Toon)\r\n"));
		case SUBSTRATE_MATERIAL_TYPE_COMPLEX_SPECIAL:  	return FString::Printf(TEXT("COMPLEX SPECIAL (Glints)\r\n"));
		default: 										return FString::Printf(TEXT("UNKOWN => ERROR!\r\n"));
	}
}

void SMaterialEditorSubstrateWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!bUpdateRequested || !Substrate::IsSubstrateEnabled())
	{
		return;
	}
	bUpdateRequested = false;

	FText SubstrateMaterialDescription;
	if (MaterialEditorPtr.IsValid())
	{
		TSharedPtr<IMaterialEditor> MaterialEditorPinned = MaterialEditorPtr.Pin();
		FMaterialResource* MaterialResource = MaterialEditorPinned->GetMaterialInterface()->GetMaterialResource(GMaxRHIShaderPlatform);

		if (Cast<UMaterial>(MaterialEditorPinned->GetMaterialInterface()))
		{
			TSharedPtr<FMaterialEditor> MaterialEditor = StaticCastSharedPtr<FMaterialEditor>(MaterialEditorPinned);
			UMaterial* MaterialForStats = MaterialEditor->bStatsFromPreviewMaterial ? MaterialEditor->Material : MaterialEditor->OriginalMaterial;
			MaterialResource = MaterialForStats->GetMaterialResource(GMaxRHIShaderPlatform);
		}

		SubstrateMaterialDescription = FText::FromString(FString(TEXT("SubstrateMaterialDescription")));

		if (MaterialResource)
		{
			FString MaterialDescription;
			FString MaterialBudget;
			FString MaterialFeatures0;
			FString MaterialFeatures1;
			FString MaterialFeatures2;

			bool bMaterialOutOfBudgetHasBeenSimplified = false;
			FMaterialShaderMap* ShaderMap = MaterialResource->GetGameThreadShaderMap();
			if (ShaderMap)
			{
				const FSubstrateMaterialCompilationOutput& CompilationOutput = ShaderMap->GetSubstrateMaterialCompilationOutput();
				const uint32 FinalPixelByteCount = CompilationOutput.SubstrateUintPerPixel * sizeof(uint32);
				const uint32 FinalPixelClosureCount = CompilationOutput.SubstrateClosureCount;
				bMaterialOutOfBudgetHasBeenSimplified = CompilationOutput.bMaterialOutOfBudgetHasBeenSimplified > 0;

				// Features
				{	
					// Actives features
					const ESubstrateBsdfFeature TrackedFeatures0 = 
						  ESubstrateBsdfFeature::SSS
						| ESubstrateBsdfFeature::Fuzz
						| ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat
						| ESubstrateBsdfFeature::Anisotropy;

					const ESubstrateBsdfFeature TrackedFeatures1 = 
						  ESubstrateBsdfFeature::EdgeColor
						| ESubstrateBsdfFeature::Glint
						| ESubstrateBsdfFeature::SpecularProfile;

					ESubstrateBsdfFeature TrackedFeatures = ESubstrateBsdfFeature::None;
					EnumAddFlags(TrackedFeatures, TrackedFeatures0);
					EnumAddFlags(TrackedFeatures, TrackedFeatures1);

					// Untracked features:
					// - ESubstrateBsdfFeature::MFPPluggedIn
					// - ESubstrateBsdfFeature::Eye
					// - ESubstrateBsdfFeature::Hair
					if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, TrackedFeatures))
					{
						// Selection & Sorting based on logic in:
						// - SubstrateGetLegacyShadingModels(..)
						// - SubstrateMaterialExportOut(..)					
						FString& PotentiallySupported   = Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform) ? MaterialFeatures1 : MaterialFeatures0;
						FString& PotentiallyUnsupported = Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform) ? MaterialFeatures2 : MaterialFeatures0;

						if (Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform))
						{
							if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, TrackedFeatures0))
							{
								MaterialFeatures1 += FString::Printf(TEXT("Enabled features: \n"));
							}
							if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, TrackedFeatures1))
							{
								MaterialFeatures2 += FString::Printf(TEXT("Disabled features: \n"));
							}
						}
						else
						{
							MaterialFeatures0 += FString::Printf(TEXT("Enabled features: \n"));
						}

						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SSS                              )) { PotentiallySupported 	+= TEXT(" - SSS\n"); }
						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Fuzz                             )) { PotentiallySupported 	+= TEXT(" - Fuzz\n"); }
						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat )) { PotentiallySupported 	+= TEXT(" - ClearCoat\n"); }
						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Anisotropy                       )) { PotentiallySupported 	+= TEXT(" - Anisotropy\n"); }
						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::EdgeColor                        )) { PotentiallyUnsupported+= TEXT(" - F90\n"); }
						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Glint                            )) { PotentiallyUnsupported+= TEXT(" - Glint\n"); }
						if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SpecularProfile                  )) { PotentiallyUnsupported+= TEXT(" - Specular Profile\n"); }

						if (Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform))
						{
							if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, TrackedFeatures0))
							{
								MaterialFeatures1 += FString::Printf(TEXT("\nFormat : Blendable GBuffer => Only one feature will be active, sorted by priority. The other features will be disabled.\n"));
							}
							if (EnumHasAnyFlags(CompilationOutput.SubstrateMaterialBsdfFeatures, TrackedFeatures1))
							{
								MaterialFeatures2 += FString::Printf(TEXT("\nFormat : Blendable GBuffer => Unsupported features.\n"));
							}
						}
					}
				}

				// Budget
				if (bMaterialOutOfBudgetHasBeenSimplified)
				{
					MaterialBudget += FString::Printf(TEXT("Format: %s\n\nThe material was OUT-OF-BUDGET so it has been simplified:\n  - Requested Bytes = %i / budget = %i\n  - Requested Closures = %i / budget = %i\r\n\n"),
						Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform) ? TEXT("Blendable GBuffer") : TEXT("Adaptive GBuffer"),
						CompilationOutput.RequestedBytePerPixel, CompilationOutput.PlatformBytePerPixel,
						CompilationOutput.RequestedClosurePerPixel, CompilationOutput.PlatformClosurePixel);
					MaterialBudget += FString::Printf(TEXT("Final per pixel byte count      = %i\r\n"), FinalPixelByteCount);
					MaterialBudget += FString::Printf(TEXT("Final per pixel closure count   = %i\r\n"), FinalPixelClosureCount);
				}
				else
				{
					MaterialBudget += FString::Printf(TEXT("Material per pixel byte count    = %i / budget = %i\r\n"),
						FinalPixelByteCount, CompilationOutput.PlatformBytePerPixel);
					MaterialBudget += FString::Printf(TEXT("Material per pixel closure count = %i / budget = %i\r\n"),
						FinalPixelClosureCount, CompilationOutput.PlatformClosurePixel);
				}

				// Description
				MaterialDescription += FString::Printf(TEXT("Closures Count               = %i\r\n"), CompilationOutput.SubstrateClosureCount);
				MaterialDescription += FString::Printf(TEXT("Local bases Count            = %i\r\n"), CompilationOutput.SharedLocalBasesCount);
				MaterialDescription += FString::Printf(TEXT("Material complexity          = %s\r\n"), *SubstrateMaterialTypeToString(CompilationOutput.SubstrateMaterialType));
				MaterialDescription += FString::Printf(TEXT("Root Node Is Thin            = %i\r\n"), CompilationOutput.bIsThin);

				//if (CompilationOutput.SubstrateBSDFCount == 1)
				//{
				//	auto GetSingleOperatorBSDF = [&](const FSubstrateOperator& Op)
				//	{
				//		switch (Op.OperatorType)
				//		{
				//		case SUBSTRATE_OPERATOR_WEIGHT:
				//		{
				//			return ProcessOperator(CompilationOutput.Operators[Op.LeftIndex]);	// Continue walking the tree
				//			break;
				//		}
				//		case SUBSTRATE_OPERATOR_VERTICAL:
				//		case SUBSTRATE_OPERATOR_HORIZONTAL:
				//		case SUBSTRATE_OPERATOR_ADD:
				//		{
				//			return nullptr;	// A operator with multiple entries must use slabs and so should be a Substrate material made of slabs
				//			break;
				//		}
				//		case SUBSTRATE_OPERATOR_BSDF_LEGACY:
				//		case SUBSTRATE_OPERATOR_BSDF:
				//		{
				//			return Op;
				//			break;
				//		}
				//		}
				//		return nullptr;
				//	};
				//	const FSubstrateOperator& RootOperator = CompilationOutput.Operators[CompilationOutput.RootOperatorIndex];
				//	const FSubstrateOperator* OpBSDF = GetSingleOperatorBSDF(RootOperator);
				//
				//	if (OpBSDF)
				//	{
				//		// SUBSTRATE_TODO gather information about SSSPRofile, subsurface, two sided, etc?
				//	}
				//}

				FString MaterialType;
				switch (CompilationOutput.MaterialType)
				{
				case SUBSTRATE_MATERIAL_TYPE_SINGLESLAB:			MaterialType = TEXT("Single Slab"); break;
				case SUBSTRATE_MATERIAL_TYPE_MULTIPLESLABS:			MaterialType = TEXT("Multiple Slabs"); break;
				case SUBSTRATE_MATERIAL_TYPE_MULTIPLEBSDFS:			MaterialType = TEXT("Multiple BSDFs"); break;
				case SUBSTRATE_MATERIAL_TYPE_VOLUMETRICFOGCLOUD:	MaterialType = TEXT("Volumetric (Fog or Cloud)"); break;
				case SUBSTRATE_MATERIAL_TYPE_UNLIT:					MaterialType = TEXT("Unlit"); break;
				case SUBSTRATE_MATERIAL_TYPE_HAIR:					MaterialType = TEXT("Hair"); break;
				case SUBSTRATE_MATERIAL_TYPE_SINGLELAYERWATER:		MaterialType = TEXT("SingleLayerWater"); break;
				case SUBSTRATE_MATERIAL_TYPE_EYE:					MaterialType = TEXT("Eye"); break;
				case SUBSTRATE_MATERIAL_TYPE_LIGHTFUNCTION:			MaterialType = TEXT("LightFunction"); break;
				case SUBSTRATE_MATERIAL_TYPE_POSTPROCESS:			MaterialType = TEXT("PostProcess"); break;
				case SUBSTRATE_MATERIAL_TYPE_UI:					MaterialType = TEXT("UI"); break;
				case SUBSTRATE_MATERIAL_TYPE_DECAL:					MaterialType = TEXT("Decal"); break;
				case SUBSTRATE_MATERIAL_TYPE_TOON:					MaterialType = TEXT("Toon"); break;
				}

				FString MaterialTypeDetail;
				if ((CompilationOutput.MaterialType == SUBSTRATE_MATERIAL_TYPE_SINGLESLAB || CompilationOutput.MaterialType == SUBSTRATE_MATERIAL_TYPE_MULTIPLESLABS) && CompilationOutput.bIsThin)
				{
					MaterialTypeDetail = TEXT("(Thin Surface / Two-Sided Lighting)");
				}

				MaterialDescription += FString::Printf(TEXT("Material Type                = %s %s\r\n"), *MaterialType, *MaterialTypeDetail);

				MaterialDescription += FString::Printf(TEXT(" \r\n"));
				MaterialDescription += FString::Printf(TEXT(" \r\n"));
				MaterialDescription += FString::Printf(TEXT("================================================================================\r\n"));
				MaterialDescription += FString::Printf(TEXT("================================Detailed Output=================================\r\n"));
				MaterialDescription += FString::Printf(TEXT("================================================================================\r\n"));
				MaterialDescription += CompilationOutput.SubstrateMaterialDescription;

				// Now generate a visual representation of the material from the topology tree of operators.
				
				if (CompilationOutput.bMaterialUsesLegacyGBufferDataPassThrough)
				{
					auto TreeError = SNew(SErrorText)
						.ErrorText(LOCTEXT("NoSubstrateTree", "No Substrate Tree - Pass Through Legacy GBuffer Data"))
						.BackgroundColor(FSlateColor(EStyleColor::AccentGreen));
					const TSharedRef<SWidget>& TreeErrorAsShared = TreeError->AsShared();
					MaterialBox->SetContent(TreeErrorAsShared);
				}
				else if (CompilationOutput.RootOperatorIndex >= 0)
				{
					MaterialBox->SetContent(FSubstrateWidget::ProcessOperator(CompilationOutput));
				}
				else
				{
					// The tree does not looks sane so generate a visual error without crashing.
					auto TreeError = SNew(SErrorText)
						.ErrorText(LOCTEXT("TreeError", "Tree Error"))
						.BackgroundColor(FSlateColor(EStyleColor::AccentRed));
					const TSharedRef<SWidget>& TreeErrorAsShared = TreeError->AsShared();
					MaterialBox->SetContent(TreeErrorAsShared);
				}
			}
			else
			{
				MaterialDescription = TEXT("Shader map not found.");
				MaterialBox->SetContent(SNullWidget::NullWidget);
			}

			if (bMaterialOutOfBudgetHasBeenSimplified)
			{
				MaterialBudgetTextBlock->SetColorAndOpacity(FLinearColor(0.8f, 0.5f, 0.07f, 1.0f)); // Same color as Graph.WarningText
			}
			else
			{
				MaterialBudgetTextBlock->SetColorAndOpacity(FLinearColor::White);
			}

			{
				MaterialFeatures0TextBlock->SetColorAndOpacity(FLinearColor::White);
				MaterialFeatures1TextBlock->SetColorAndOpacity(FLinearColor(0.8f, 0.5f, 0.07f, 1.0f));
				MaterialFeatures2TextBlock->SetColorAndOpacity(FLinearColor(0.8f, 0.07f, 0.07f, 1.0f));
			}

			MaterialBudgetTextBlock->SetText(FText::FromString(MaterialBudget));
			MaterialFeatures0TextBlock->SetText(FText::FromString(MaterialFeatures0));
			MaterialFeatures1TextBlock->SetText(FText::FromString(MaterialFeatures1));
			MaterialFeatures2TextBlock->SetText(FText::FromString(MaterialFeatures2));
			DescriptionTextBlock->SetText(FText::FromString(MaterialDescription));
		}
	}
}

void SMaterialEditorSubstrateWidget::OnBytesPerPixelChanged(uint32 NewValue)
{
	BytesPerPixelOverride = NewValue;
}

void SMaterialEditorSubstrateWidget::OnBytesPerPixelCommitted(uint32 NewValue, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		BytesPerPixelOverride = NewValue;
	}
}

void SMaterialEditorSubstrateWidget::OnBeginBytesPerPixelSliderMovement()
{
	if (bBytesPerPixelStartedTransaction == false)
	{
		bBytesPerPixelStartedTransaction = true;
		GEditor->BeginTransaction(LOCTEXT("PastePoseTransation", "Paste Pose"));
	}
}
void SMaterialEditorSubstrateWidget::OnEndBytesPerPixelSliderMovement(uint32 NewValue)
{
	if (bBytesPerPixelStartedTransaction)
	{
		GEditor->EndTransaction();
		bBytesPerPixelStartedTransaction = false;
		BytesPerPixelOverride = NewValue;
	}
}

TOptional<uint32> SMaterialEditorSubstrateWidget::GetBytesPerPixelValue() const
{
	return BytesPerPixelOverride;
}

void SMaterialEditorSubstrateWidget::OnCheckBoxBytesPerPixelChanged(ECheckBoxState InCheckBoxState)
{
	BytesPerPixelOverrideInput->SetEnabled(InCheckBoxState == ECheckBoxState::Checked);
}

void SMaterialEditorSubstrateWidget::OnClosuresPerPixelChanged(uint32 NewValue)
{
	ClosuresPerPixelOverride = NewValue;
}

void SMaterialEditorSubstrateWidget::OnClosuresPerPixelCommitted(uint32 NewValue, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		ClosuresPerPixelOverride = NewValue;
	}
}

void SMaterialEditorSubstrateWidget::OnBeginClosuresPerPixelSliderMovement()
{
	if (bClosuresPerPixelStartedTransaction == false)
	{
		bClosuresPerPixelStartedTransaction = true;
		GEditor->BeginTransaction(LOCTEXT("PastePoseTransation", "Paste Pose"));
	}
}
void SMaterialEditorSubstrateWidget::OnEndClosuresPerPixelSliderMovement(uint32 NewValue)
{
	if (bClosuresPerPixelStartedTransaction)
	{
		GEditor->EndTransaction();
		bClosuresPerPixelStartedTransaction = false;
		ClosuresPerPixelOverride = NewValue;
	}
}

TOptional<uint32> SMaterialEditorSubstrateWidget::GetClosuresPerPixelValue() const
{
	return ClosuresPerPixelOverride;
}

void SMaterialEditorSubstrateWidget::OnCheckBoxClosuresPerPixelChanged(ECheckBoxState InCheckBoxState)
{
	ClosuresPerPixelOverrideInput->SetEnabled(InCheckBoxState == ECheckBoxState::Checked);
}

FReply SMaterialEditorSubstrateWidget::OnButtonApplyToPreview()
{
	if (MaterialEditorPtr.IsValid())
	{
		UMaterialInterface* MaterialInterface = MaterialEditorPtr.Pin()->GetMaterialInterface();

		FSubstrateCompilationConfig SubstrateCompilationConfig;
		SubstrateCompilationConfig.bFullSimplify = CheckBoxForceFullSimplification->IsChecked();

		// Have a look at MaterialStats.cpp when we want to visualize a specific platform.
		SubstrateCompilationConfig.BytesPerPixelOverride	= CheckBoxBytesPerPixelOverride->IsChecked()	? FMath::Clamp(BytesPerPixelOverride,		12,	Substrate::GetBytePerPixel(GMaxRHIShaderPlatform))																: -1;
		SubstrateCompilationConfig.ClosuresPerPixelOverride = CheckBoxClosuresPerPixelOverride->IsChecked() ? FMath::Clamp(ClosuresPerPixelOverride,	1,	Substrate::GetClosurePerPixel(GMaxRHIShaderPlatform, Substrate::IgnoreMaterialQualityLevelsForClosurePerPixel))	: -1;

		MaterialInterface->SetSubstrateCompilationConfig(SubstrateCompilationConfig);

		MaterialInterface->ForceRecompileForRendering();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
