// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"

class IMaterialEditor;

class SMaterialEditorSubstrateWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialEditorSubstrateWidget)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<IMaterialEditor> InMaterialEditorPtr);

	void UpdateFromMaterial() { bUpdateRequested = true; }

	/** Gets the widget contents of the app */
	virtual TSharedRef<SWidget> GetContent();

	virtual ~SMaterialEditorSubstrateWidget();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	TSharedPtr<class SCheckBox> CheckBoxForceFullSimplification;
	TSharedPtr<class SCheckBox> CheckBoxBytesPerPixelOverride;
	TSharedPtr<class SCheckBox> CheckBoxClosuresPerPixelOverride;

	TSharedPtr<class SNumericEntryBox<uint32>> BytesPerPixelOverrideInput;
	TSharedPtr<class SNumericEntryBox<uint32>> ClosuresPerPixelOverrideInput;

	TSharedPtr<class SButton> ButtonApplyToPreview;

	TSharedPtr<class STextBlock> MaterialBudgetTextBlock;
	TSharedPtr<class STextBlock> MaterialFeatures0TextBlock;
	TSharedPtr<class STextBlock> MaterialFeatures1TextBlock;
	TSharedPtr<class STextBlock> MaterialFeatures2TextBlock;
	TSharedPtr<class STextBlock> DescriptionTextBlock;

	TSharedPtr<class SBox> MaterialBox;

	FReply OnButtonApplyToPreview();

	/** Pointer back to the material editor that owns this */
	TWeakPtr<IMaterialEditor> MaterialEditorPtr;

	bool bUpdateRequested = true;

	bool bBytesPerPixelStartedTransaction;
	uint32 BytesPerPixelOverride;
	void OnBytesPerPixelChanged(uint32 NewValue);
	void OnBytesPerPixelCommitted(uint32 NewValue, ETextCommit::Type InCommitType);
	void OnBeginBytesPerPixelSliderMovement();
	void OnEndBytesPerPixelSliderMovement(uint32 NewValue);
	TOptional<uint32> GetBytesPerPixelValue() const;

	void OnCheckBoxBytesPerPixelChanged(ECheckBoxState InCheckBoxState);

	bool bClosuresPerPixelStartedTransaction;
	uint32 ClosuresPerPixelOverride;
	void OnClosuresPerPixelChanged(uint32 NewValue);
	void OnClosuresPerPixelCommitted(uint32 NewValue, ETextCommit::Type InCommitType);
	void OnBeginClosuresPerPixelSliderMovement();
	void OnEndClosuresPerPixelSliderMovement(uint32 NewValue);
	TOptional<uint32> GetClosuresPerPixelValue() const;

	void OnCheckBoxClosuresPerPixelChanged(ECheckBoxState InCheckBoxState);
};

