// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialEditorTitleBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SImage.h"

#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"

#include "MaterialGraph/MaterialGraph.h"

#define LOCTEXT_NAMESPACE "SMaterialEditorTitleBar"

void SMaterialEditorTitleBar::Construct(const FArguments& InArgs)
{
	
	EdGraphObj = InArgs._EdGraphObj;
	OnDifferentGraphCrumbClicked = InArgs._OnDifferentGraphCrumbClicked;

	check(EdGraphObj);

	// Set-up shared breadcrumb defaults
	FMargin BreadcrumbTrailPadding = FMargin(4.f, 2.f);
	const FSlateBrush* BreadcrumbButtonImage = FAppStyle::GetBrush("BreadcrumbTrail.Delimiter");
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( TEXT("Graph.TitleBackground") ) )
			.HAlign(HAlign_Fill)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EventGraphTitleBar")))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSeparator)
					.Orientation(Orient_Vertical)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					InArgs._HistoryNavigationWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSeparator)
					.Orientation(Orient_Vertical)
				]
				// Title text/icon
				+SHorizontalBox::Slot()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding( 10.0f,5.0f )
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image( this, &SMaterialEditorTitleBar::GetTypeGlyph ) 
						]
						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						[
							SAssignNew(BreadcrumbTrailScrollBox, SScrollBox)
							.Orientation(Orient_Horizontal)
							.ScrollBarVisibility(EVisibility::Collapsed)

							+SScrollBox::Slot()
							.Padding(0.f)
							.VAlign(VAlign_Center)
							[
								SNew(SHorizontalBox)
								//show fake 'root' breadcrumb for the title
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(BreadcrumbTrailPadding)
								[
									SNew(STextBlock)
									.Text(InArgs._TitleText)
									.TextStyle( FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText") )
									.Visibility( EVisibility::Visible )
								]
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SImage)
									.Image( BreadcrumbButtonImage )
									.Visibility( EVisibility::Visible )
								]

								// New style breadcrumb
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<UEdGraph*>)
									.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
									.TextStyle(FAppStyle::Get(), "GraphBreadcrumbButtonText")
									.ButtonContentPadding( BreadcrumbTrailPadding )
									.DelimiterImage( BreadcrumbButtonImage )
									.PersistentBreadcrumbs( true )
									.OnCrumbClicked( this, &SMaterialEditorTitleBar::OnBreadcrumbClicked )
								]
							]
						]
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SAssignNew(MaterialInfoList, SListView<TSharedPtr<FMaterialInfo>>)
				.ListItemsSource(InArgs._MaterialInfoList)
				.OnGenerateRow(this, &SMaterialEditorTitleBar::MakeMaterialInfoWidget)
				.SelectionMode( ESelectionMode::None )
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

	RebuildBreadcrumbTrail();
	BreadcrumbTrailScrollBox->ScrollToEnd();
}

TSharedRef<ITableRow> SMaterialEditorTitleBar::MakeMaterialInfoWidget(TSharedPtr<FMaterialInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	const int32 FontSize = 9;

	FMaterialInfo Info = *Item.Get();
	FLinearColor TextColor = Info.Color;
	FString Text = Info.Text;

	if( Text.IsEmpty() )
	{
		return
			SNew(STableRow< TSharedPtr<FMaterialInfo> >, OwnerTable)
			[
				SNew(SSpacer)
			];
	}
	else 
	{
		return
			SNew(STableRow< TSharedPtr<FMaterialInfo> >, OwnerTable)
			[
				SNew(STextBlock)
				.ColorAndOpacity(TextColor)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", FontSize))
				.Text(FText::FromString(Text))
			];
	}
}

void SMaterialEditorTitleBar::RequestRefresh()
{
	if (MaterialInfoList)
	{
		MaterialInfoList->RequestListRefresh();
	}

	RebuildBreadcrumbTrail();
}

const FSlateBrush* SMaterialEditorTitleBar::GetTypeGlyph() const
{
	check(EdGraphObj != nullptr);
	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(EdGraphObj);
	return FAppStyle::GetBrush( MaterialGraph->SubgraphExpression ? TEXT("GraphEditor.SubGraph_24x") : TEXT("GraphEditor.EventGraph_24x") );
}

void SMaterialEditorTitleBar::OnBreadcrumbClicked(UEdGraph* const& Item)
{
	OnDifferentGraphCrumbClicked.ExecuteIfBound(Item);
}

void SMaterialEditorTitleBar::RebuildBreadcrumbTrail()
{
	// Build up a stack of graphs so we can pop them in reverse order and create breadcrumbs
	TArray<UEdGraph*> Stack;
	for (UEdGraph* OuterChain = EdGraphObj; OuterChain != nullptr; OuterChain = UEdGraph::GetOuterGraph(OuterChain))
	{
		Stack.Push(OuterChain);
	}

	BreadcrumbTrail->ClearCrumbs(false);

	//Get the last object in the array
	UEdGraph* LastObj = nullptr;
	if( Stack.Num() > 0 )
	{
		LastObj = Stack[Stack.Num() -1];
	}

	while (Stack.Num() > 0)
	{
		UEdGraph* Graph = Stack.Pop();
		
		auto TitleText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&SMaterialEditorTitleBar::GetTitleForOneCrumb, (const UEdGraph*)LastObj, (const UEdGraph*)Graph));

		BreadcrumbTrail->PushCrumb(TitleText, Graph);
	}
}

FText SMaterialEditorTitleBar::GetTitleForOneCrumb(const UEdGraph* BaseGraph, const UEdGraph* CurrGraph)
{
	const UEdGraphSchema* Schema = CurrGraph->GetSchema();

	FGraphDisplayInfo DisplayInfo;
	Schema->GetGraphDisplayInformation(*CurrGraph, /*out*/ DisplayInfo);

	FFormatNamedArguments Args;
	Args.Add(TEXT("BreadcrumbDisplayName"), BaseGraph == CurrGraph ? LOCTEXT("BaseMaterialGraph", "Material Graph") : DisplayInfo.DisplayName);
	Args.Add(TEXT("BreadcrumbNotes"), FText::FromString(DisplayInfo.GetNotesAsString()));
	return FText::Format(LOCTEXT("BreadcrumbTitle", "{BreadcrumbDisplayName} {BreadcrumbNotes}"), Args);
}

#undef LOCTEXT_NAMESPACE