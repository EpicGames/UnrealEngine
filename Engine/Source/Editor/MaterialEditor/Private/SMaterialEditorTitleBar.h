// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "MaterialEditor.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "BlueprintUtilities.h"

class UEdGraph;
class SScrollBox;

//////////////////////////////////////////////////////////////////////////
// SMaterialEditorTitleBar

class SMaterialEditorTitleBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SMaterialEditorTitleBar )
		: _TitleText()
		, _EdGraphObj(nullptr)
		, _MaterialInfoList(nullptr)
	{}

		SLATE_ATTRIBUTE( FText, TitleText )
		SLATE_ARGUMENT( UEdGraph*, EdGraphObj )
		SLATE_ARGUMENT( const TArray<TSharedPtr<FMaterialInfo>>*, MaterialInfoList )
		SLATE_EVENT( FEdGraphEvent, OnDifferentGraphCrumbClicked )
		SLATE_ARGUMENT( TSharedPtr<SWidget>, HistoryNavigationWidget )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback used to populate SListView */
	TSharedRef<ITableRow> MakeMaterialInfoWidget(TSharedPtr<FMaterialInfo> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Request a refresh of the list view */
	void RequestRefresh();

	/** Material info we display when compile errors occur. */
	TSharedPtr<SListView<TSharedPtr<FMaterialInfo>>>	MaterialInfoList;

protected:

	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox;

	/** Edited graph */
	UEdGraph* EdGraphObj;

	/** Breadcrumb trail widget */
	TSharedPtr< SBreadcrumbTrail<UEdGraph*> > BreadcrumbTrail;

	/** Callback to call when the user wants to change the active graph via the breadcrumb trail */
	FEdGraphEvent OnDifferentGraphCrumbClicked;

	/** Get the icon to use */
	const FSlateBrush* GetTypeGlyph() const;

	/** Function to fetch outer class which is of type UEGraph. */
	UEdGraph* GetOuterGraph(UObject* Obj);

	/** Helper methods */
	void OnBreadcrumbClicked(UEdGraph* const& Item);
	void RebuildBreadcrumbTrail();
	static FText GetTitleForOneCrumb(const UEdGraph* BaseGraph, const UEdGraph* CurrGraph);
};
