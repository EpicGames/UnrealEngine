// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialEditorTopologyWidget.h"

#include "EditorWidgetsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MaterialEditor.h"
#include "MaterialInstanceEditor.h"
#include "SubstrateDefinitions.h"
#include "Rendering/SubstrateMaterialShared.h"
#include <functional>

#include "MaterialInstanceEditor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Notifications/SErrorText.h"
#include "AssetThumbnail.h"
#include "MaterialEditor/SGraphSubstrateMaterial.h"
#include "RHIShaderPlatform.h"

#define LOCTEXT_NAMESPACE "SMaterialEditorTopologyWidget"

void SMaterialEditorTopologyWidget::Construct(const FArguments& InArgs, TWeakPtr<IMaterialEditor> InMaterialEditorPtr)
{
	MaterialEditorPtr = InMaterialEditorPtr;


	if (Substrate::IsSubstrateEnabled())
	{
		this->ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ActionableMessage.Border"))
				.BorderBackgroundColor(FLinearColor(0.22f, 0.22f, 0.22f, 0.75f))
				.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FLinearColor::White)
					.ShadowColorAndOpacity(FLinearColor::Black)
					.ShadowOffset(FVector2D::UnitVector)
					.Text(LOCTEXT("MaterialTopology", "Material Topology"))
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f, 0.0f, 0.0f)
				[
					SNew(SWrapBox)
					.UseAllottedSize(true)
					+ SWrapBox::Slot()
					.Padding(10.0f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(MaterialBox, SBox)
					]
				]
			]
		];
	}
	else
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)
				+SWrapBox::Slot()
				.Padding(5.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FLinearColor::Yellow)
					.ShadowColorAndOpacity(FLinearColor::Black)
					.ShadowOffset(FVector2D::UnitVector)
					.Text(LOCTEXT("SubstrateWidgetNotEnable", "Details cannot be shown: Substrate is not enabled for this project (See the project settings window, rendering settings section)."))
				]
			]
		];
	}
}

TSharedRef<SWidget> SMaterialEditorTopologyWidget::GetContent()
{
	return SharedThis(this);
}

SMaterialEditorTopologyWidget::~SMaterialEditorTopologyWidget()
{
}


void SMaterialEditorTopologyWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!bUpdateRequested || !Substrate::IsSubstrateEnabled())
	{
		return;
	}
	bUpdateRequested = false;

	FText SubstrateMaterialDescription;
	if (MaterialEditorPtr.IsValid())
	{
		TSharedPtr<IMaterialEditor> MaterialEditorPinned = MaterialEditorPtr.Pin();
		FMaterialResource* MaterialResource = MaterialEditorPinned->GetMaterialInterface()->GetMaterialResource(GMaxRHIShaderPlatform);

		if (Cast<UMaterial>(MaterialEditorPinned->GetMaterialInterface()))
		{
			TSharedPtr<FMaterialEditor> MaterialEditor = StaticCastSharedPtr<FMaterialEditor>(MaterialEditorPinned);
			UMaterial* MaterialForStats = MaterialEditor->bStatsFromPreviewMaterial ? MaterialEditor->Material : MaterialEditor->OriginalMaterial;
			MaterialResource = MaterialForStats->GetMaterialResource(GMaxRHIShaderPlatform);
		}

		SubstrateMaterialDescription = FText::FromString(FString(TEXT("SubstrateMaterialDescription")));
		
		if (MaterialResource)
		{
			FString MaterialDescription;
			FString MaterialBudget;

			bool bMaterialOutOfBudgetHasBeenSimplified = false;
			FMaterialShaderMap* ShaderMap = MaterialResource->GetGameThreadShaderMap();
			if (ShaderMap)
			{
				const FSubstrateMaterialCompilationOutput& CompilationOutput = ShaderMap->GetSubstrateMaterialCompilationOutput();
				const uint32 FinalPixelByteCount = CompilationOutput.SubstrateUintPerPixel * sizeof(uint32);
				const uint32 FinalPixelClosureCount = CompilationOutput.SubstrateClosureCount;
				bMaterialOutOfBudgetHasBeenSimplified = CompilationOutput.bMaterialOutOfBudgetHasBeenSimplified > 0;

				// Now generate a visual representation of the material from the topology tree of operators.
				if (CompilationOutput.RootOperatorIndex >= 0)
				{
					const FMaterialLayersFunctions* LayersFunctions = MaterialResource->GetMaterialLayers();
					MaterialBox->SetContent(ProcessOperatorAsThumbnails(CompilationOutput, LayersFunctions));
				}
				else
				{
					// The tree does not looks sane so generate a visual error without crashing.
					auto TreeError = SNew(SErrorText)
						.ErrorText(LOCTEXT("TreeError", "Tree Error"))
						.BackgroundColor(FSlateColor(EStyleColor::AccentRed));
					const TSharedRef<SWidget>& TreeErrorAsShared = TreeError->AsShared();
					MaterialBox->SetContent(TreeErrorAsShared);
				}
			}
			else
			{
				MaterialDescription = TEXT("Shader map not found.");
				MaterialBox->SetContent(SNullWidget::NullWidget);
			}

		}
		
	}
}

static const TSharedRef<SWidget> InternalProcessOperatorAsThumbnails(
	const FSubstrateMaterialCompilationOutput& CompilationOutput, 
	const FSubstrateOperator& Op, 
	const TArray<FGuid>& InGuid,
	const FMaterialLayersFunctions* LayersFunctions,
	EStyleColor OverrideColor)
{
	const bool bIsCurrent = false;
	const EStyleColor Color0 = OverrideColor;
	const EStyleColor Color1 = OverrideColor;
	switch (Op.OperatorType)
	{
		case SUBSTRATE_OPERATOR_WEIGHT:
		{
			const EStyleColor Color = bIsCurrent ? EStyleColor::AccentGreen : OverrideColor;
			return InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], InGuid, LayersFunctions, Color);
		}
		case SUBSTRATE_OPERATOR_VERTICAL:
		{
			auto VerticalOperator = SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], InGuid, LayersFunctions, Color0)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.RightIndex], InGuid, LayersFunctions, Color1)
				];
			return VerticalOperator->AsShared();
		}
		case SUBSTRATE_OPERATOR_HORIZONTAL:
		case SUBSTRATE_OPERATOR_SELECT:
		{
			auto HorizontalOperator = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], InGuid, LayersFunctions, Color0)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.RightIndex], InGuid, LayersFunctions, Color1)
				];
			return HorizontalOperator->AsShared();
		}
		case SUBSTRATE_OPERATOR_ADD:
		{
			auto HorizontalOperator = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.LeftIndex], InGuid, LayersFunctions, Color0)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[Op.RightIndex], InGuid, LayersFunctions, Color1)
				];
			return HorizontalOperator->AsShared();
		}
		case SUBSTRATE_OPERATOR_BSDF_LEGACY:	// legacy BSDF should have been converted to BSDF already.
		case SUBSTRATE_OPERATOR_BSDF:
		{
			int32 ExpressionIndex = INDEX_NONE;
			const float ThumbnailSize = 40.0f;
			UObject* ThumbnailObject = nullptr;

			const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable(new FAssetThumbnail(ThumbnailObject, ThumbnailSize, ThumbnailSize, UThumbnailManager::Get().GetSharedThumbnailPool()));
			TSharedRef<SWidget> ThumbnailWidget = SNew(SBorder)
				.Padding(2.0f)
				[
					AssetThumbnail->MakeThumbnailWidget()
				];
			AssetThumbnail->SetRealTime(true);
			return ThumbnailWidget;
		}
	}

	static FString NoVisualization = FString(TEXT("Tree Operator Error"));
	auto TreeOperatorError = SNew(SErrorText)
		.ErrorText(FText::FromString(NoVisualization))
		.BackgroundColor(FSlateColor(EStyleColor::AccentRed));
	return TreeOperatorError->AsShared();
}

const TSharedRef<SWidget> SMaterialEditorTopologyWidget::ProcessOperatorAsThumbnails(const FSubstrateMaterialCompilationOutput& CompilationOutput, const FMaterialLayersFunctions* LayersFunctions)
{
	return InternalProcessOperatorAsThumbnails(CompilationOutput, CompilationOutput.Operators[CompilationOutput.RootOperatorIndex], TArray<FGuid>(), LayersFunctions, EStyleColor::MAX);
}


#undef LOCTEXT_NAMESPACE
