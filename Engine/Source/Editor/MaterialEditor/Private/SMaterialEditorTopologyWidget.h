// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Widgets/SWidget.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"

class IMaterialEditor;

class SMaterialEditorTopologyWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialEditorTopologyWidget)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<IMaterialEditor> InMaterialEditorPtr);

	void UpdateFromMaterial() { bUpdateRequested = true; }

	/** Gets the widget contents of the app */
	virtual TSharedRef<SWidget> GetContent();

	virtual ~SMaterialEditorTopologyWidget();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	static const TSharedRef<SWidget> ProcessOperatorAsThumbnails(const struct FSubstrateMaterialCompilationOutput& CompilationOutput, const struct FMaterialLayersFunctions* LayersFunctions);
	
private:

	TSharedPtr<class SBox> MaterialBox;

	/** Pointer back to the material editor that owns this */
	TWeakPtr<IMaterialEditor> MaterialEditorPtr;

	bool bUpdateRequested = true;

};

