// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialEditorViewport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Components/MeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"
#include "Editor/UnrealEdEngine.h"
#include "MaterialEditor.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "MaterialEditor/PreviewMaterial.h"
#include "Materials/MaterialExpressionUserSceneTexture.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/EnumerateRange.h"
#include "Engine/StaticMesh.h"
#include "SceneInterface.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "MaterialEditorActions.h"
#include "Slate/SceneViewport.h"
#include "MaterialEditor.h"
#include "MaterialInstanceEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Engine/TextureCube.h"
#include "ComponentAssetBroker.h"
#include "Modules/ModuleManager.h"
#include "SlateMaterialBrush.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "AdvancedPreviewScene.h"
#include "AssetViewerSettings.h"
#include "Engine/PostProcessVolume.h"
#include "MaterialEditorSettings.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateBorderBrush.h"
#include "ImageUtils.h"
#include "ISettingsModule.h"
#include "Framework/Layout/ScrollyZoomy.h"
#include "MaterialEditorViewportToolbarSections.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "AssetThumbnail.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#define LOCTEXT_NAMESPACE "MaterialEditor"
#include "AdvancedPreviewSceneMenus.h"
#include "ContentBrowserDataDragDropOp.h"
#include "MaterialEditorTabs.h"
#include "PreviewProfileController.h"
#include "SMaterialEditorTopologyWidget.h"
#include "UnrealWidget.h"
#include "Preferences/MaterialEditorOptions.h"

/** Viewport Client for the preview viewport */
class FMaterialEditorViewportClient : public FEditorViewportClient
{
public:
	FMaterialEditorViewportClient(TWeakPtr<IMaterialEditor> InMaterialEditor, FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SMaterialEditor3DPreviewViewport>& InMaterialEditorViewport);

	// FEditorViewportClient interface
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override;

	/**
	* Focuses the viewport to the center of the bounding box/sphere ensuring that the entire bounds are in view
	*
	* @param Bounds   The bounds to focus
	* @param bInstant Whether or not to focus the viewport instantly or over time
	*/
	void FocusViewportOnBounds(const FBoxSphereBounds Bounds, bool bInstant = false);

private:

	/** Pointer back to the material editor tool that owns us */
	TWeakPtr<IMaterialEditor> MaterialEditorPtr;

	/** Preview Scene - uses advanced preview settings */
	class FAdvancedPreviewScene* AdvancedPreviewScene;
};

FMaterialEditorViewportClient::FMaterialEditorViewportClient(TWeakPtr<IMaterialEditor> InMaterialEditor, FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SMaterialEditor3DPreviewViewport>& InMaterialEditorViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InMaterialEditorViewport))
	, MaterialEditorPtr(InMaterialEditor)
{
	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = false;
	DrawHelper.GridColorAxis = FColor(80,80,80);
	DrawHelper.GridColorMajor = FColor(72,72,72);
	DrawHelper.GridColorMinor = FColor(64,64,64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	
	SetViewMode(VMI_Lit);
	
	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetSeparateTranslucency(true);

	OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = true;

	// Don't want to display the widget in this viewport
	Widget->SetDefaultVisibility(false);

	AdvancedPreviewScene = &InPreviewScene;

}



void FMaterialEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}


void FMaterialEditorViewportClient::Draw(FViewport* InViewport,FCanvas* Canvas)
{
	FEditorViewportClient::Draw(InViewport, Canvas);

	if (MaterialEditorPtr.IsValid())
	{
		MaterialEditorPtr.Pin()->DrawMessages(InViewport, Canvas);
	}
}

bool FMaterialEditorViewportClient::ShouldOrbitCamera() const
{
	// Should always orbit around the preview object to keep it in view.
	return true;
}

bool FMaterialEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = FEditorViewportClient::InputKey(EventArgs);

	// Handle viewport screenshot.
	bHandled |= InputTakeScreenshot(EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

	bHandled |= AdvancedPreviewScene->HandleInputKey(EventArgs);

	return bHandled;
}

bool FMaterialEditorViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	bool bResult = true;

	if (!bDisableInput)
	{
		bResult = AdvancedPreviewScene->HandleViewportInput(Args.Viewport, Args.InputDevice, Args.Key, Args.AmountDepressed, Args.DeltaTime, Args.NumSamples, Args.IsGamepad());
		if (bResult)
		{
			Invalidate();
		}
		else
		{
			bResult = FEditorViewportClient::InputAxis(Args);
		}
	}

	return bResult;
}

FLinearColor FMaterialEditorViewportClient::GetBackgroundColor() const
{
	if (AdvancedPreviewScene != nullptr)
	{
		return AdvancedPreviewScene->GetBackgroundColor();
	}
	else
	{
		FLinearColor BackgroundColor = FLinearColor::Black;
		if (MaterialEditorPtr.IsValid())
		{
			UMaterialInterface* MaterialInterface = MaterialEditorPtr.Pin()->GetMaterialInterface();
			if (MaterialInterface)
			{
				const EBlendMode PreviewBlendMode = (EBlendMode)MaterialInterface->GetBlendMode();
				if (IsModulateBlendMode(*MaterialInterface))
				{
					BackgroundColor = FLinearColor::White;
				}
				else if (IsTranslucentOnlyBlendMode(*MaterialInterface) || IsAlphaCompositeBlendMode(*MaterialInterface) || IsAlphaHoldoutBlendMode(*MaterialInterface))
				{
					BackgroundColor = FColor(64, 64, 64);
				}
			}
		}
		return BackgroundColor;
	}
}

void FMaterialEditorViewportClient::FocusViewportOnBounds(const FBoxSphereBounds Bounds, bool bInstant /*= false*/)
{
	const FVector Position = Bounds.Origin;
	float Radius = Bounds.SphereRadius;

	float AspectToUse = AspectRatio;
	FIntPoint ViewportSize = Viewport->GetSizeXY();
	if (!bUseControllingActorViewInfo && ViewportSize.X > 0 && ViewportSize.Y > 0)
	{
		AspectToUse = Viewport->GetDesiredAspectRatio();
	}

	const bool bEnable = false;
	ToggleOrbitCamera(bEnable);

	/**
	* We need to make sure we are fitting the sphere into the viewport completely, so if the height of the viewport is less
	* than the width of the viewport, we scale the radius by the aspect ratio in order to compensate for the fact that we have
	* less visible vertically than horizontally.
	*/
	if (AspectToUse > 1.0f)
	{
		Radius *= AspectToUse;
	}

	/**
	* Now that we have a adjusted radius, we are taking half of the viewport's FOV,
	* converting it to radians, and then figuring out the camera's distance from the center
	* of the bounding sphere using some simple trig.  Once we have the distance, we back up
	* along the camera's forward vector from the center of the sphere, and set our new view location.
	*/
	const float HalfFOVRadians = FMath::DegreesToRadians(ViewFOV / 2.0f);
	const float DistanceFromSphere = Radius / FMath::Sin(HalfFOVRadians);
	FViewportCameraTransform& ViewTransform = GetViewTransform();
	FVector CameraOffsetVector = ViewTransform.GetRotation().Vector() * -DistanceFromSphere;

	ViewTransform.SetLookAt(Position);
	ViewTransform.TransitionToLocation(Position + CameraOffsetVector, EditorViewportWidget, bInstant);

	// Tell the viewport to redraw itself.
	Invalidate();
}

void SMaterialEditor3DPreviewViewport::Construct(const FArguments& InArgs)
{
	MaterialEditorPtr = InArgs._MaterialEditor;
	PreviewMaterialPtr = InArgs._PreviewMaterial;

	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));

	// restore last used feature level
	UWorld* PreviewWorld = AdvancedPreviewScene->GetWorld();
	if (PreviewWorld != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		PreviewWorld->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}	

	UEditorEngine* Editor = CastChecked<UEditorEngine>(GEngine);
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
		{
			AdvancedPreviewScene->GetWorld()->ShaderPlatformChanged(NewShaderPlatform);
		});


	PreviewPrimType = TPT_None;

	SEditorViewport::Construct( SEditorViewport::FArguments() );

	PreviewMaterial = nullptr;
	PreviewMeshComponent = nullptr;
	PostProcessVolumeActor = nullptr;

	UMaterialInterface* Material = GetPreviewingMaterialInterface();
	if (Material)
	{
		SetPreviewMaterial(Material);
	}

	//If no preview mesh has been found for this material, use the default editor option mesh.
	if (const UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
	{
		SetPreviewAsset(MaterialEditorOptions->GetPreviewMesh());
	}	

	OnPropertyChangedHandle = FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate::CreateRaw(this, &SMaterialEditor3DPreviewViewport::OnPropertyChanged);
	OnPropertyChangedHandleDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.Add(OnPropertyChangedHandle);
	
	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(AdvancedPreviewScene, EditorViewportClient);
}

SMaterialEditor3DPreviewViewport::~SMaterialEditor3DPreviewViewport()
{
	CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);
	
	if (PreviewMeshComponent != nullptr)
	{
		PreviewMeshComponent->OverrideMaterials.Empty();
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnPropertyChangedHandleDelegateHandle);

	PostProcessVolumeActor = nullptr;
}

void SMaterialEditor3DPreviewViewport::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( PreviewMeshComponent );
	Collector.AddReferencedObject( PreviewMaterial );
	Collector.AddReferencedObject( PostProcessVolumeActor );
}

void SMaterialEditor3DPreviewViewport::RefreshViewport()
{
	// reregister the preview components, so if the preview material changed it will be propagated to the render thread
	if (PreviewMeshComponent != nullptr)
	{
		PreviewMeshComponent->MarkRenderStateDirty();
	}
	SceneViewport->InvalidateDisplay();

	if (EditorViewportClient.IsValid() && AdvancedPreviewScene.IsValid())
	{
		UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
		const int32 ProfileIndex = AdvancedPreviewScene->GetCurrentProfileIndex();
		if (Settings->Profiles.IsValidIndex(ProfileIndex))
		{
			AdvancedPreviewScene->UpdateScene(Settings->Profiles[ProfileIndex]);
			if(Settings->Profiles[ProfileIndex].bRotateLightingRig && !EditorViewportClient->IsRealtime())
			{
				EditorViewportClient->SetRealtime(true);
			}
		}
	}

	// Also request to update the Substrate slab.
	if (SubstrateWidget)
		SubstrateWidget->UpdateFromMaterial();

}

bool SMaterialEditor3DPreviewViewport::SetPreviewAsset(UObject* InAsset)
{
	UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get();
	if (MaterialEditorPtr.IsValid() && !MaterialEditorPtr.Pin()->ApproveSetPreviewAsset(InAsset))
	{
		//If an invalid object is selected, revert usage to default asset.
		if (MaterialEditorOptions)
		{
			InAsset = MaterialEditorOptions->GetPreviewMesh();
		}
		else
		{
			InAsset = GUnrealEd->GetThumbnailManager()->EditorSphere;
		}
	}

	if (InAsset == nullptr)
	{
		// No change
		return true;
	}

	// This should never occur since we are comparing a mesh against a component...
	if (PreviewMeshComponent == InAsset)
	{
		// No change
		return true;
	}

	if (PreviewMeshComponent && FComponentAssetBrokerage::GetAssetFromComponent(PreviewMeshComponent) == InAsset)
	{
		// No change
		return true;
	}

	// Unregister the current component
	if (PreviewMeshComponent != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(PreviewMeshComponent);
		PreviewMeshComponent = nullptr;
	}

	FTransform Transform = FTransform::Identity;

	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(InAsset))
	{
		// Special case handling for static meshes, to use more accurate bounds via a subclass
		UStaticMeshComponent* NewSMComponent = NewObject<UMaterialEditorMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		NewSMComponent->SetStaticMesh(StaticMesh);

		PreviewMeshComponent = NewSMComponent;

		// Update the toolbar state implicitly through PreviewPrimType.
		if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorCylinder)
		{
			PreviewPrimType = TPT_Cylinder;
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorCube)
		{
			PreviewPrimType = TPT_Cube;
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorSphere)
		{
			PreviewPrimType = TPT_Sphere;
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorPlane)
		{
			PreviewPrimType = TPT_Plane;
			// Update the rotation of the plane mesh so that it is front facing to the viewport camera's default forward view.
			const FRotator PlaneRotation(0.0f, 180.0f, 0.0f);
			Transform.SetRotation(FQuat(PlaneRotation));
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorShaderBall)
		{
			PreviewPrimType = TPT_ShaderBall;
		}
		else
		{
			PreviewPrimType = TPT_None;
		}
	}
	else if (InAsset != nullptr)
	{
		// Fall back to the component asset broker
		if (TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(InAsset->GetClass()))
		{
			if (ComponentClass->IsChildOf(UMeshComponent::StaticClass()))
			{
				PreviewMeshComponent = NewObject<UMeshComponent>(GetTransientPackage(), ComponentClass, NAME_None, RF_Transient);

				FComponentAssetBrokerage::AssignAssetToComponent(PreviewMeshComponent, InAsset);

				PreviewPrimType = TPT_None;
			}
		}
	}

	// Add the new component to the scene
	if (PreviewMeshComponent != nullptr)
	{
		if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
		{
			PreviewMeshComponent->SetMobility(EComponentMobility::Static);
		}
		AdvancedPreviewScene->AddComponent(PreviewMeshComponent, Transform);
		AdvancedPreviewScene->SetFloorOffset(-PreviewMeshComponent->Bounds.Origin.Z + PreviewMeshComponent->Bounds.BoxExtent.Z);

		//Repopulate the material slot states as the mesh option is selected
		MaterialSlots.Empty();
		bool bValidMesh = false;
		if (UStaticMesh* SM = Cast<UStaticMesh>(InAsset))
		{
			const TArray<FStaticMaterial>& StaticMatInfos = SM->GetStaticMaterials();
			for (const TConstEnumerateRef<FStaticMaterial> Slot : EnumerateRange(StaticMatInfos))
			{
				const bool bEnabled = MaterialEditorOptions ? MaterialEditorOptions->PreviewMeshMaterialSlots.Contains(Slot.GetIndex()) : true;
				MaterialSlots.Add({ Slot->MaterialSlotName, Slot->MaterialInterface, bEnabled});
			}
			bValidMesh = true;
		}
		else if (USkeletalMesh* SK = Cast<USkeletalMesh>(InAsset))
		{
			const TArray<FSkeletalMaterial>& SkeletalMatInfos = SK->GetMaterials();
			for (const TConstEnumerateRef<FSkeletalMaterial> Slot : EnumerateRange(SkeletalMatInfos))
			{
				const bool bEnabled = MaterialEditorOptions ? MaterialEditorOptions->PreviewMeshMaterialSlots.Contains(Slot.GetIndex()) : true;
				MaterialSlots.Add({ Slot->MaterialSlotName, Slot->MaterialInterface, bEnabled });
			}
			bValidMesh = true;
		}

		if (bValidMesh)
		{
			if (PreviewPrimType != TPT_None)
			{
				if (PreviewMaterial)
				{
					PreviewMaterial->PreviewMesh = nullptr;
				}
			}
			else
			{
				if (PreviewMaterial)
				{
					PreviewMaterial->PreviewMesh = InAsset->GetPathName();
				}
			}
			
			// Save the preview mesh as default
			if (MaterialEditorOptions)
			{
				MaterialEditorOptions->SetPreviewMesh(InAsset);
			}
		}
	}

	// Make sure the preview material is applied to the component
	SetPreviewMaterial(PreviewMaterial);

	return (PreviewMeshComponent != nullptr);
}

bool SMaterialEditor3DPreviewViewport::SetPreviewAssetByName(const TCHAR* InAssetName)
{
	bool bSuccess = false;
	if ((InAssetName != nullptr) && (*InAssetName != 0))
	{
		if (UObject* Asset = LoadObject<UObject>(nullptr, InAssetName))
		{
			bSuccess = SetPreviewAsset(Asset);
		}
	}
	return bSuccess;
}

// Add user scene texture inputs from a material.  Doesn't clear TSet first, so can be used to accumulate inputs from multiple materials.
static void GetUserSceneTextureInputs(UMaterialInterface* Material, TSet<FName>& OutUserSceneTextures)
{
	UMaterial* BaseMaterial = Material->GetBaseMaterial();
	if (BaseMaterial)
	{
		// Get inputs from base material.  TMap key stores input, value stores instance override if present.
		TMap<FName, FName> NewInputs;
		TArray<const UMaterialExpressionUserSceneTexture*> UserSceneTextureExpressions;
		BaseMaterial->GetAllExpressionsInMaterialAndFunctionsOfType(UserSceneTextureExpressions);

		for (const UMaterialExpressionUserSceneTexture* UserSceneTextureExpression : UserSceneTextureExpressions)
		{
			if (!UserSceneTextureExpression->UserSceneTexture.IsNone())
			{
				NewInputs.Add(UserSceneTextureExpression->UserSceneTexture, NAME_None);
			}
		}

		// Then get any overrides from material instances
		TSet<UMaterialInterface*> MaterialDependencies;
		Material->GetDependencies(MaterialDependencies);

		for (UMaterialInterface* MaterialDependency : MaterialDependencies)
		{
			UMaterialInstanceConstant* MaterialInstance = Cast<UMaterialInstanceConstant>(MaterialDependency);
			if (MaterialInstance)
			{
				for (FUserSceneTextureOverride& Override : MaterialInstance->UserSceneTextureOverrides)
				{
					FName* FoundInput = NewInputs.Find(Override.Key);

					// Only accept the first override of a given value
					if (FoundInput && FoundInput->IsNone())
					{
						*FoundInput = Override.Value;
					}
				}
			}
		}

		// Finally, add the inputs to the output
		for (auto NewInputIterator : NewInputs)
		{
			if (!NewInputIterator.Value.IsNone())
			{
				OutUserSceneTextures.Add(NewInputIterator.Value);
			}
			else
			{
				OutUserSceneTextures.Add(NewInputIterator.Key);
			}
		}
	}
}

static bool IsParentOfEditedMaterialInstance(const FMaterialEditor* MaterialEditor, const TArray<UMaterialInstanceConstant*>& EditedMaterialInstances)
{
	for (UMaterialInstanceConstant* OtherInstance : EditedMaterialInstances)
	{
		UMaterial* BaseMaterial = OtherInstance->GetBaseMaterial();
		if (BaseMaterial == MaterialEditor->Material || BaseMaterial == MaterialEditor->OriginalMaterial)
		{
			return true;
		}
	}
	return false;
}

static bool IsParentOfEditedMaterialInstance(UMaterialInstanceConstant* EditedMaterialInstance, const TArray<UMaterialInstanceConstant*>& EditedMaterialInstances)
{
	for (UMaterialInstanceConstant* OtherInstance : EditedMaterialInstances)
	{
		if (OtherInstance != EditedMaterialInstance)
		{
			FMaterialInheritanceChain OtherInheritanceChain;
			OtherInstance->GetMaterialInheritanceChain(OtherInheritanceChain);

			if (OtherInheritanceChain.MaterialInstances.Contains(EditedMaterialInstance))
			{
				return true;
			}
		}
	}
	return false;
}

// Recursively get all edited materials that have UserSceneTexture outputs that feed into Material.  Also returns inputs that
// are missing, which may be useful to report as warnings to the log in the future.
static void GetUserSceneTextureDependencies(UMaterialInterface* Material, TSet<UMaterialInterface*>& OutDependencies, TSet<FName>& OutMissingInputs)
{
	// Check if the current material has any UserSceneTexture inputs first
	TSet<FName> InputsToProcess;
	GetUserSceneTextureInputs(Material, InputsToProcess);
	if (InputsToProcess.IsEmpty())
	{
		return;
	}

	// Generate a global list of edited materials that generate a given UserSceneTexture output (minus Material itself)
	TMap<FName, TSet<UMaterialInterface*>> MaterialsByUserSceneTextureOutput;

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();

	// Get a list of material instances first -- if a material or instance is a parent of other loaded instances, only consider the outermost child instance
	TArray<UMaterialInstanceConstant*> EditedMaterialInstances;
	for (UObject* EditedAsset : EditedAssets)
	{
		UMaterialInstanceConstant* EditedMaterialInstance = Cast<UMaterialInstanceConstant>(EditedAsset);
		if (EditedMaterialInstance)
		{
			EditedMaterialInstances.Add(EditedMaterialInstance);
		}
	}

	for (UObject* EditedAsset : EditedAssets)
	{
		UPreviewMaterial* EditedMaterial = Cast<UPreviewMaterial>(EditedAsset);

		if (EditedMaterial && EditedMaterial != Material && EditedMaterial->IsPostProcessMaterial() && !EditedMaterial->UserSceneTexture.IsNone())
		{
			TArray<IAssetEditorInstance*> Editors = AssetEditorSubsystem->FindEditorsForAsset(EditedAsset);
			if (!Editors.IsEmpty() && Editors[0]->GetEditorName() == FName("MaterialEditor"))
			{
				const FMaterialEditor* MaterialEditor = (const FMaterialEditor*)Editors[0];

				// If we are editing a material instance and its parent, we only want the child instance to be previewed.  Previewing multiple
				// copies of the same base material would cause confusing and indeterminate results.  We pass in the MaterialEditor rather than
				// the material, so we can check against both original and previewed variations of the material.
				if (!MaterialEditor->bDestructing && !IsParentOfEditedMaterialInstance(MaterialEditor, EditedMaterialInstances))
				{
					MaterialsByUserSceneTextureOutput.FindOrAdd(EditedMaterial->UserSceneTexture).Add(EditedMaterial);
				}
			}
		}

		UMaterialInstanceConstant* EditedMaterialInstance = Cast<UMaterialInstanceConstant>(EditedAsset);

		// If we are editing a material instance and its parent, we only want the child instance to be previewed.
		if (EditedMaterialInstance && EditedMaterialInstance != Material && !IsParentOfEditedMaterialInstance(EditedMaterialInstance, EditedMaterialInstances))
		{
			UMaterial* BaseMaterial = EditedMaterialInstance->GetMaterial();

			if (BaseMaterial->IsPostProcessMaterial())
			{
				FName UserSceneTextureOutput = EditedMaterialInstance->GetUserSceneTextureOutput(BaseMaterial);
				if (UserSceneTextureOutput != NAME_None)
				{
					TArray<IAssetEditorInstance*> Editors = AssetEditorSubsystem->FindEditorsForAsset(EditedAsset);
					if (!Editors.IsEmpty() && Editors[0]->GetEditorName() == FName("MaterialInstanceEditor"))
					{
						const FMaterialInstanceEditor* MaterialInstanceEditor = (const FMaterialInstanceEditor*)Editors[0];

						if (!MaterialInstanceEditor->IsDestructing())
						{
							MaterialsByUserSceneTextureOutput.FindOrAdd(UserSceneTextureOutput).Add(EditedMaterialInstance);
						}
					}
				}
			}
		}
	}

	// Recursively process materials that generate inputs we care about.  InputsToProcess starts with the inputs from the original
	// material, and accumulates inputs from other encountered materials.  Stops when no new unique elements get added to InputsToProcess.
	for (int32 ElementIndex = 0; ElementIndex < InputsToProcess.Num(); ++ElementIndex)
	{
		// Find materials that generate an input we care about
		FName Input = InputsToProcess[FSetElementId::FromInteger(ElementIndex)];
		TSet<UMaterialInterface*>* MaterialsGeneratingInput = MaterialsByUserSceneTextureOutput.Find(Input);

		if (MaterialsGeneratingInput)
		{
			// Add the materials to the dependency list
			OutDependencies.Append(*MaterialsGeneratingInput);

			// Add any inputs the new dependencies require
			for (UMaterialInterface* MaterialGeneratingOutput : *MaterialsGeneratingInput)
			{
				GetUserSceneTextureInputs(MaterialGeneratingOutput, InputsToProcess);
			}
		}
		else
		{
			OutMissingInputs.Add(Input);
		}
	}
}

void SMaterialEditor3DPreviewViewport::SetPreviewMaterial(UMaterialInterface* InMaterialInterface)
{
	if (!InMaterialInterface)
	{
		return;
	}

	PreviewMaterial = InMaterialInterface;

	// Spawn post processing volume actor if the material has post processing as domain.
	if (PreviewMaterial->GetMaterial()->IsPostProcessMaterial())
	{
		if (PostProcessVolumeActor == nullptr)
		{
			PostProcessVolumeActor = GetWorld()->SpawnActor<APostProcessVolume>(APostProcessVolume::StaticClass(), FTransform::Identity);

			GetViewportClient()->EngineShowFlags.SetPostProcessing(true);
			GetViewportClient()->EngineShowFlags.SetPostProcessMaterial(true);
		}

		// Clear blendables, and re-add them (cleans up any post process materials with UserSceneTextures that are no longer used or loaded)
		PostProcessVolumeActor->Settings.WeightedBlendables.Array.Empty(1);

		{
			// Add any edited post process materials that write UserSceneTextures used by this material, for better visualization.
			// We want to add these before the main material, so the main material renders last (assuming equal priority).
			TSet<UMaterialInterface*> UserSceneTextureDependencies;
			TSet<FName> UserSceneTextureMissingInputs;

			GetUserSceneTextureDependencies(PreviewMaterial, UserSceneTextureDependencies, UserSceneTextureMissingInputs);

			// Add dependencies in reverse order, as dependency tree is traversed backwards from the main material's inputs, so earlier
			// dependencies tend to end up later in the TSet.  This isn't a perfect dependency sort, but works correctly in typical cases.
			// The user should set Blendable Priority as needed to sort in complex cases.
			for (int32 DependencyIndex = UserSceneTextureDependencies.Num() - 1; DependencyIndex >= 0; --DependencyIndex)
			{
				UMaterialInterface* Dependency = UserSceneTextureDependencies[FSetElementId::FromInteger(DependencyIndex)];
				PostProcessVolumeActor->AddOrUpdateBlendable(Dependency);
			}
		}

		check (PreviewMaterial != nullptr);
		PostProcessVolumeActor->AddOrUpdateBlendable(PreviewMaterial);
		PostProcessVolumeActor->bEnabled = true;
		PostProcessVolumeActor->BlendWeight = 1.0f;
		PostProcessVolumeActor->bUnbound = true;

		// Setting this forces this post process material to write to SceneColor instead of any UserSceneTexture it may have assigned, for preview purposes
		PostProcessVolumeActor->Settings.PreviewBlendable = PreviewMaterial;

		// Remove preview material from the preview mesh.
		if (PreviewMeshComponent != nullptr)
		{
			PreviewMeshComponent->OverrideMaterials.Empty();
			PreviewMeshComponent->MarkRenderStateDirty();
		}

		GetViewportClient()->RedrawRequested(GetSceneViewport().Get());
	}
	else
	{
		// Add the preview material to the preview mesh.
		if (PreviewMeshComponent != nullptr)
		{
			int32 NumMaterialSlots = PreviewMeshComponent->GetNumMaterials();
			if (NumMaterialSlots == MaterialSlots.Num())
			{
				//Ensure the override materials has enough array slots to apply the preview material
				PreviewMeshComponent->OverrideMaterials.SetNum(NumMaterialSlots, EAllowShrinking::Yes);
				for (int32 SlotIndex = 0; SlotIndex < NumMaterialSlots; SlotIndex++)
				{
					PreviewMeshComponent->OverrideMaterials[SlotIndex] = MaterialSlots[SlotIndex].bSlotEnabled ? PreviewMaterial : nullptr;
					
					if (UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
					{
						if (MaterialSlots[SlotIndex].bSlotEnabled)
						{
							MaterialEditorOptions->PreviewMeshMaterialSlots.Add(SlotIndex);
						}
						else
						{
							MaterialEditorOptions->PreviewMeshMaterialSlots.Remove(SlotIndex);
						}
					}
				}
				PreviewMeshComponent->MarkRenderStateDirty();
			}
		}
		
		PostProcessVolumeActor = nullptr;
	}

	FMaterialEditor::UpdateThumbnailInfoPreviewMesh(PreviewMaterial);
	RefreshViewport();
}

void SMaterialEditor3DPreviewViewport::OnAddedToTab( const TSharedRef<SDockTab>& OwnerTab )
{
	ParentTab = OwnerTab;
}

bool SMaterialEditor3DPreviewViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground()) && SEditorViewport::IsVisible() ;
}

void SMaterialEditor3DPreviewViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FMaterialEditorCommands& Commands = FMaterialEditorCommands::Get();

	if(MaterialEditorPtr.IsValid())
	{
		CommandList->Append(MaterialEditorPtr.Pin()->GetToolkitCommands());
	}

	// Add the commands to the toolkit command list so that the toolbar buttons can find them
	CommandList->MapAction(
		Commands.SetCylinderPreview,
		FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::OnSetPreviewPrimitive, TPT_Cylinder),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsPreviewPrimitiveChecked, TPT_Cylinder ) );

	CommandList->MapAction(
		Commands.SetSpherePreview,
		FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::OnSetPreviewPrimitive, TPT_Sphere),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsPreviewPrimitiveChecked, TPT_Sphere ) );

	CommandList->MapAction(
		Commands.SetPlanePreview,
		FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::OnSetPreviewPrimitive, TPT_Plane),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsPreviewPrimitiveChecked, TPT_Plane ) );

	CommandList->MapAction(
		Commands.SetCubePreview,
		FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::OnSetPreviewPrimitive, TPT_Cube),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsPreviewPrimitiveChecked, TPT_Cube ) );

	CommandList->MapAction(
    	Commands.SetShaderBallPreview,
    	FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::OnSetPreviewPrimitive, TPT_ShaderBall),
    	FCanExecuteAction(),
    	FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsPreviewPrimitiveChecked, TPT_ShaderBall ) );

	CommandList->MapAction(
		Commands.SetPreviewMeshFromSelection,
		FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::OnSetPreviewMeshFromSelection ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsPreviewMeshFromSelectionChecked ) );

	CommandList->MapAction(
		Commands.SelectPreviewMeshMaterialSlots,
		FExecuteAction::CreateSP(this, &SMaterialEditor3DPreviewViewport::OnSetPreviewMeshMaterialSlots),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMaterialEditor3DPreviewViewport::DoesPreviewMeshContainSlots));

	CommandList->MapAction(
		Commands.TogglePreviewBackground,
		FExecuteAction::CreateSP( this, &SMaterialEditor3DPreviewViewport::TogglePreviewBackground ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SMaterialEditor3DPreviewViewport::IsTogglePreviewBackgroundChecked ) );
}

void SMaterialEditor3DPreviewViewport::OnFocusViewportToSelection()
{
	if( PreviewMeshComponent != nullptr )
	{
		EditorViewportClient->FocusViewportOnBounds( PreviewMeshComponent->Bounds );
	}
}

FReply SMaterialEditor3DPreviewViewport::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InEvent)
{
	if (const TSharedPtr<FContentBrowserDataDragDropOp> ContentBrowserAsset = InEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
	{
		if (!ContentBrowserAsset->GetDraggedFiles().IsEmpty())
		{
			for (const FContentBrowserItem& Item : ContentBrowserAsset->GetDraggedFiles())
			{
				FAssetData AssetData;
				if (Item.Legacy_TryGetAssetData(AssetData))
				{
					// Only load class first to validate
					if (const UClass* Class = AssetData.GetClass(EResolveClass::Yes))
					{
						if (Class->IsChildOf<UStaticMesh>() || Class->IsChildOf<USkeletalMesh>())
						{
							// Load asset now
							if (UObject* Asset = AssetData.GetAsset())
							{
								SetPreviewAsset(Asset);
								
								// Save the preview mesh as default
								if (UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
								{
									MaterialEditorOptions->SetPreviewMesh(Asset);
								}

								// Notify the change
								RefreshViewport();

								return FReply::Handled();
							}
						}
					}
				}
			}
		}
	}
	return FReply::Unhandled();
}

UMaterialInterface* SMaterialEditor3DPreviewViewport::GetPreviewingMaterialInterface()
{
	return MaterialEditorPtr.IsValid() ? MaterialEditorPtr.Pin()->GetMaterialInterface() : PreviewMaterialPtr.Get();
}

void SMaterialEditor3DPreviewViewport::OnSetPreviewPrimitive(EThumbnailPrimType PrimType)
{
	if (SceneViewport.IsValid())
	{
		UObject* Primitive = nullptr;
		switch (PrimType)
		{
		case TPT_None:
			{
				if (const UMaterialEditorOptions* MaterialEditorOptions = UMaterialEditorOptions::Get())
				{
					Primitive = MaterialEditorOptions->GetPreviewMesh();
					
					// If invalid, take the next preview mesh otherwise break
					if (!!Primitive)
					{
						break;
					}
				}
			}
		case TPT_Sphere: Primitive = GUnrealEd->GetThumbnailManager()->EditorSphere; break;
		case TPT_Cube: Primitive = GUnrealEd->GetThumbnailManager()->EditorCube; break;
		case TPT_Plane: Primitive = GUnrealEd->GetThumbnailManager()->EditorPlane; break;
		case TPT_Cylinder: Primitive = GUnrealEd->GetThumbnailManager()->EditorCylinder; break;
		case TPT_ShaderBall: Primitive = GUnrealEd->GetThumbnailManager()->EditorShaderBall; break;
		}

		if (Primitive != nullptr)
		{
			SetPreviewAsset(Primitive);
		}
	}
}

bool SMaterialEditor3DPreviewViewport::IsPreviewPrimitiveChecked(EThumbnailPrimType PrimType) const
{
	return PreviewPrimType == PrimType;
}

void SMaterialEditor3DPreviewViewport::OnSetPreviewMeshFromSelection()
{
	UMaterialInterface* MaterialInterface = GetPreviewingMaterialInterface();
	if (!MaterialInterface)
	{
		return;
	}

	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
	UObject* InitialSelection = nullptr;
	// Look for a selected asset that can be converted to a mesh component
	for (FSelectionIterator SelectionIt(*GEditor->GetSelectedObjects()); SelectionIt; ++SelectionIt)
	{
		UObject* TestAsset = *SelectionIt;
		if (TestAsset && (TestAsset->IsA<UStaticMesh>() || TestAsset->IsA<USkeletalMesh>()))
		{
			InitialSelection = TestAsset;
			break;
		}
	}

	if (InitialSelection)
	{
		SetPreviewAsset(InitialSelection);
	}

	FContentBrowserModule& CBModule =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig PickerConfig;

	// Filter: allow only static & skeletal meshes
	PickerConfig.Filter.ClassPaths.Add(UStaticMesh::StaticClass()->GetClassPathName());
	PickerConfig.Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());

	PickerConfig.InitialAssetSelection = InitialSelection;
	PickerConfig.bAllowNullSelection = false;
	PickerConfig.bFocusSearchBoxWhenOpened = true;
	PickerConfig.InitialAssetViewType = EAssetViewType::Tile;

	PickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(
		[this, MaterialInterface](const FAssetData& AssetData)
		{
			if (UObject* SelectedAsset = AssetData.GetAsset())
			{
				SetPreviewAsset(SelectedAsset);
			}
		}
	);

	TSharedRef<SWidget> AssetPicker =
		CBModule.Get().CreateAssetPicker(PickerConfig);

	TSharedRef<SWidget> WrappedPicker =
		SNew(SBorder)
		.Padding(FMargin(6.f))
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		[
			SNew(SBox)
				.WidthOverride(400.0f)
				.HeightOverride(400.0f)
				[
					AssetPicker
				]
		];

	const FVector2D ScreenPos = FSlateApplication::Get().GetCursorPos();
	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		WrappedPicker,
		ScreenPos,
		FPopupTransitionEffect(FPopupTransitionEffect::SubMenu));
}

bool SMaterialEditor3DPreviewViewport::IsPreviewMeshFromSelectionChecked() const
{
	return (PreviewPrimType == TPT_None && PreviewMeshComponent != nullptr);
}

void SMaterialEditor3DPreviewViewport::OnSetPreviewMeshMaterialSlots()
{
	TSharedRef<SWidget> MenuContent = GenerateMaterialSlotMenu();
	TSharedRef<SWidget> OwnerWidget = AsShared();

	const FVector2D ScreenPos = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		OwnerWidget,
		FWidgetPath(),
		MenuContent,
		ScreenPos,
		FPopupTransitionEffect(FPopupTransitionEffect::SubMenu)
	);
}

bool SMaterialEditor3DPreviewViewport::DoesPreviewMeshContainSlots() const
{
	return PreviewMeshComponent != nullptr && PreviewMeshComponent->GetNumMaterials() > 0;
}

void SMaterialEditor3DPreviewViewport::TogglePreviewBackground()
{
	UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
	const int32 ProfileIndex = AdvancedPreviewScene->GetCurrentProfileIndex();
	if (Settings->Profiles.IsValidIndex(ProfileIndex))
	{
		AdvancedPreviewScene->SetEnvironmentVisibility(!Settings->Profiles[ProfileIndex].bShowEnvironment);
	}
	RefreshViewport();
}

bool SMaterialEditor3DPreviewViewport::IsTogglePreviewBackgroundChecked() const
{
	UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
	const int32 ProfileIndex = AdvancedPreviewScene->GetCurrentProfileIndex();
	if (Settings->Profiles.IsValidIndex(ProfileIndex))
	{
		return Settings->Profiles[ProfileIndex].bShowEnvironment;
	}
	return false;
}

TSharedRef<class SEditorViewport> SMaterialEditor3DPreviewViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SMaterialEditor3DPreviewViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SMaterialEditor3DPreviewViewport::OnFloatingButtonClicked()
{
}

TSharedRef<FEditorViewportClient> SMaterialEditor3DPreviewViewport::MakeEditorViewportClient() 
{
	EditorViewportClient = MakeShareable( new FMaterialEditorViewportClient(MaterialEditorPtr, *AdvancedPreviewScene.Get(), SharedThis(this)) );
	EditorViewportClient->SetViewLocation( FVector::ZeroVector );
	EditorViewportClient->SetViewRotation( FRotator(-15.0f, -90.0f, 0.0f) );
	EditorViewportClient->SetViewLocationForOrbiting( FVector::ZeroVector );
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->EngineShowFlags.EnableAdvancedFeatures();
	EditorViewportClient->EngineShowFlags.SetLighting(true);
	EditorViewportClient->EngineShowFlags.SetIndirectLightingCache(true);
	EditorViewportClient->EngineShowFlags.SetPostProcessing(true);
	EditorViewportClient->Invalidate();
	EditorViewportClient->VisibilityDelegate.BindSP( this, &SMaterialEditor3DPreviewViewport::IsVisible );

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SMaterialEditor3DPreviewViewport::BuildViewportToolbar()
{
	const FName ViewportToolbarName = "MaterialEditor.ViewportToolbar";

	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowLensControls()));

			RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());

			// Add Preview Scene Submenu
			{
				const FName AssetViewerProfileMenuName = "MaterialEditor.ViewportToolbar.AssetViewerProfile";
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
				UE::MaterialEditor::ExtendPreviewSceneSettingsSubmenu(AssetViewerProfileMenuName);
				UE::AdvancedPreviewScene::Menus::ExtendAdvancedPreviewSceneSettings(
					AssetViewerProfileMenuName,
					UE::AdvancedPreviewScene::Menus::FSettingsOptions()
				);
				UE::UnrealEd::ExtendPreviewSceneSettingsWithTabEntry(AssetViewerProfileMenuName);
			}
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(AdvancedPreviewScene->GetCommandList());
		ViewportToolbarContext.AppendCommandList(GetCommandList());
		
		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));

			if (TSharedPtr<IMaterialEditor> MaterialEditorPinned = MaterialEditorPtr.Pin())
			{
				if (Cast<UMaterial>(MaterialEditorPinned->GetMaterialInterface()))
				{
					ContextObject->PreviewSettingsTabId = FMaterialEditorTabs::PreviewSettingsTabId;
				}
				else if (Cast<UMaterialInstance>(MaterialEditorPinned->GetMaterialInterface()))
				{
					ContextObject->PreviewSettingsTabId = FMaterialInstanceEditorTabs::PreviewSettingsTabId;
				}

				ContextObject->AssetEditorToolkit = MaterialEditorPinned;
			}

			ViewportToolbarContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

TSharedPtr<IPreviewProfileController> SMaterialEditor3DPreviewViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

void SMaterialEditor3DPreviewViewport::ApplyStateToAllSlots(bool bState)
{
	for (const MaterialSlotPreviewInfo& SlotInfo : MaterialSlots)
	{
		SlotInfo.bSlotEnabled = bState;
	}

	SetPreviewMaterial(PreviewMaterial);
}

void SMaterialEditor3DPreviewViewport::OnSelectAllSlots()
{
	ApplyStateToAllSlots(true);
}

void SMaterialEditor3DPreviewViewport::OnSelectNoSlots()
{
	ApplyStateToAllSlots(false);
}

void SMaterialEditor3DPreviewViewport::OnMaterialSlotCheckStateChanged(ECheckBoxState NewState, int32 SlotIndex)
{
	if (MaterialSlots.IsValidIndex(SlotIndex))
	{
		MaterialSlots[SlotIndex].bSlotEnabled = NewState == ECheckBoxState::Checked;
		SetPreviewMaterial(PreviewMaterial);
	}
}

ECheckBoxState SMaterialEditor3DPreviewViewport::IsMaterialSlotChecked(int32 SlotIndex) const
{
	return MaterialSlots.IsValidIndex(SlotIndex) && MaterialSlots[SlotIndex].bSlotEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<SWidget> SMaterialEditor3DPreviewViewport::GenerateMaterialSlotMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/ false, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SelectAllSlots", "All Slots"),
		LOCTEXT("SelectAllSlots_Tooltip", "Apply the preview material to all slots."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SMaterialEditor3DPreviewViewport::OnSelectAllSlots))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SelectNoSlots", "None"),
		LOCTEXT("SelectNoSlots_Tooltip", "Apply the preview material to no slots."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SMaterialEditor3DPreviewViewport::OnSelectNoSlots))
	);

	MenuBuilder.AddSeparator();

	TSharedPtr<FAssetThumbnailPool> ThumbnailPool = UThumbnailManager::Get().GetSharedThumbnailPool();	
	TArray<TSharedPtr<FAssetThumbnail>> SlotThumbnails;
	const int32 ThumbSize = 40;
	for (int32 SlotIndex = 0; SlotIndex < MaterialSlots.Num(); SlotIndex++)
	{
		SlotThumbnails.Add(
			MakeShareable(new FAssetThumbnail(MaterialSlots[SlotIndex].OriginalMaterial, ThumbSize, ThumbSize, ThumbnailPool))
		);

		const FText SlotLabel = FText::Format(
			LOCTEXT("SlotLabelFmt", "{0}: {1}"),
			FText::AsNumber(SlotIndex),
			FText::FromName(MaterialSlots[SlotIndex].SlotName)
		);

		TSharedRef<SHorizontalBox> Row =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f, 0.f)
			[
				SNew(SBox)
					.WidthOverride(40.f)
					.HeightOverride(40.f)
					[
						SlotThumbnails.IsValidIndex(SlotIndex)
							? SlotThumbnails[SlotIndex]->MakeThumbnailWidget()
							: SNullWidget::NullWidget
					]
			]
		+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f)
			[
				SNew(SCheckBox)
					.IsChecked(this, &SMaterialEditor3DPreviewViewport::IsMaterialSlotChecked, SlotIndex)
					.OnCheckStateChanged(this,
						&SMaterialEditor3DPreviewViewport::OnMaterialSlotCheckStateChanged, SlotIndex)
					[
						SNew(STextBlock)
							.Text(SlotLabel)
					]
			];

		MenuBuilder.AddWidget(
			Row,
			FText::GetEmpty()
		);
	}

	return MenuBuilder.MakeWidget();
}

void SMaterialEditor3DPreviewViewport::PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay)
{
	// add the feature level display widget
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];
}

EVisibility SMaterialEditor3DPreviewViewport::OnGetViewportContentVisibility() const
{
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SMaterialEditor3DPreviewViewport::OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	static const FString MaterialDomain = TEXT("MaterialDomain");
	static const FString UserSceneTexture = TEXT("UserSceneTexture");
	static const FString PostProcessOverrides = TEXT("PostProcessOverrides");

	// We need to refresh other edited post process materials when a change is made that affects UserSceneTexture inputs or outputs, as previews
	// include materials that generate UserSceneTexture dependencies.  Changing the material domain potentially converts a material to or from a
	// Post Process domain material, adding or removing it as relevant to other previews.  Or changing any UserSceneTexture input or output, which
	// includes the "UMaterial::UserSceneTexture" field, plus any FName field in the PostProcessOverrides member struct.
	//
	// We also need to refresh PreviewMaterial itself if its domain changes, regardless of whether it's a post process material.
	if (ObjectBeingModified != nullptr && PropertyThatChanged != nullptr && PreviewMaterial)
	{
		if ((ObjectBeingModified == PreviewMaterial && PropertyThatChanged->GetName() == MaterialDomain) ||
			(PreviewMaterial->GetMaterial()->IsPostProcessMaterial() &&
			 (PropertyThatChanged->GetName() == MaterialDomain ||
			  PropertyThatChanged->GetName() == UserSceneTexture ||
			  (PropertyThatChanged->IsA<FNameProperty>() && PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetName() == PostProcessOverrides))))
		{
			SetPreviewMaterial(PreviewMaterial);
		}
	}
}

class SMaterialEditorUIPreviewZoomer : public SPanel, public IScrollableZoomable, public FGCObject
{
public:
	using FMaterialPreviewPanelSlot = FSingleWidgetChildrenWithSlot;

	SLATE_BEGIN_ARGS(SMaterialEditorUIPreviewZoomer)
		: _OnContextMenuRequested()
		, _OnZoomed()
		, _InitialPreviewSize(FVector2D(250.f))
		, _BackgroundSettings()
		{}
		SLATE_EVENT(FNoReplyPointerEventHandler, OnContextMenuRequested)
		SLATE_EVENT(FSimpleDelegate, OnZoomed)
		SLATE_ARGUMENT(FVector2D, InitialPreviewSize)
		SLATE_ARGUMENT(FPreviewBackgroundSettings, BackgroundSettings)
	SLATE_END_ARGS()

	SMaterialEditorUIPreviewZoomer()
		: CachedSize(ForceInitToZero)
		, PhysicalOffset(ForceInitToZero)
		, ScrollyZoomy(/* bUseInertialScrolling */ false)
		, ChildSlot(this)
		, CheckerboardTexture(nullptr)
	{
	}

	void Construct( const FArguments& InArgs, UMaterialInterface* InPreviewMaterial );

	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Begin SWidget Interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	/** End SWidget Interface */

	/** Begin IScrollableZoomable Interface */
	virtual bool ScrollBy(const FVector2D& Offset) override;
	/** End IScrollableZoomable Interface */

	bool ZoomBy(const float Amount);
	float GetZoomLevel() const;
	bool SetZoomLevel(float Level);
	FVector2D ComputeZoomedPreviewSize() const;

	FReply HandleScrollEvent(const FPointerEvent& MouseEvent);
	bool IsCurrentlyScrollable() const;
	void ScrollToCenter();
	bool IsCentered() const;

	void SetPreviewSize( const FVector2D PreviewSize );
	void SetPreviewMaterial(UMaterialInterface* InPreviewMaterial);

	void SetBackgroundSettings(const FPreviewBackgroundSettings& NewSettings);
	FSlateColor GetBorderColor() const;
	FMargin GetBorderPadding() const;

	/** Begin FGCObject Interface */
	void AddReferencedObjects(FReferenceCollector& Collector);
	virtual FString GetReferencerName() const override
	{
		return TEXT("SMaterialEditorUIPreviewZoomer");
	}
	/** End FGCObject Interface */
private:

	FSimpleDelegate OnZoomed;
	FNoReplyPointerEventHandler OnContextMenuRequested;

	void ModifyCheckerboardTextureColors(const FCheckerboardSettings& Checkerboard);
	void SetupCheckerboardTexture(const FColor& ColorOne, const FColor& ColorTwo, int32 CheckerSize);
	void DestroyCheckerboardTexture();

	FSlateColor GetSolidBackgroundColor() const;
	EVisibility GetVisibilityForBackgroundType(EBackgroundType BackgroundType) const;

	void ClampViewOffset(const FVector2D& ZoomedPreviewSize, const FVector2D& LocalSize);
	float ClampViewOffsetAxis(const float ZoomedPreviewSize, const float LocalSize, const float CurrentOffset);

	mutable FVector2D CachedSize;
	float ZoomLevel;
	FVector2D PhysicalOffset;
	FScrollyZoomy ScrollyZoomy;
	bool bCenterInFrame;

	FMaterialPreviewPanelSlot ChildSlot;

	TSharedPtr<FSlateMaterialBrush> PreviewBrush;
	TSharedPtr<FSlateImageBrush> CheckerboardBrush;
	TObjectPtr<UTexture2D> CheckerboardTexture;
	TSharedPtr<SImage> ImageWidget;
	FPreviewBackgroundSettings BackgroundSettings;
};


void SMaterialEditorUIPreviewZoomer::Construct( const FArguments& InArgs, UMaterialInterface* InPreviewMaterial )
{
	OnContextMenuRequested = InArgs._OnContextMenuRequested;
	OnZoomed = InArgs._OnZoomed;
	
	ZoomLevel = 1.0f;
	bCenterInFrame = true;
	
	BackgroundSettings = InArgs._BackgroundSettings;
	ModifyCheckerboardTextureColors(BackgroundSettings.Checkerboard);

	if (InPreviewMaterial)
	{
		PreviewBrush = MakeShared<FSlateMaterialBrush>(*InPreviewMaterial, InArgs._InitialPreviewSize);
	}
	else
	{
		PreviewBrush = MakeShared<FSlateMaterialBrush>(InArgs._InitialPreviewSize);
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
		.BorderBackgroundColor(this, &SMaterialEditorUIPreviewZoomer::GetBorderColor)
		.Padding(this, &SMaterialEditorUIPreviewZoomer::GetBorderPadding) // Leave space for our border (drawn in OnPaint) to remain visible when scrolled to the edges
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(this, &SMaterialEditorUIPreviewZoomer::GetSolidBackgroundColor)
			.Padding(0)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SImage)
					.Image(CheckerboardBrush.Get())
					.Visibility(this, &SMaterialEditorUIPreviewZoomer::GetVisibilityForBackgroundType, EBackgroundType::Checkered)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ImageWidget, SImage)
					.Image(PreviewBrush.Get())
				]
			]
		]
	];
}

FSlateColor SMaterialEditorUIPreviewZoomer::GetBorderColor() const
{
	return FLinearColor(BackgroundSettings.BorderColor);
}

FMargin SMaterialEditorUIPreviewZoomer::GetBorderPadding() const
{
	return FMargin(BackgroundSettings.bShowBorder ? 1.f : 0.f);
}

FSlateColor SMaterialEditorUIPreviewZoomer::GetSolidBackgroundColor() const
{
	return FLinearColor(BackgroundSettings.BackgroundColor);
}

EVisibility SMaterialEditorUIPreviewZoomer::GetVisibilityForBackgroundType(EBackgroundType BackgroundType) const
{
	return BackgroundSettings.BackgroundType == BackgroundType ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

void SMaterialEditorUIPreviewZoomer::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const 
{
	CachedSize = AllottedGeometry.GetLocalSize();

	const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
	if( ChildWidget->GetVisibility() != EVisibility::Collapsed )
	{
		SMaterialEditorUIPreviewZoomer* const MutableThis = const_cast<SMaterialEditorUIPreviewZoomer*>(this);

		FVector2D SizeWithBorder = GetDesiredSize();

		// Ensure we're centered within our current geometry
		if (bCenterInFrame)
		{
			MutableThis->PhysicalOffset = ((CachedSize - SizeWithBorder) * 0.5f).RoundToVector();
		}

		// Re-clamp since our parent might have changed size
		MutableThis->ClampViewOffset(SizeWithBorder, CachedSize);

		// Round so that we get a crisp checkerboard at all zoom levels
		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ChildWidget, PhysicalOffset, SizeWithBorder));
	}
}

FVector2D SMaterialEditorUIPreviewZoomer::ComputeDesiredSize(float) const
{
	FVector2D ThisDesiredSize = FVector2D::ZeroVector;

	const TSharedRef<SWidget>& ChildWidget = ChildSlot.GetWidget();
	if( ChildWidget->GetVisibility() != EVisibility::Collapsed )
	{
		ThisDesiredSize = ComputeZoomedPreviewSize() + GetBorderPadding().GetDesiredSize();
	}

	return ThisDesiredSize;
}

FVector2D SMaterialEditorUIPreviewZoomer::ComputeZoomedPreviewSize() const
{
	// Our desired size includes the 1px border (if enabled), but this is purely the size of the actual preview quad
	return (ImageWidget->GetDesiredSize() * ZoomLevel).RoundToVector();
}

FChildren* SMaterialEditorUIPreviewZoomer::GetChildren()
{
	return &ChildSlot;
}

int32 SMaterialEditorUIPreviewZoomer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// Set a UI scale for materials to use as reference, done on a per-window basis since don't want to change global uniforms per element
	if (SWindow* ParentWindow = OutDrawElements.GetPaintWindow())
	{
		ParentWindow->SetViewportScaleUIOverride(ZoomLevel);
	}

	if (IsCurrentlyScrollable())
	{
		LayerId = ScrollyZoomy.PaintSoftwareCursorIfNeeded(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	return LayerId;
}

bool SMaterialEditorUIPreviewZoomer::IsCurrentlyScrollable() const
{
	FVector2D ContentSize = GetDesiredSize();
	bool bCanScroll = ContentSize.X > CachedSize.X || ContentSize.Y > CachedSize.Y;
	return bCanScroll;
}

bool SMaterialEditorUIPreviewZoomer::ZoomBy( const float Amount )
{
	return SetZoomLevel(ZoomLevel + (Amount * 0.05f));
}

float SMaterialEditorUIPreviewZoomer::GetZoomLevel() const
{
	return ZoomLevel;
}

bool SMaterialEditorUIPreviewZoomer::SetZoomLevel(float NewLevel)
{
	static const float MinZoomLevel = 0.2f;
	static const float MaxZoomLevel = 4.0f;

	const float PrevZoomLevel = ZoomLevel;
	ZoomLevel = FMath::Clamp(NewLevel, MinZoomLevel, MaxZoomLevel);

	// Fire regardless of whether it actually changed, since still useful to give
	// the user feedback feedback when attempting to zoom past the limit
	OnZoomed.ExecuteIfBound();

	const bool bZoomChanged = ZoomLevel != PrevZoomLevel;
	return bZoomChanged;
}

void SMaterialEditorUIPreviewZoomer::SetPreviewSize( const FVector2D PreviewSize )
{
	PreviewBrush->ImageSize = PreviewSize;
}

void SMaterialEditorUIPreviewZoomer::SetPreviewMaterial(UMaterialInterface* InPreviewMaterial)
{
	// Just create a new brush to avoid possible invalidation issues from only the resource changing
	if (InPreviewMaterial)
	{
		PreviewBrush = MakeShared<FSlateMaterialBrush>(*InPreviewMaterial, PreviewBrush->ImageSize);
	}
	else
	{
		PreviewBrush = MakeShared<FSlateMaterialBrush>(PreviewBrush->ImageSize);
	}
	ImageWidget->SetImage(PreviewBrush.Get());
}

void SMaterialEditorUIPreviewZoomer::SetBackgroundSettings(const FPreviewBackgroundSettings& NewSettings)
{
	const bool bCheckerboardChanged = NewSettings.Checkerboard != BackgroundSettings.Checkerboard;

	BackgroundSettings = NewSettings;

	if (bCheckerboardChanged)
	{
		ModifyCheckerboardTextureColors(BackgroundSettings.Checkerboard);
	}
}

void SMaterialEditorUIPreviewZoomer::ModifyCheckerboardTextureColors(const FCheckerboardSettings& Checkerboard)
{
	DestroyCheckerboardTexture();
	SetupCheckerboardTexture(Checkerboard.ColorOne, Checkerboard.ColorTwo, Checkerboard.Size);

	if (!CheckerboardBrush.IsValid())
	{
		CheckerboardBrush = MakeShared<FSlateImageBrush>(CheckerboardTexture, FVector2D(Checkerboard.Size), FLinearColor::White, ESlateBrushTileType::Both);
	}
	else
	{
		// TODO: May need to invalidate paint here if the widget isn't aware the brush changed?
		CheckerboardBrush->SetResourceObject(CheckerboardTexture);
		CheckerboardBrush->SetImageSize(FVector2D(Checkerboard.Size));
	}
}

void SMaterialEditorUIPreviewZoomer::SetupCheckerboardTexture(const FColor& ColorOne, const FColor& ColorTwo, int32 CheckerSize)
{
	if (CheckerboardTexture == nullptr)
	{
		CheckerboardTexture = FImageUtils::CreateCheckerboardTexture(ColorOne, ColorTwo, CheckerSize);
	}
}

void SMaterialEditorUIPreviewZoomer::DestroyCheckerboardTexture()
{
	if (CheckerboardTexture)
	{
		if (CheckerboardTexture->GetResource())
		{
			CheckerboardTexture->ReleaseResource();
		}
		CheckerboardTexture->MarkAsGarbage();
		CheckerboardTexture = nullptr;
	}
}

void SMaterialEditorUIPreviewZoomer::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (CheckerboardTexture != nullptr)
	{
		Collector.AddReferencedObject(CheckerboardTexture);
	}
}

void SMaterialEditorUIPreviewZoomer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	ScrollyZoomy.Tick(InDeltaTime, *this);
}

FReply SMaterialEditorUIPreviewZoomer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return ScrollyZoomy.OnMouseButtonDown(MouseEvent);
}

FReply SMaterialEditorUIPreviewZoomer::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// If they didn't drag far enough to trigger a scroll, then treat it like a normal click,
		// which would show the context menu for rmb
		bool bWasPanning = ScrollyZoomy.IsRightClickScrolling();
		if (!bWasPanning)
		{
			OnContextMenuRequested.ExecuteIfBound(MyGeometry, MouseEvent);
		}
	}

	return ScrollyZoomy.OnMouseButtonUp(AsShared(), MyGeometry, MouseEvent);
}

FReply SMaterialEditorUIPreviewZoomer::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Only pass this on if we're scrollable, otherwise ScrollyZoomy will hide the cursor while rmb is down
	if (IsCurrentlyScrollable())
	{
		return ScrollyZoomy.OnMouseMove(AsShared(), *this, MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

void SMaterialEditorUIPreviewZoomer::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	ScrollyZoomy.OnMouseLeave(AsShared(), MouseEvent);
}

FReply SMaterialEditorUIPreviewZoomer::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return HandleScrollEvent(MouseEvent);
}

FCursorReply SMaterialEditorUIPreviewZoomer::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	// Only pass this on if we're scrollable, otherwise ScrollyZoomy will hide the cursor while rmb is down
	if (IsCurrentlyScrollable())
	{
		return ScrollyZoomy.OnCursorQuery();
	}

	return FCursorReply::Unhandled();
}

bool SMaterialEditorUIPreviewZoomer::ScrollBy(const FVector2D& Offset)
{
	const FVector2D PrevPhysicalOffset = PhysicalOffset;
	
	PhysicalOffset += Offset.RoundToVector();

	bCenterInFrame = false;

	ClampViewOffset(GetDesiredSize(), CachedSize);

	return PhysicalOffset != PrevPhysicalOffset;
}

FReply SMaterialEditorUIPreviewZoomer::HandleScrollEvent(const FPointerEvent& MouseEvent)
{
	return ScrollyZoomy.OnMouseWheel(MouseEvent, *this);
}

void SMaterialEditorUIPreviewZoomer::ScrollToCenter()
{
	bCenterInFrame = true;
}

bool SMaterialEditorUIPreviewZoomer::IsCentered() const
{
	return bCenterInFrame;
}

void SMaterialEditorUIPreviewZoomer::ClampViewOffset(const FVector2D& ZoomedPreviewSize, const FVector2D& LocalSize)
{
	PhysicalOffset.X = ClampViewOffsetAxis(ZoomedPreviewSize.X, LocalSize.X, PhysicalOffset.X);
	PhysicalOffset.Y = ClampViewOffsetAxis(ZoomedPreviewSize.Y, LocalSize.Y, PhysicalOffset.Y);
}

float SMaterialEditorUIPreviewZoomer::ClampViewOffsetAxis(const float ZoomedPreviewSize, const float LocalSize, const float CurrentOffset)
{
	if (ZoomedPreviewSize <= LocalSize)
	{
		// If the viewport is smaller than the available size, then we can't be scrolled
		return 0.0f;
	}

	// Given the size of the viewport, and the current size of the window, work how far we can scroll
	// Note: This number is negative since scrolling down/right moves the viewport up/left
	const float MaxScrollOffset = LocalSize - ZoomedPreviewSize;
	const float MinScrollOffset = 0.f;

	// Clamp the left/top edge
	if (CurrentOffset < MaxScrollOffset)
	{
		return MaxScrollOffset;
	}

	// Clamp the right/bottom edge
	if (CurrentOffset > MinScrollOffset)
	{
		return MinScrollOffset;
	}

	return CurrentOffset;
}

void SMaterialEditorUIPreviewViewport::Construct( const FArguments& InArgs, UMaterialInterface* PreviewMaterial )
{
	ZoomLevelFade = FCurveSequence(0.0f, 1.0f);
	ZoomLevelFade.JumpToEnd();

	UMaterialEditorSettings* Settings = GetMutableDefault<UMaterialEditorSettings>();
	PreviewSize = Settings->GetPreviewViewportStartingSize();

	// Take a copy of the global background settings at this moment, and listen for changes so we can update our colors as the user changes them
	BackgroundSettings = Settings->PreviewBackground;
	Settings->OnPostEditChange.AddSP(this, &SMaterialEditorUIPreviewViewport::HandleSettingsChanged);

	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SBorder )
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding( 3.f )
					.AutoWidth()
					[
						SNew( STextBlock )
						.Text( LOCTEXT("PreviewSize", "Preview Size" ) )
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding( 3.f )
					.MaxWidth( 75 )
					[
						SNew( SNumericEntryBox<int32> )
						.AllowSpin( true )
						.MinValue(1)
						.MaxSliderValue( 4096 )
						.OnValueChanged( this, &SMaterialEditorUIPreviewViewport::OnPreviewXChanged )
						.OnValueCommitted( this, &SMaterialEditorUIPreviewViewport::OnPreviewXCommitted )
						.Value( this, &SMaterialEditorUIPreviewViewport::OnGetPreviewXValue )
						.MinDesiredValueWidth( 75 )
						.Label()
						[	
							SNew( SBox )
							.VAlign( VAlign_Center )
							[
								SNew( STextBlock )
								.Text( LOCTEXT("PreviewSize_X", "X") )
							]
						]
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding( 3.f )
					.MaxWidth( 75 )
					[
						SNew( SNumericEntryBox<int32> )
						.AllowSpin( true )
						.MinValue(1)
						.MaxSliderValue( 4096 )
						.MinDesiredValueWidth( 75 )
						.OnValueChanged( this, &SMaterialEditorUIPreviewViewport::OnPreviewYChanged )
						.OnValueCommitted( this, &SMaterialEditorUIPreviewViewport::OnPreviewYCommitted )
						.Value( this, &SMaterialEditorUIPreviewViewport::OnGetPreviewYValue )
						.Label()
						[	
							SNew( SBox )
							.VAlign( VAlign_Center )
							[
								SNew( STextBlock )
								.Text( LOCTEXT("PreviewSize_Y", "Y") )
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew( SHorizontalBox )
					
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
					[
						SNew(STextBlock)
						.Text(this, &SMaterialEditorUIPreviewViewport::GetZoomText)
						.ColorAndOpacity(this, &SMaterialEditorUIPreviewViewport::GetZoomTextColorAndOpacity)
						//.Visibility(EVisibility::SelfHitTestInvisible)
						.ToolTip(SNew(SToolTip).Text(this, &SMaterialEditorUIPreviewViewport::GetDisplayedAtSizeText))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew( SComboButton )
						.ContentPadding(0)
						.ForegroundColor( FSlateColor::UseForeground() )
						.ButtonStyle( FAppStyle::Get(), "ToggleButton" )
						.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
						.MenuContent()
						[
							BuildViewOptionsMenu().MakeWidget()
						]
						.ButtonContent()
						[
							SNew(SImage)
							.Image( FAppStyle::GetBrush("GenericViewButton") )
						]
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		[
			SAssignNew( PreviewArea, SBorder )
			.Padding(0.f)
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.OnMouseButtonUp(this, &SMaterialEditorUIPreviewViewport::OnViewportClicked)
			.BorderImage( FAppStyle::GetBrush("BlackBrush") )
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SAssignNew( PreviewZoomer, SMaterialEditorUIPreviewZoomer, PreviewMaterial )
				.OnContextMenuRequested(this, &SMaterialEditorUIPreviewViewport::ShowContextMenu)
				.OnZoomed(this, &SMaterialEditorUIPreviewViewport::HandleDidZoom)
				.InitialPreviewSize(FVector2D(PreviewSize))
				.BackgroundSettings(BackgroundSettings)
			]
		]
	];
}

SMaterialEditorUIPreviewViewport::~SMaterialEditorUIPreviewViewport()
{
	UMaterialEditorSettings* Settings = GetMutableDefault<UMaterialEditorSettings>();
	Settings->OnPostEditChange.RemoveAll(this);
}

FText SMaterialEditorUIPreviewViewport::GetDisplayedAtSizeText() const
{
	FVector2D DisplayedSize = PreviewZoomer->ComputeZoomedPreviewSize().RoundToVector();
	return FText::Format(LOCTEXT("DisplayedAtSize", "Currently displayed at: {0}x{1}"), FText::AsNumber(DisplayedSize.X), FText::AsNumber(DisplayedSize.Y));
}

FText SMaterialEditorUIPreviewViewport::GetZoomText() const
{
	static FText ZoomLevelFormat = LOCTEXT("ZoomLevelFormat", "Zoom: {0}");
	
	FText ZoomLevelPercent = FText::AsPercent(PreviewZoomer->GetZoomLevel());
	return FText::FormatOrdered(FTextFormat(ZoomLevelFormat), ZoomLevelPercent);
}

void SMaterialEditorUIPreviewViewport::HandleDidZoom()
{
	ZoomLevelFade.Play(this->AsShared());
}

void SMaterialEditorUIPreviewViewport::ExecuteZoomToActual()
{
	PreviewZoomer->SetZoomLevel(1.f);
	PreviewZoomer->ScrollToCenter();
}

bool SMaterialEditorUIPreviewViewport::CanZoomToActual() const
{
	return !FMath::IsNearlyEqual(PreviewZoomer->GetZoomLevel(), 1.f, 0.01f) || !PreviewZoomer->IsCentered();
}

FSlateColor SMaterialEditorUIPreviewViewport::GetZoomTextColorAndOpacity() const
{
	return FLinearColor(1, 1, 1, 1.25f - ZoomLevelFade.GetLerp() * 0.75f);
}

void SMaterialEditorUIPreviewViewport::HandleSettingsChanged()
{
	const UMaterialEditorSettings& Settings = *GetDefault<UMaterialEditorSettings>();
	
	// Keep any global settings up to date when the user changes them in the editor prefs window
	BackgroundSettings.Checkerboard = Settings.PreviewBackground.Checkerboard;
	BackgroundSettings.BackgroundColor = Settings.PreviewBackground.BackgroundColor;
	BackgroundSettings.BorderColor = Settings.PreviewBackground.BorderColor;

	PreviewZoomer->SetBackgroundSettings(BackgroundSettings);
}

void SMaterialEditorUIPreviewViewport::SetPreviewMaterial(UMaterialInterface* InMaterialInterface)
{
	PreviewZoomer->SetPreviewMaterial(InMaterialInterface);
}

FReply SMaterialEditorUIPreviewViewport::OnViewportClicked(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		ShowContextMenu(Geometry, MouseEvent);
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void SMaterialEditorUIPreviewViewport::ShowContextMenu(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, BuildViewOptionsMenu(/* bForContextMenu */ true).MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
}

FReply SMaterialEditorUIPreviewViewport::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Forward scrolls over the preview area to the zoomer so you can still scroll over the blank space around the preview
	if (PreviewArea->IsHovered())
	{
		return PreviewZoomer->HandleScrollEvent(MouseEvent);
	}

	return FReply::Unhandled();
}

FMenuBuilder SMaterialEditorUIPreviewViewport::BuildViewOptionsMenu(bool bForContextMenu)
{
	auto GenerateBackgroundMenuContent = [this](FMenuBuilder& MenuBuilder)
	{
		// Not bothering to create commands for these since they'll probably be rarely changed,
		// and would mean needing to duplicate your bindings between texture editor and material editor
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SolidBackground", "Solid Color"),
			LOCTEXT("SolidBackground_ToolTip", "Displays a solid background color behind the preview."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMaterialEditorUIPreviewViewport::SetBackgroundType, EBackgroundType::SolidColor),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FIsActionChecked::CreateSP(this, &SMaterialEditorUIPreviewViewport::IsBackgroundTypeChecked, EBackgroundType::SolidColor)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CheckeredBackground", "Checkerboard"),
			LOCTEXT("CheckeredBackground_ToolTip", "Displays a checkerboard behind the preview."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMaterialEditorUIPreviewViewport::SetBackgroundType, EBackgroundType::Checkered),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FIsActionChecked::CreateSP(this, &SMaterialEditorUIPreviewViewport::IsBackgroundTypeChecked, EBackgroundType::Checkered)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton);
	};

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("ZoomSection", LOCTEXT("ZoomSectionHeader", "Zoom"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ZoomToActual", "Zoom to 100%"),
			LOCTEXT("ZoomToActual_Tooltip", "Resets the zoom to 100% and centers the preview."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMaterialEditorUIPreviewViewport::ExecuteZoomToActual),
				FCanExecuteAction::CreateSP(this, &SMaterialEditorUIPreviewViewport::CanZoomToActual)
			)
		);
	}
	MenuBuilder.EndSection();

	// view port options
	MenuBuilder.BeginSection("ViewportSection", LOCTEXT("ViewportSectionHeader", "Viewport Options"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("Background", "Background"),
			LOCTEXT("BackgroundTooltip", "Configure the preview's background."),
			FNewMenuDelegate::CreateLambda(GenerateBackgroundMenuContent)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowBorder", "Show Border"),
			LOCTEXT("ShowBorder_Tooltip", "Displays a border around the preview bounds."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMaterialEditorUIPreviewViewport::ToggleShowBorder),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FIsActionChecked::CreateSP(this, &SMaterialEditorUIPreviewViewport::IsShowBorderChecked)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();

	// Don't include settings item for right-clicks
	if (!bForContextMenu)
	{
		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Settings", "Settings"),
			LOCTEXT("Settings_Tooltip", "Opens the material editor preferences pane."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMaterialEditorUIPreviewViewport::HandleSettingsActionExecute)
			),
			NAME_None,
			EUserInterfaceActionType::Button);
	}

	return MenuBuilder;
}

void SMaterialEditorUIPreviewViewport::HandleSettingsActionExecute()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "ContentEditors", "Material Editor"); // Note: This has a space, unlike a lot of other setting sections - see MaterialEditorModuleConstants::SettingsSectionName
}

void SMaterialEditorUIPreviewViewport::OnPreviewXChanged( int32 NewValue )
{
	PreviewSize.X = NewValue;
	PreviewZoomer->SetPreviewSize( FVector2D( PreviewSize ) );
}

void SMaterialEditorUIPreviewViewport::OnPreviewXCommitted( int32 NewValue, ETextCommit::Type )
{
	OnPreviewXChanged( NewValue );
}

void SMaterialEditorUIPreviewViewport::OnPreviewYChanged( int32 NewValue )
{
	PreviewSize.Y = NewValue;
	PreviewZoomer->SetPreviewSize( FVector2D( PreviewSize ) );
}

void SMaterialEditorUIPreviewViewport::OnPreviewYCommitted( int32 NewValue, ETextCommit::Type )
{
	OnPreviewYChanged( NewValue );
}

bool SMaterialEditorUIPreviewViewport::IsBackgroundTypeChecked(EBackgroundType BackgroundType) const
{
	return BackgroundSettings.BackgroundType == BackgroundType;
}

bool SMaterialEditorUIPreviewViewport::IsShowBorderChecked() const
{
	return BackgroundSettings.bShowBorder;
}

void SMaterialEditorUIPreviewViewport::SetBackgroundType(EBackgroundType NewBackgroundType)
{
	BackgroundSettings.BackgroundType = NewBackgroundType;
	PreviewZoomer->SetBackgroundSettings(BackgroundSettings);

	// Use this as the default of newly opened preview viewports
	GetMutableDefault<UMaterialEditorSettings>()->PreviewBackground.BackgroundType = BackgroundSettings.BackgroundType;
}

void SMaterialEditorUIPreviewViewport::ToggleShowBorder()
{
	BackgroundSettings.bShowBorder = !BackgroundSettings.bShowBorder;
	PreviewZoomer->SetBackgroundSettings(BackgroundSettings);

	// Use this as the default of newly opened preview viewports
	GetMutableDefault<UMaterialEditorSettings>()->PreviewBackground.bShowBorder = BackgroundSettings.bShowBorder;
}

#undef LOCTEXT_NAMESPACE
