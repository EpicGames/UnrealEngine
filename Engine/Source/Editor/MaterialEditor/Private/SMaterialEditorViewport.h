// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/GCObject.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PreviewScene.h"
#include "EditorViewportClient.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "MaterialEditorSettings.h"
#include "Animation/CurveSequence.h"

class FMaterialEditorViewportClient;
class IMaterialEditor;
class SMaterialEditorUIPreviewZoomer;
class UMaterialInterface;
class UMeshComponent;

/** Helper struct for containing data for the material slot submenu UI and where to apply the preview material */
struct MaterialSlotPreviewInfo
{
	FName SlotName;
	TObjectPtr<UMaterialInterface> OriginalMaterial;
	mutable bool bSlotEnabled;
};

/**
 * Material Editor Preview viewport widget
 */
class SMaterialEditor3DPreviewViewport : public SEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS( SMaterialEditor3DPreviewViewport )
		: _PreviewMaterial(nullptr){}
		SLATE_ARGUMENT(TWeakPtr<IMaterialEditor>, MaterialEditor)
		SLATE_ARGUMENT(UMaterialInterface*, PreviewMaterial)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SMaterialEditor3DPreviewViewport();
	
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SMaterialEditor3DPreviewViewport");
	}

	void RefreshViewport();
	
	/**
	 * Sets the mesh on which to preview the material.  One of either InStaticMesh or InSkeletalMesh must be non-NULL!
	 * Does nothing if a skeletal mesh was specified but the material has bUsedWithSkeletalMesh=false.
	 *
	 * @return	true if a mesh was set successfully, false otherwise.
	 */
	bool SetPreviewAsset(UObject* InAsset);

	/**
	 * Sets the preview asset to the named asset, if it can be found and is convertible into a UMeshComponent subclass.
	 * Does nothing if the named asset is not found or if the named asset is a skeletal mesh but the material has
	 * bUsedWithSkeletalMesh=false.
	 *
	 * @return	true if the named asset was found and set successfully, false otherwise.
	 */
	bool SetPreviewAssetByName(const TCHAR* InAssetName);

	void SetPreviewMaterial(UMaterialInterface* InMaterialInterface);

	/** Component for the preview mesh. */
	TObjectPtr<UMeshComponent> PreviewMeshComponent;

	/** Material for the preview mesh */
	TObjectPtr<UMaterialInterface> PreviewMaterial;
	
	/** The preview primitive we are using. */
	EThumbnailPrimType PreviewPrimType;
	
	/** The material editor has been added to a tab */
	void OnAddedToTab( const TSharedRef<SDockTab>& OwnerTab );

	TSharedRef<class FAdvancedPreviewScene> GetPreviewScene() { return AdvancedPreviewScene.ToSharedRef(); }

	/** Event handlers */
	void OnSetPreviewPrimitive(EThumbnailPrimType PrimType);
	bool IsPreviewPrimitiveChecked(EThumbnailPrimType PrimType) const;
	void OnSetPreviewMeshFromSelection();
	bool IsPreviewMeshFromSelectionChecked() const;
	void OnSetPreviewMeshMaterialSlots();
	bool DoesPreviewMeshContainSlots() const;

	void TogglePreviewBackground();
	bool IsTogglePreviewBackgroundChecked() const;

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	TSharedRef<SWidget> GenerateMaterialSlotMenu();

	virtual void PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay) override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	
	//~ Begin SWidget
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InEvent) override;
	//~ End SWidget
	
private:
	/** If we have a MaterialEditor, it will look up the preview material from that. Otherwise it will fallback to previewing this material */
	UMaterialInterface* GetPreviewingMaterialInterface();

	void OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	/** Helper function for applying state logic to all material slots + updates*/
	void ApplyStateToAllSlots(bool bState);

	/** Set true for usage to all material slots */
	void OnSelectAllSlots();

	/** Set false for usage to all material slots */
	void OnSelectNoSlots();

	/** Updates the scene preview mesh when a checkbox state changes */
	void OnMaterialSlotCheckStateChanged(ECheckBoxState NewState, int32 SlotIndex);

	/** Returns if a material slot is checked/should show the preview material */
	ECheckBoxState IsMaterialSlotChecked(int32 SlotIndex) const;

	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

	/** The Substrate control tab */
	TSharedPtr<class SMaterialEditorTopologyWidget> SubstrateWidget;

	bool IsVisible() const override;

	/** Pointer back to the material editor tool that owns us */
	TWeakPtr<IMaterialEditor> MaterialEditorPtr;

	/** The material we're previewing if we're not hosted by a material editor. Used for diffs. */
	TWeakObjectPtr<UMaterialInterface> PreviewMaterialPtr;

	/** Level viewport client */
	TSharedPtr<class FMaterialEditorViewportClient> EditorViewportClient;

	/** Preview Scene - uses advanced preview settings */
	TSharedPtr<class FAdvancedPreviewScene> AdvancedPreviewScene;

	/** Post process volume actor. */
	TObjectPtr<class APostProcessVolume> PostProcessVolumeActor;

	/** Property changed delegate. */
	FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate OnPropertyChangedHandle;

	/** Handle to the registered OnPropertyChangedHandle delegate. */
	FDelegateHandle OnPropertyChangedHandleDelegateHandle;

	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	/** Material Slot data for populating and processing the material slot selections */
	TArray<const MaterialSlotPreviewInfo> MaterialSlots;
};

/**
 * A preview viewport used for 2D UI materials
 */
class SMaterialEditorUIPreviewViewport : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SMaterialEditorUIPreviewViewport ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, UMaterialInterface* PreviewMaterial );
	virtual ~SMaterialEditorUIPreviewViewport();

	void SetPreviewMaterial(UMaterialInterface* InMaterialInterface);

private:
	void OnPreviewXChanged( int32 NewValue );
	void OnPreviewXCommitted( int32 NewValue, ETextCommit::Type );
	void OnPreviewYChanged( int32 NewValue );
	void OnPreviewYCommitted( int32 NewValue, ETextCommit::Type );
	TOptional<int32> OnGetPreviewXValue() const { return PreviewSize.X; }
	TOptional<int32> OnGetPreviewYValue() const { return PreviewSize.Y; }

	bool IsBackgroundTypeChecked(EBackgroundType BackgroundType) const;
	bool IsShowBorderChecked() const;

	void SetBackgroundType(EBackgroundType NewBackgroundType);
	void ToggleShowBorder();

	FMenuBuilder BuildViewOptionsMenu(bool bForContextMenu = false);
	FReply OnViewportClicked(const FGeometry& Geometry, const FPointerEvent& MouseEvent);
	void ShowContextMenu(const FGeometry& Geometry, const FPointerEvent& MouseEvent);

	FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void HandleSettingsActionExecute();
	void HandleSettingsChanged();
	
	void HandleDidZoom();
	void ExecuteZoomToActual();
	bool CanZoomToActual() const;
	FSlateColor GetZoomTextColorAndOpacity() const;
	FText GetZoomText() const;
	FText GetDisplayedAtSizeText() const;

private:
	FIntPoint PreviewSize;
	TSharedPtr<class SMaterialEditorUIPreviewZoomer> PreviewZoomer;
	TSharedPtr<SWidget> PreviewArea;
	FPreviewBackgroundSettings BackgroundSettings;
	FCurveSequence ZoomLevelFade;
};
