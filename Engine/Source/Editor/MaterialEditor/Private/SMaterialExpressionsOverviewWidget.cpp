// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialExpressionsOverviewWidget.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Widgets/SToolTip.h"

class SCustomSharedSamplerRow : public SMultiColumnTableRow<TSharedPtr<FSharedSamplerDataRowData>>
{
public:
	SLATE_BEGIN_ARGS(SCustomSharedSamplerRow) {}
		SLATE_ARGUMENT(TSharedPtr<FSharedSamplerDataRowData>, Entry)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		RowData = Args._Entry;

		FSuperRowType::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedRef<STextBlock> TextBlock = SNew(STextBlock);

		if (RowData->bIsDuplicate)
		{
			TextBlock->SetToolTip(SNew(SToolTip).Text(FText::FromString("This slot is potentially incorrectly overlapping")).BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground")));
			TextBlock->SetColorAndOpacity(FSlateColor(FLinearColor::Red));
		}

		if (ColumnName == "Reference Count")
		{
			TextBlock->SetText(FText::FromString(FString::FormatAsNumber(RowData->Count)));
			return TextBlock;
		}
		else if (ColumnName == "Name")
		{
			TextBlock->SetText(FText::FromString(RowData->Name));
			return TextBlock;
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FSharedSamplerDataRowData> RowData;
};

void SMaterialExpressionsOverviewPanel::Construct(const FArguments& InArgs)
{
	this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(4.0f))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(FMargin(3.0f, 4.0f))
						.HAlign(HAlign_Left)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Textures"))
						]
					]
					+ SVerticalBox::Slot()
					.Padding(FMargin(3.0f, 2.0f, 3.0f, 3.0f))
					[
						SNew(SBorder)
						[
							SNew(SScrollBox)
							+ SScrollBox::Slot()
							[
								SAssignNew(ListViewWidget, SListView<TSharedPtr<FSharedSamplerDataRowData>>)
								.ListItemsSource(&Items)
								.OnGenerateRow(this, &SMaterialExpressionsOverviewPanel::OnGenerateRowForList)
								.OnMouseButtonClick(this, &SMaterialExpressionsOverviewPanel::OnMouseButtonClick)
								.SelectionMode(ESelectionMode::Single)
								.HeaderRow
								(
									SNew(SHeaderRow)
									+ SHeaderRow::Column("Name").DefaultLabel(FText::FromString("Name"))
									+ SHeaderRow::Column("Reference Count").DefaultLabel(FText::FromString("Reference Count"))
								)
							]
						]
					]
				]
			]
		];

	MaterialEditorInstance = InArgs._InMaterialEditorInstance;
	MaterialDetailsView = InArgs._InMaterialDetailsView;
	Refresh();
}

void SMaterialExpressionsOverviewPanel::SetEditorInstance(UMaterialEditorPreviewParameters* InMaterialEditorInstance)
{
	MaterialEditorInstance = InMaterialEditorInstance;
	Refresh();
}

void SMaterialExpressionsOverviewPanel::Refresh()
{
	if (!MaterialEditorInstance)
	{
		// Early out in case we do not have a material editor instance available
		return;
	}

	TMap<FString, int32> SamplerPairs;
	TArray<UMaterialExpressionTextureSample*> TextureSamples;
	MaterialEditorInstance->PreviewMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionTextureSample>(TextureSamples);

	for (const UMaterialExpressionTextureSample* Expr : TextureSamples)
	{
		const FString TextureName = Expr->GetReferencedTexture() != nullptr ? Expr->GetReferencedTexture()->GetName() : Expr->GetName();
		int32 Sampler = SamplerPairs.FindOrAdd(TextureName);
		SamplerPairs.Add(TextureName, ++Sampler);
	}

	Items.Empty();

	// Go through and mark slots as duplicate while copying over to the shared ptr array
	for (const auto& Params : SamplerPairs)
	{
		Items.Add(MakeShareable(new FSharedSamplerDataRowData(Params.Key, Params.Value)));
	}

	// Sort the items in descending order based on the amount of times a given sampler has been referenced
	Items.StableSort([](const TSharedPtr<FSharedSamplerDataRowData>& A, const TSharedPtr<FSharedSamplerDataRowData>& B) {
		int32 CountA = A->Count;
		int32 CountB = B->Count;
		if (CountA == CountB)
		{
			return A->Name < B->Name;
		}
		return CountA < CountB;
	});

	ListViewWidget->RequestListRefresh();
}

TSharedRef<ITableRow> SMaterialExpressionsOverviewPanel::OnGenerateRowForList(TSharedPtr<FSharedSamplerDataRowData> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCustomSharedSamplerRow, OwnerTable).Entry(Item);
}

void SMaterialExpressionsOverviewPanel::OnMouseButtonClick(const TSharedPtr<FSharedSamplerDataRowData> RowData)
{
	// Update the details panel to display info about the selected texture node
	if (!MaterialDetailsView)
	{
		return;
	}

	TArray<UMaterialExpressionTextureSample*> TextureSamples;
	MaterialEditorInstance->PreviewMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionTextureSample>(TextureSamples);

	UObject* SelectedObject = nullptr;
	for (UMaterialExpressionTextureSample* Expr : TextureSamples)
	{
		const FString TextureName = Expr->GetReferencedTexture() != nullptr ? Expr->GetReferencedTexture()->GetName() : Expr->GetName();
		if (RowData->Name == TextureName)
		{
			SelectedObject = CastChecked<UObject>(Expr);
		}
	}

	if (SelectedObject)
	{
		MaterialDetailsView->SetObject(SelectedObject, true);
	}
}

