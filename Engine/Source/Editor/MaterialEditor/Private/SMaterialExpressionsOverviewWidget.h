// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialEditor.h"
#include "IDetailTreeNode.h"
#include "IDetailPropertyRow.h"
#include "IDetailsView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class UMaterialEditorPreviewParameters;

/** Data for a row in the list of shared sampler entries*/
struct FSharedSamplerDataRowData
{
	FSharedSamplerDataRowData(FString InName, int32 InCount, bool bInIsDuplicate = false) : Name(InName), Count(InCount), bIsDuplicate(bInIsDuplicate) {}

	FString Name = "";
	int32 Count = 0;
	bool bIsDuplicate = false;
};

// ********* SMaterialExpressionsOverviewPanel *******
class SMaterialExpressionsOverviewPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialExpressionsOverviewPanel)
		: _InMaterialEditorInstance(nullptr)
		, _InGenerator()
		, _InMaterialDetailsView()
	{}

	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, InMaterialEditorInstance)
	SLATE_ARGUMENT(TSharedPtr<class IPropertyRowGenerator>, InGenerator)
	SLATE_ARGUMENT(TSharedPtr<class IDetailsView>, InMaterialDetailsView)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/* Sets the Material Editor Preview Parameters to be displayed and refreshes the UI */
	void SetEditorInstance(UMaterialEditorPreviewParameters* InMaterialEditorInstance);

	/* The material details widget is initialized after this widget is created so we cannot pass the view into the constructor */
	void SetMaterialDetailsView(TSharedPtr<class IDetailsView> InDetailsView) { MaterialDetailsView = InDetailsView; }

private:
	void Refresh();

	/** Adds a new text box with the string to the list */
	TSharedRef<ITableRow> OnGenerateRowForList(TSharedPtr<FSharedSamplerDataRowData> Item, const TSharedRef<STableViewBase>& OwnerTable);
	TWeakPtr<class IPropertyRowGenerator> Generator;
	
	void OnMouseButtonClick(const TSharedPtr<FSharedSamplerDataRowData> RowData);

	/** The list of strings */
	TArray<TSharedPtr<FSharedSamplerDataRowData>> Items;

	/** The actual UI list */
	TSharedPtr<SListView<TSharedPtr<FSharedSamplerDataRowData>>> ListViewWidget;

	/** The set of material parameters this is associated with */
	UMaterialEditorPreviewParameters* MaterialEditorInstance;

	/** Pointer to the Material Editor instance this widget was created for **/
	TSharedPtr<class IDetailsView> MaterialDetailsView;
};
