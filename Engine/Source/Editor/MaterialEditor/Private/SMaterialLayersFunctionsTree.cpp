// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialLayersFunctionsTree.h"
#include "MaterialEditor/DEditorMaterialLayersParameterValue.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Materials/Material.h"
#include "PropertyHandle.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ISinglePropertyView.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Styling/AppStyle.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "IPropertyRowGenerator.h"
#include "Widgets/Views/STreeView.h"
#include "IDetailTreeNode.h"
#include "AssetThumbnail.h"
#include "MaterialEditorInstanceDetailCustomization.h"
#include "MaterialPropertyHelpers.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorSupportDelegates.h"
#include "Widgets/Images/SImage.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "Widgets/Input/SButton.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "SResetToDefaultPropertyEditor.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Styling/StyleColors.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "MaterialLayerCustomization"

const FSlateBrush* SMaterialLayersFunctionsInstanceTreeItem::GetBorderImage() const
{
	return FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
}

FSlateColor SMaterialLayersFunctionsInstanceTreeItem::GetOuterBackgroundColor(TSharedPtr<FSortedParamData> InParamData) const
{
	if (InParamData->StackDataType == EStackDataType::Stack)
	{
		if (bIsBeingDragged)
		{
			return FAppStyle::Get().GetSlateColor("Colors.Recessed");
		}
		else if (bIsHoveredDragTarget)
		{
			return FAppStyle::Get().GetSlateColor("Colors.Highlight");
		}
		else
		{
			return FAppStyle::Get().GetSlateColor("Colors.Header");
		}
	}
	else if (IsHovered() || InParamData->StackDataType == EStackDataType::Group)
	{
		return FAppStyle::Get().GetSlateColor("Colors.Header");
	}

	return FAppStyle::Get().GetSlateColor("Colors.Panel");
}

void SMaterialLayersFunctionsInstanceTreeItem::RefreshOnRowChange(const FAssetData& AssetData, SMaterialLayersFunctionsInstanceTree* InTree)
{
	if (SMaterialLayersFunctionsInstanceWrapper* Wrapper = InTree->GetWrapper())
	{
		if (Wrapper->OnLayerPropertyChanged.IsBound())
		{
			Wrapper->OnLayerPropertyChanged.Execute();
		}
		else
		{
			InTree->CreateGroupsWidget();
		}
	}
}

bool SMaterialLayersFunctionsInstanceTreeItem::GetFilterState(SMaterialLayersFunctionsInstanceTree* InTree, TSharedPtr<FSortedParamData> InStackData) const
{
	if (InStackData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
	{
		return InTree->FunctionInstance->EditorOnly.RestrictToLayerRelatives[InStackData->ParameterInfo.Index];
	}
	if (InStackData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
	{
		return InTree->FunctionInstance->EditorOnly.RestrictToBlendRelatives[InStackData->ParameterInfo.Index];
	}
	return false;
}

void SMaterialLayersFunctionsInstanceTreeItem::FilterClicked(const ECheckBoxState NewCheckedState, SMaterialLayersFunctionsInstanceTree* InTree, TSharedPtr<FSortedParamData> InStackData)
{
	if (InStackData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
	{
		InTree->FunctionInstance->EditorOnly.RestrictToLayerRelatives[InStackData->ParameterInfo.Index] = !InTree->FunctionInstance->EditorOnly.RestrictToLayerRelatives[InStackData->ParameterInfo.Index];
	}
	if (InStackData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
	{
		InTree->FunctionInstance->EditorOnly.RestrictToBlendRelatives[InStackData->ParameterInfo.Index] = !InTree->FunctionInstance->EditorOnly.RestrictToBlendRelatives[InStackData->ParameterInfo.Index];
	}
}

ECheckBoxState SMaterialLayersFunctionsInstanceTreeItem::GetFilterChecked(SMaterialLayersFunctionsInstanceTree* InTree, TSharedPtr<FSortedParamData> InStackData) const
{
	return GetFilterState(InTree, InStackData) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SMaterialLayersFunctionsInstanceTreeItem::GetLayerName(SMaterialLayersFunctionsInstanceTree* InTree, int32 Counter) const
{
	return InTree->FunctionInstance->GetLayerName(Counter);
}

void SMaterialLayersFunctionsInstanceTreeItem::OnNameChanged(const FText& InText, ETextCommit::Type CommitInfo, SMaterialLayersFunctionsInstanceTree* InTree, int32 Counter)
{
	const FScopedTransaction Transaction(LOCTEXT("RenamedSection", "Renamed layer and blend section"));
	InTree->FunctionInstanceHandle->NotifyPreChange();
	InTree->FunctionInstance->EditorOnly.LayerNames[Counter] = InText;
	InTree->FunctionInstance->UnlinkLayerFromParent(Counter);
	InTree->MaterialEditorInstance->CopyToSourceInstance(true);
	InTree->FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

FReply SMaterialLayersFunctionsInstanceTreeItem::OnLayerDrop(const FDragDropEvent& DragDropEvent)
{
	if (!bIsHoveredDragTarget)
	{
		return FReply::Unhandled();
	}
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveLayer", "Move Layer"));
	Tree->FunctionInstanceHandle->NotifyPreChange();
	bIsHoveredDragTarget = false;
	TSharedPtr<FLayerDragDropOp> ArrayDropOp = DragDropEvent.GetOperationAs< FLayerDragDropOp >();
	TSharedPtr<SMaterialLayersFunctionsInstanceTreeItem> LayerPtr = nullptr;
	if (ArrayDropOp.IsValid() && ArrayDropOp->OwningStack.IsValid())
	{
		LayerPtr = StaticCastWeakPtr<SMaterialLayersFunctionsInstanceTreeItem>(ArrayDropOp->OwningStack).Pin();
	}
	if (!LayerPtr.IsValid())
	{
		return FReply::Unhandled();
	}
	
	LayerPtr->bIsBeingDragged = false;
	TSharedPtr<FSortedParamData> SwappingPropertyData = LayerPtr->StackParameterData;
	TSharedPtr<FSortedParamData> SwappablePropertyData = StackParameterData;
	if (SwappingPropertyData.IsValid() && SwappablePropertyData.IsValid())
	{
		if (SwappingPropertyData != SwappablePropertyData)
		{
			int32 OriginalIndex = SwappingPropertyData->ParameterInfo.Index;
			if (SwappingPropertyData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
			{
				OriginalIndex++;
			}

			int32 NewIndex = SwappablePropertyData->ParameterInfo.Index;
			if (SwappablePropertyData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
			{
				NewIndex++;
			}

			if (OriginalIndex != NewIndex)
			{
				Tree->MaterialEditorInstance->SourceInstance->SwapLayerParameterIndices(OriginalIndex, NewIndex);

				// Need to save the moving and target expansion states before swapping
				const bool bOriginalSwappableExpansion = IsItemExpanded();
				const bool bOriginalSwappingExpansion = LayerPtr->IsItemExpanded();

				TArray<void*> StructPtrs;
				Tree->FunctionInstanceHandle->AccessRawData(StructPtrs);
				FMaterialLayersFunctions* MaterialLayersFunctions = static_cast<FMaterialLayersFunctions*>(StructPtrs[0]);
				MaterialLayersFunctions->MoveBlendedLayer(OriginalIndex, NewIndex);

				Tree->OnExpansionChanged(SwappablePropertyData, bOriginalSwappingExpansion);
				Tree->OnExpansionChanged(SwappingPropertyData, bOriginalSwappableExpansion);
				Tree->FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				Tree->CreateGroupsWidget();
				Tree->RequestTreeRefresh();
				Tree->SetParentsExpansionState();
			}
		}
	}

	return FReply::Handled();
}


bool SMaterialLayersFunctionsInstanceTree::IsOverriddenExpression(class UDEditorParameterValue* Parameter, int32 InIndex)
{
	return FMaterialPropertyHelpers::IsOverriddenExpression(Parameter) && FunctionInstance->EditorOnly.LayerStates[InIndex];
}

bool SMaterialLayersFunctionsInstanceTree::IsOverriddenExpression(TObjectPtr<UDEditorParameterValue> Parameter, int32 InIndex)
{
	return IsOverriddenExpression(Parameter.Get(), InIndex);
}

FGetShowHiddenParameters SMaterialLayersFunctionsInstanceTree::GetShowHiddenDelegate() const
{
	return ShowHiddenDelegate;
}

void  SMaterialLayersFunctionsInstanceTreeItem::OnOverrideParameter(bool NewValue, class UDEditorParameterValue* Parameter)
{
	FMaterialPropertyHelpers::OnOverrideParameter(NewValue, Parameter, MaterialEditorInstance);
}

void  SMaterialLayersFunctionsInstanceTreeItem::OnOverrideParameter(bool NewValue, TObjectPtr<UDEditorParameterValue> Parameter)
{
	OnOverrideParameter(NewValue, Parameter.Get());
}

void SMaterialLayersFunctionsInstanceTreeItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	StackParameterData = InArgs._StackParameterData;
	MaterialEditorInstance = InArgs._MaterialEditorInstance;
	Tree = InArgs._InTree;

	TSharedRef<SWidget> LeftSideWidget = SNullWidget::NullWidget;
	TSharedRef<SWidget> RightSideWidget = SNullWidget::NullWidget;
	TSharedRef<SWidget> ResetWidget = SNullWidget::NullWidget;
	FText NameOverride;
	TSharedRef<SVerticalBox> WrapperWidget = SNew(SVerticalBox);
	EHorizontalAlignment ValueAlignment = HAlign_Left;
	FMargin RightSidePadding;

// STACK --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Stack)
	{
		WrapperWidget->AddSlot()
			.Padding(0.0f)
			.AutoHeight()
			[
				SNullWidget::NullWidget
			];
#if WITH_EDITOR
		NameOverride = Tree->FunctionInstance->GetLayerName(StackParameterData->ParameterInfo.Index);
#endif
		TSharedRef<SHorizontalBox> HeaderRowWidget = SNew(SHorizontalBox);

		if (StackParameterData->ParameterInfo.Index != 0)
		{
			TAttribute<bool>::FGetter IsEnabledGetter = TAttribute<bool>::FGetter::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::IsLayerVisible, StackParameterData->ParameterInfo.Index);
			TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::Create(IsEnabledGetter);

			FOnClicked VisibilityClickedDelegate = FOnClicked::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::ToggleLayerVisibility, StackParameterData->ParameterInfo.Index);

			HeaderRowWidget->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					PropertyCustomizationHelpers::MakeVisibilityButton(VisibilityClickedDelegate, FText(), IsEnabledAttribute)
				];
		}
		const float ThumbnailSize = 24.0f;
		TArray<TSharedPtr<FSortedParamData>> AssetChildren = StackParameterData->Children;
		if (AssetChildren.Num() > 0)
		{
			HeaderRowWidget->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				.AutoWidth()
				[
					SNullWidget::NullWidget
				];
		}
		for (TSharedPtr<FSortedParamData> AssetChild : AssetChildren)
		{
			TSharedPtr<SBox> ThumbnailBox;
			UObject* AssetObject = nullptr;
			AssetChild->ParameterHandle->GetValue(AssetObject);
			int32 PreviewIndex = INDEX_NONE;
			int32 ThumbnailIndex = INDEX_NONE;
			EMaterialParameterAssociation PreviewAssociation = EMaterialParameterAssociation::GlobalParameter;
			if (AssetObject)
			{
				if (Cast<UMaterialFunctionInterface>(AssetObject)->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayer)
				{
					PreviewIndex = StackParameterData->ParameterInfo.Index;
					PreviewAssociation = EMaterialParameterAssociation::LayerParameter;
					Tree->UpdateThumbnailMaterial(PreviewAssociation, PreviewIndex);
					ThumbnailIndex = PreviewIndex;
				}
				if (Cast<UMaterialFunctionInterface>(AssetObject)->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayerBlend)
				{
					PreviewIndex = StackParameterData->ParameterInfo.Index;
					PreviewAssociation = EMaterialParameterAssociation::BlendParameter;
					Tree->UpdateThumbnailMaterial(PreviewAssociation, PreviewIndex, true);
					ThumbnailIndex = PreviewIndex - 1;
				}
			}
			HeaderRowWidget->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				.MaxWidth(ThumbnailSize)
				[
					SAssignNew(ThumbnailBox, SBox)
					[
						Tree->CreateThumbnailWidget(PreviewAssociation, ThumbnailIndex, ThumbnailSize)
					]
				];
			ThumbnailBox->SetMaxDesiredHeight(ThumbnailSize);
			ThumbnailBox->SetMinDesiredHeight(ThumbnailSize);
			ThumbnailBox->SetMinDesiredWidth(ThumbnailSize);
			ThumbnailBox->SetMaxDesiredWidth(ThumbnailSize);
		}

		

		if (StackParameterData->ParameterInfo.Index != 0)
		{
			HeaderRowWidget->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5.0f)
				[
					SNew(SEditableTextBox)
					.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::GetLayerName, InArgs._InTree, StackParameterData->ParameterInfo.Index)))
					.OnTextCommitted(FOnTextCommitted::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::OnNameChanged, InArgs._InTree, StackParameterData->ParameterInfo.Index))
				];
		}
		else
		{
			HeaderRowWidget->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5.0f)
				[
					SNew(STextBlock)
					.Text(NameOverride)
				];
		}

		// Unlink UI
		HeaderRowWidget->AddSlot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNullWidget::NullWidget
			];
		HeaderRowWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Unlink", "Unlink"))
				.HAlign(HAlign_Center)
				.OnClicked(Tree, &SMaterialLayersFunctionsInstanceTree::UnlinkLayer, StackParameterData->ParameterInfo.Index)
				.ToolTipText(LOCTEXT("UnlinkLayer", "Whether or not to unlink this layer/blend combination from the parent."))
				.Visibility(Tree, &SMaterialLayersFunctionsInstanceTree::GetUnlinkLayerVisibility, StackParameterData->ParameterInfo.Index)
			];

		// Can only remove layers that aren't the base layer.
		if (StackParameterData->ParameterInfo.Index != 0)
		{
			HeaderRowWidget->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 5.0f, 0.0f)
				[
					PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(InArgs._InTree, &SMaterialLayersFunctionsInstanceTree::RemoveLayer, StackParameterData->ParameterInfo.Index))
				];
		}
		LeftSideWidget = HeaderRowWidget;
	}
// END STACK

// GROUP --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Group)
	{
		NameOverride = FText::FromName(StackParameterData->Group.GroupName);
		LeftSideWidget = SNew(STextBlock)
			.Text(NameOverride)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.TransformPolicy(ETextTransformPolicy::ToUpper);
		const int32 LayerStateIndex = StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter ? StackParameterData->ParameterInfo.Index + 1 : StackParameterData->ParameterInfo.Index;
		LeftSideWidget->SetEnabled(InArgs._InTree->FunctionInstance->EditorOnly.LayerStates[LayerStateIndex]);
		RightSideWidget->SetEnabled(InArgs._InTree->FunctionInstance->EditorOnly.LayerStates[LayerStateIndex]);
	}
// END GROUP

// ASSET --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Asset)
	{
		ValueAlignment = HAlign_Fill;
		FOnSetObject ObjectChanged = FOnSetObject::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::RefreshOnRowChange, Tree);
		StackParameterData->ParameterHandle->GetProperty()->SetMetaData(FName(TEXT("DisplayThumbnail")), TEXT("true"));
		FIntPoint ThumbnailOverride;
		if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
		{
			NameOverride = FMaterialPropertyHelpers::LayerID;
			ThumbnailOverride = FIntPoint(64, 64);
		}
		else if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
		{
			NameOverride = FMaterialPropertyHelpers::BlendID;
			ThumbnailOverride = FIntPoint(32, 32);
		}


		const int32 LayerStateIndex = StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter ? StackParameterData->ParameterInfo.Index + 1 : StackParameterData->ParameterInfo.Index;

		TAttribute<bool> IsParamEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::IsOverriddenExpression, StackParameterData->Parameter, LayerStateIndex));
		TAttribute<bool> IsAssetResetVisible = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(&FMaterialPropertyHelpers::ShouldLayerAssetShowResetToDefault, StackParameterData, MaterialEditorInstance->SourceInstance));
		FSimpleDelegate ResetAssetHandler = FSimpleDelegate::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::ResetAssetToDefault, StackParameterData);
		FResetToDefaultOverride ResetAssetOverride = FResetToDefaultOverride::Create(IsAssetResetVisible, ResetAssetHandler);

		IDetailTreeNode& Node = *StackParameterData->ParameterNode;

		ResetWidget = SNew(SResetToDefaultPropertyEditor, Node.CreatePropertyHandle())
			.IsEnabled(IsParamEnabled)
			.CustomResetToDefault(ResetAssetOverride);

		FNodeWidgets NodeWidgets = Node.CreateNodeWidgets();

		LeftSideWidget = StackParameterData->ParameterHandle->CreatePropertyNameWidget(NameOverride);

		StackParameterData->ParameterHandle->MarkResetToDefaultCustomized(false);

		EMaterialParameterAssociation InAssociation = StackParameterData->ParameterInfo.Association;

		FOnShouldFilterAsset AssetFilter = FOnShouldFilterAsset::CreateStatic(&FMaterialPropertyHelpers::FilterLayerAssets, Tree->FunctionInstance, InAssociation, StackParameterData->ParameterInfo.Index);

		FOnSetObject AssetChanged = FOnSetObject::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::RefreshOnAssetChange, StackParameterData->ParameterInfo.Index, InAssociation);

		FOnClicked OnChildButtonClicked;
		FOnClicked OnSiblingButtonClicked;
		UMaterialFunctionInterface* LocalFunction = nullptr;
		TSharedPtr<SBox> ThumbnailBox;

		if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
		{
			LocalFunction = Tree->FunctionInstance->Layers[StackParameterData->ParameterInfo.Index];
		}
		else if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
		{
			LocalFunction = Tree->FunctionInstance->Blends[StackParameterData->ParameterInfo.Index];
		}

		OnChildButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewLayerInstance,
			ImplicitConv<UMaterialFunctionInterface*>(LocalFunction), StackParameterData);

		TSharedPtr<SHorizontalBox> SaveInstanceBox;

		RightSideWidget = SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				.MaxWidth(ThumbnailOverride.X)
				[
					SAssignNew(ThumbnailBox, SBox)
					[
						Tree->CreateThumbnailWidget(StackParameterData->ParameterInfo.Association, StackParameterData->ParameterInfo.Index, ThumbnailOverride.X)
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMaterialFunctionInterface::StaticClass())
					.ObjectPath(this, &SMaterialLayersFunctionsInstanceTreeItem::GetInstancePath, Tree)
					.OnShouldFilterAsset(AssetFilter)
					.OnObjectChanged(AssetChanged)
					.DisplayCompactSize(true)
					.NewAssetFactories(FMaterialPropertyHelpers::GetAssetFactories(InAssociation))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
				[
					SNew(SCheckBox)
					.Type(ESlateCheckBoxType::ToggleButton)
					.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckboxAlt"))
					.OnCheckStateChanged(this, &SMaterialLayersFunctionsInstanceTreeItem::FilterClicked, InArgs._InTree, StackParameterData)
					.IsChecked(this, &SMaterialLayersFunctionsInstanceTreeItem::GetFilterChecked, InArgs._InTree, StackParameterData)
					.ToolTipText(LOCTEXT("FilterLayerAssets", "Filter asset picker to only show related layers or blends. \nStaying within the inheritance hierarchy can improve instruction count."))
					.Padding(2.0f)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(SaveInstanceBox, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.FillWidth(1.0)
				[
					SNullWidget::NullWidget
				]
			]
		;
		ThumbnailBox->SetMaxDesiredHeight(ThumbnailOverride.Y);
		ThumbnailBox->SetMinDesiredHeight(ThumbnailOverride.Y);
		ThumbnailBox->SetMinDesiredWidth(ThumbnailOverride.X);
		ThumbnailBox->SetMaxDesiredWidth(ThumbnailOverride.X);

		SaveInstanceBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("SaveChild", "Save Child"))
				.HAlign(HAlign_Center)
				.OnClicked(OnChildButtonClicked)
				.ToolTipText(LOCTEXT("SaveToChildInstance", "Save To Child Instance"))
			];
			
		LeftSideWidget->SetEnabled(InArgs._InTree->FunctionInstance->EditorOnly.LayerStates[LayerStateIndex]);
	}
// END ASSET

// PROPERTY ----------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Property)
	{
		const int32 LayerStateIndex = StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter ? StackParameterData->ParameterInfo.Index + 1 : StackParameterData->ParameterInfo.Index;
		WrapperWidget->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FMaterialPropertyHelpers::ShouldShowExpression, StackParameterData->Parameter, MaterialEditorInstance, Tree->GetShowHiddenDelegate())));

		// Handle our own EditCondition, force Visibility to "Visible" since WrapperWidget controls our actual visibility
		FMaterialParameterPropertyRowOverrides PropertyRowOverrides;
		PropertyRowOverrides.EditConditionValue = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::IsOverriddenExpression, StackParameterData->Parameter, LayerStateIndex));
		PropertyRowOverrides.OnEditConditionValueChanged = FOnBooleanValueChanged::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::OnOverrideParameter, StackParameterData->Parameter);
		PropertyRowOverrides.Visibility = TAttribute<EVisibility>(EVisibility::Visible);

		IDetailTreeNode& Node = *StackParameterData->ParameterNode;
		TSharedPtr<IPropertyHandle> PropertyHandle = Node.CreatePropertyHandle();
		TSharedPtr<IDetailPropertyRow> GeneratedRow = StaticCastSharedPtr<IDetailPropertyRow>(Node.GetRow());
		IDetailPropertyRow& Row = *GeneratedRow.Get();

		FMaterialPropertyHelpers::ConfigurePropertyRowForParameter(Row, StackParameterData->Parameter, PropertyHandle, MaterialEditorInstance, PropertyRowOverrides);
		FMaterialPropertyHelpers::SetPropertyRowParameterWidget(Row, StackParameterData->Parameter, PropertyHandle, MaterialEditorInstance);

		FNodeWidgets NodeWidgets = Node.CreateNodeWidgets();
		LeftSideWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(6.0f, 0.0f, 6.0f, 0.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				NodeWidgets.EditConditionWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			[
				NodeWidgets.NameWidget.ToSharedRef()
			];
		RightSideWidget = NodeWidgets.ValueWidget.ToSharedRef();

		ResetWidget = SNew(SResetToDefaultPropertyEditor, PropertyHandle)
			.IsEnabled(PropertyRowOverrides.EditConditionValue)
			.CustomResetToDefault(FMaterialPropertyHelpers::CreateResetToDefaultOverride(StackParameterData->Parameter, MaterialEditorInstance, PropertyHandle));

		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::UpdateThumbnailMaterial, StackParameterData->ParameterInfo.Association, StackParameterData->ParameterInfo.Index, false));
		PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(Tree, &SMaterialLayersFunctionsInstanceTree::UpdateThumbnailMaterial, StackParameterData->ParameterInfo.Association, StackParameterData->ParameterInfo.Index, false));

		LeftSideWidget->SetEnabled(InArgs._InTree->FunctionInstance->EditorOnly.LayerStates[LayerStateIndex]);

		RightSidePadding = FMargin(12.0f, 0.0f, 2.0f, 0.0f);	// Match DetailWidgetConstants::RightRowPadding
	}
// END PROPERTY

// PROPERTY CHILD ----------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::PropertyChild)
	{
		FNodeWidgets NodeWidgets = StackParameterData->ParameterNode->CreateNodeWidgets();
		LeftSideWidget = NodeWidgets.NameWidget.ToSharedRef();
		RightSideWidget = NodeWidgets.ValueWidget.ToSharedRef();

		const int32 LayerStateIndex = StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter ? StackParameterData->ParameterInfo.Index + 1 : StackParameterData->ParameterInfo.Index;
		LeftSideWidget->SetEnabled(InArgs._InTree->FunctionInstance->EditorOnly.LayerStates[LayerStateIndex]);

		TAttribute<bool> EnabledAttribute = TAttribute<bool>::Create([this, LayerStateIndex]() -> bool
			{
				return FMaterialPropertyHelpers::IsOverriddenExpression(StackParameterData->Parameter) && Tree->FunctionInstance->EditorOnly.LayerStates[LayerStateIndex];
			});
		RightSideWidget->SetEnabled(EnabledAttribute);

		RightSidePadding = FMargin(12.0f, 0.0f, 2.0f, 0.0f);	// Match DetailWidgetConstants::RightRowPadding
	}
// END PROPERTY CHILD

// FINAL WRAPPER
	if (StackParameterData->StackDataType == EStackDataType::Stack)
	{
		TSharedPtr<SHorizontalBox> FinalStack;
		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBorder)
					.BorderImage(this, &SMaterialLayersFunctionsInstanceTreeItem::GetBorderImage)
					.BorderBackgroundColor(this, &SMaterialLayersFunctionsInstanceTreeItem::GetOuterBackgroundColor, StackParameterData)
					.Padding(0.0f)
					[
						SAssignNew(FinalStack, SHorizontalBox)
					]
				]
			];
		if (StackParameterData->ParameterInfo.Index != 0)
		{
			FinalStack->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.5f, 0)
				.AutoWidth()
				[
					FMaterialPropertyHelpers::MakeStackReorderHandle(SharedThis(this))
				];
		}
		FinalStack->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0f))
			[
				SNew(SExpanderArrow, SharedThis(this))
			];
		FinalStack->AddSlot()
			.Padding(FMargin(2.0f))
			.VAlign(VAlign_Center)
			[
				LeftSideWidget
			];
	}
	else if (StackParameterData->StackDataType == EStackDataType::Group)
	{
		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(this, &SMaterialLayersFunctionsInstanceTreeItem::GetBorderImage)
					.BorderBackgroundColor(this, &SMaterialLayersFunctionsInstanceTreeItem::GetOuterBackgroundColor, StackParameterData)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(3.0f))
						[
							SNew(SExpanderArrow, SharedThis(this))
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(2.0f))
						.VAlign(VAlign_Center)
						[
							LeftSideWidget
						]
					]
				]
			];
	}
	else
	{
		if (ResetWidget == SNullWidget::NullWidget)
		{
			const FSlateBrush* DiffersFromDefaultBrush = FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault");
			ResetWidget = SNew(SSpacer)
				.Size(DiffersFromDefaultBrush != nullptr ? DiffersFromDefaultBrush->ImageSize : FVector2D(8.0f, 8.0f));
		}

		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(this, &SMaterialLayersFunctionsInstanceTreeItem::GetBorderImage)
					.BorderBackgroundColor(this, &SMaterialLayersFunctionsInstanceTreeItem::GetOuterBackgroundColor, StackParameterData)
					[
						SNew(SBox)
						.MinDesiredHeight(26.0f)	// Match PropertyEditorConstants::PropertyRowHeight
						[
							SNew(SSplitter)
							.Style(FAppStyle::Get(), "DetailsView.Splitter")
							.PhysicalSplitterHandleSize(1.0f)
							.HitDetectionSplitterHandleSize(5.0f)
							.HighlightedHandleIndex(Tree->ColumnSizeData.GetHoveredSplitterIndex())
							+ SSplitter::Slot()
							.Value(Tree->ColumnSizeData.GetNameColumnWidth())
							.OnSlotResized(Tree->ColumnSizeData.GetOnNameColumnResized())
							.Value(0.25f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(FMargin(3.0f))
								[
									SNew(SExpanderArrow, SharedThis(this))
								]
								+ SHorizontalBox::Slot()
								.Padding(FMargin(2.0f))
								.VAlign(VAlign_Center)
								[
									LeftSideWidget
								]
							]
							+ SSplitter::Slot()
							.Value(Tree->ColumnSizeData.GetValueColumnWidth())
							.OnSlotResized(Tree->ColumnSizeData.GetOnValueColumnResized())
							[
								SNew(SHorizontalBox)
								.Clipping(EWidgetClipping::OnDemand)
								+ SHorizontalBox::Slot()
								.HAlign(ValueAlignment)
								.VAlign(VAlign_Center)
								.Padding(RightSidePadding)
								[
									RightSideWidget
								]
							]
							+ SSplitter::Slot()
							.SizeRule(SSplitter::ESizeRule::SizeToContent)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.Padding(5.0f)
								[
									ResetWidget
								]
							]
						]
					]
				]
			];
	}


	this->ChildSlot
		[
			WrapperWidget
		];

	FOnTableRowDragEnter LayerDragDelegate = FOnTableRowDragEnter::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::OnLayerDragEnter);
	FOnTableRowDragLeave LayerDragLeaveDelegate = FOnTableRowDragLeave::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::OnLayerDragLeave);
	FOnTableRowDrop LayerDropDelegate = FOnTableRowDrop::CreateSP(this, &SMaterialLayersFunctionsInstanceTreeItem::OnLayerDrop);

	STableRow< TSharedPtr<FSortedParamData> >::ConstructInternal(
		STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false)
		.OnDragEnter(LayerDragDelegate)
		.OnDragLeave(LayerDragLeaveDelegate)
		.OnDrop(LayerDropDelegate),
		InOwnerTableView
	);
}

FString SMaterialLayersFunctionsInstanceTreeItem::GetInstancePath(SMaterialLayersFunctionsInstanceTree* InTree) const
{
	FString InstancePath;
	if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter && InTree->FunctionInstance->Blends.IsValidIndex(StackParameterData->ParameterInfo.Index))
	{
		InstancePath = InTree->FunctionInstance->Blends[StackParameterData->ParameterInfo.Index]->GetPathName();
	}
	else if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter && InTree->FunctionInstance->Layers.IsValidIndex(StackParameterData->ParameterInfo.Index))
	{
		InstancePath = InTree->FunctionInstance->Layers[StackParameterData->ParameterInfo.Index]->GetPathName();
	}
	return InstancePath;
}

void SMaterialLayersFunctionsInstanceTree::Construct(const FArguments& InArgs)
{
	ColumnSizeData.SetValueColumnWidth(0.5f);

	MaterialEditorInstance = InArgs._InMaterialEditorInstance;
	Wrapper = InArgs._InWrapper;
	ShowHiddenDelegate = InArgs._InShowHiddenDelegate;
	NotifyHook = InArgs._InNotifyHook;
	CreateGroupsWidget();

#if WITH_EDITOR
	//Fixup for adding new bool arrays to the class
	if (FunctionInstance)
	{
		if (FunctionInstance->Layers.Num() != FunctionInstance->EditorOnly.RestrictToLayerRelatives.Num())
		{
			int32 OriginalSize = FunctionInstance->EditorOnly.RestrictToLayerRelatives.Num();
			for (int32 LayerIt = 0; LayerIt < FunctionInstance->Layers.Num() - OriginalSize; LayerIt++)
			{
				FunctionInstance->EditorOnly.RestrictToLayerRelatives.Add(false);
			}
		}
		if (FunctionInstance->Blends.Num() != FunctionInstance->EditorOnly.RestrictToBlendRelatives.Num())
		{
			int32 OriginalSize = FunctionInstance->EditorOnly.RestrictToBlendRelatives.Num();
			for (int32 BlendIt = 0; BlendIt < FunctionInstance->Blends.Num() - OriginalSize; BlendIt++)
			{
				FunctionInstance->EditorOnly.RestrictToBlendRelatives.Add(false);
			}
		}
	}
#endif

	STreeView<TSharedPtr<FSortedParamData>>::Construct(
		STreeView::FArguments()
		.TreeItemsSource(&LayerProperties)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SMaterialLayersFunctionsInstanceTree::OnGenerateRowMaterialLayersFunctionsTreeView)
		.OnGetChildren(this, &SMaterialLayersFunctionsInstanceTree::OnGetChildrenMaterialLayersFunctionsTreeView)
		.OnExpansionChanged(this, &SMaterialLayersFunctionsInstanceTree::OnExpansionChanged)
	);

	SetParentsExpansionState();
}

TSharedRef< ITableRow > SMaterialLayersFunctionsInstanceTree::OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SMaterialLayersFunctionsInstanceTreeItem > ReturnRow = SNew(SMaterialLayersFunctionsInstanceTreeItem, OwnerTable)
		.StackParameterData(Item)
		.MaterialEditorInstance(MaterialEditorInstance)
		.InTree(this);
	return ReturnRow;
}

void SMaterialLayersFunctionsInstanceTree::OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> InParent, TArray< TSharedPtr<FSortedParamData> >& OutChildren)
{
	OutChildren = InParent->Children;
}


void SMaterialLayersFunctionsInstanceTree::OnExpansionChanged(TSharedPtr<FSortedParamData> Item, bool bIsExpanded)
{
	bool* ExpansionValue = MaterialEditorInstance->SourceInstance->LayerParameterExpansion.Find(Item->NodeKey);
	if (ExpansionValue == nullptr)
	{
		MaterialEditorInstance->SourceInstance->LayerParameterExpansion.Add(Item->NodeKey, bIsExpanded);
	}
	else if (*ExpansionValue != bIsExpanded)
	{
		MaterialEditorInstance->SourceInstance->LayerParameterExpansion.Emplace(Item->NodeKey, bIsExpanded);
	}
	// Expand any children that are also expanded
	for (auto Child : Item->Children)
	{
		bool* ChildExpansionValue = MaterialEditorInstance->SourceInstance->LayerParameterExpansion.Find(Child->NodeKey);
		if (ChildExpansionValue != nullptr && *ChildExpansionValue == true)
		{
			SetItemExpansion(Child, true);
		}
	}
}

void SMaterialLayersFunctionsInstanceTree::SetParentsExpansionState()
{
	for (const auto& Pair : LayerProperties)
	{
		if (Pair->Children.Num())
		{
			bool* bIsExpanded = MaterialEditorInstance->SourceInstance->LayerParameterExpansion.Find(Pair->NodeKey);
			if (bIsExpanded)
			{
				SetItemExpansion(Pair, *bIsExpanded);
			}
		}
	}
}

void SMaterialLayersFunctionsInstanceTree::RefreshOnAssetChange(const struct FAssetData& InAssetData, int32 Index, EMaterialParameterAssociation MaterialType)
{
	FMaterialPropertyHelpers::OnMaterialLayerAssetChanged(InAssetData, Index, MaterialType, FunctionInstanceHandle, FunctionInstance);
	//set their overrides back to 0
	MaterialEditorInstance->CleanParameterStack(Index, MaterialType);
	CreateGroupsWidget();
	MaterialEditorInstance->ResetOverrides(Index, MaterialType);
	RequestTreeRefresh();
}

void SMaterialLayersFunctionsInstanceTree::ResetAssetToDefault(TSharedPtr<FSortedParamData> InData)
{
	FMaterialPropertyHelpers::ResetLayerAssetToDefault(InData->Parameter, InData->ParameterInfo.Association, InData->ParameterInfo.Index, MaterialEditorInstance);
	UpdateThumbnailMaterial(InData->ParameterInfo.Association, InData->ParameterInfo.Index, false);
	MaterialEditorInstance->RegenerateArrays();
	CreateGroupsWidget();
	RequestTreeRefresh();
}

void SMaterialLayersFunctionsInstanceTree::AddLayer()
{
	const FScopedTransaction Transaction(LOCTEXT("AddLayerAndBlend", "Add a new Layer and a Blend into it"));
	FunctionInstanceHandle->NotifyPreChange();
	FunctionInstance->AppendBlendedLayer();
	FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	MaterialEditorInstance->RegenerateArrays();
	CreateGroupsWidget();
	RequestTreeRefresh();
}

void SMaterialLayersFunctionsInstanceTree::RemoveLayer(int32 Index)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveLayerAndBlend", "Remove a Layer and the attached Blend"));
	FunctionInstanceHandle->NotifyPreChange();
	FunctionInstance->RemoveBlendedLayerAt(Index);
	MaterialEditorInstance->SourceInstance->RemoveLayerParameterIndex(Index);
	FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ArrayRemove);
	MaterialEditorInstance->RegenerateArrays();
	CreateGroupsWidget();
	RequestTreeRefresh();
}

FReply SMaterialLayersFunctionsInstanceTree::UnlinkLayer(int32 Index)
{
	const FScopedTransaction Transaction(LOCTEXT("UnlinkLayerFromParent", "Unlink a layer from the parent"));
	FunctionInstanceHandle->NotifyPreChange();
	FunctionInstance->UnlinkLayerFromParent(Index);
	FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	MaterialEditorInstance->RegenerateArrays();
	CreateGroupsWidget();

	return FReply::Handled();
}

FReply SMaterialLayersFunctionsInstanceTree::RelinkLayersToParent()
{
	const FScopedTransaction Transaction(LOCTEXT("RelinkLayersToParent", "Relink layers to parent"));
	FunctionInstanceHandle->NotifyPreChange();
	FunctionInstance->RelinkLayersToParent();
	FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	MaterialEditorInstance->RegenerateArrays();
	CreateGroupsWidget();

	return FReply::Handled();
}

EVisibility SMaterialLayersFunctionsInstanceTree::GetUnlinkLayerVisibility(int32 Index) const
{
	if (FunctionInstance->IsLayerLinkedToParent(Index))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

EVisibility SMaterialLayersFunctionsInstanceTree::GetRelinkLayersToParentVisibility() const
{
	if (FunctionInstance->HasAnyUnlinkedLayers())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

FReply SMaterialLayersFunctionsInstanceTree::ToggleLayerVisibility(int32 Index)
{
	if (!FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Alt))
	{
		bLayerIsolated = false;
		const FScopedTransaction Transaction(LOCTEXT("ToggleLayerAndBlendVisibility", "Toggles visibility for a blended layer"));
		FunctionInstanceHandle->NotifyPreChange();
		FunctionInstance->ToggleBlendedLayerVisibility(Index);
	}
	else
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleLayerAndBlendVisibility", "Toggles visibility for a blended layer"));
		FunctionInstanceHandle->NotifyPreChange();
		if (FunctionInstance->GetLayerVisibility(Index) == false)
		{
			// Reset if clicking on a disabled layer
			FunctionInstance->SetBlendedLayerVisibility(Index, true);
			bLayerIsolated = false;
		}
		for (int32 LayerIt = 1; LayerIt < FunctionInstance->EditorOnly.LayerStates.Num(); LayerIt++)
		{
			if (LayerIt != Index)
			{
				FunctionInstance->SetBlendedLayerVisibility(LayerIt, bLayerIsolated);
			}
		}

		bLayerIsolated = !bLayerIsolated;
	}

	FunctionInstanceHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	MaterialEditorInstance->RegenerateArrays();
	CreateGroupsWidget();

	return FReply::Handled();
}

TSharedPtr<class FAssetThumbnailPool> SMaterialLayersFunctionsInstanceTree::GetTreeThumbnailPool()
{
	return UThumbnailManager::Get().GetSharedThumbnailPool();
}

TSharedPtr<IDetailTreeNode> FindParameterGroupsNode(TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator)
{
	const TArray<TSharedRef<IDetailTreeNode>> RootNodes = PropertyRowGenerator->GetRootTreeNodes();
	if (RootNodes.Num() > 0)
	{
		TSharedPtr<IDetailTreeNode> Category = RootNodes[0];
		TArray<TSharedRef<IDetailTreeNode>> Children;
		Category->GetChildren(Children);

		for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ChildIdx++)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = Children[ChildIdx]->CreatePropertyHandle();
			if (PropertyHandle.IsValid() && PropertyHandle->GetProperty() && PropertyHandle->GetProperty()->GetName() == "ParameterGroups")
			{
				return Children[ChildIdx];
			}
		}
	}
	return nullptr;
}

void SMaterialLayersFunctionsInstanceTree::CreateGroupsWidget()
{
	check(MaterialEditorInstance);

	NonLayerProperties.Empty();
	LayerProperties.Empty();
	FunctionParameter = nullptr;
	FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	if (!Generator.IsValid())
	{
		FPropertyRowGeneratorArgs Args;
		Args.NotifyHook = NotifyHook;

		Generator = Module.CreatePropertyRowGenerator(Args);
		// the sizes of the parameter lists are only based on the parent material and not changed out from under the details panel 
		// When a parameter is added open MI editors are refreshed
		// the tree should also refresh if one of the layer or blend assets is swapped

		auto ValidationLambda = ([](const FRootPropertyNodeList& PropertyNodeList) { return true; });
		Generator->SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes::CreateLambda(MoveTemp(ValidationLambda)));

		TArray<UObject*> Objects;
		Objects.Add(MaterialEditorInstance);
		Generator->SetObjects(Objects);
	}
	else
	{
		TArray<UObject*> Objects;
		Objects.Add(MaterialEditorInstance);
		Generator->SetObjects(Objects);
	}

	TSharedPtr<IDetailTreeNode> ParameterGroups = FindParameterGroupsNode(Generator);
	if (ParameterGroups.IsValid())
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		ParameterGroups->GetChildren(Children);
		// the order of DeferredSearches should correspond to NonLayerProperties exactly
		TArray<TSharedPtr<IPropertyHandle>> DeferredSearches;
		for (int32 GroupIdx = 0; GroupIdx < Children.Num(); ++GroupIdx)
		{
			TArray<void*> GroupPtrs;
			TSharedPtr<IPropertyHandle> ChildHandle = Children[GroupIdx]->CreatePropertyHandle();
			ChildHandle->AccessRawData(GroupPtrs);
			auto GroupIt = GroupPtrs.CreateConstIterator();
			const FEditorParameterGroup* ParameterGroupPtr = reinterpret_cast<FEditorParameterGroup*>(*GroupIt);
			const FEditorParameterGroup& ParameterGroup = *ParameterGroupPtr;

			for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ParamIdx++)
			{
				UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];

				TSharedPtr<IPropertyHandle> ParametersArrayProperty = ChildHandle->GetChildHandle("Parameters");
				TSharedPtr<IPropertyHandle> ParameterProperty = ParametersArrayProperty->GetChildHandle(ParamIdx);
				TSharedPtr<IPropertyHandle> ParameterValueProperty = ParameterProperty->GetChildHandle("ParameterValue");

				if (Cast<UDEditorMaterialLayersParameterValue>(Parameter))
				{
					if (FunctionParameter == nullptr)
					{
						FunctionParameter = Parameter;
					}
					TArray<void*> StructPtrs;
					ParameterValueProperty->AccessRawData(StructPtrs);
					auto It = StructPtrs.CreateConstIterator();
					FunctionInstance = reinterpret_cast<FMaterialLayersFunctions*>(*It);
					FunctionInstanceHandle = ParameterValueProperty;

					TSharedPtr<IPropertyHandle>	LayerHandle = ChildHandle->GetChildHandle("Layers").ToSharedRef();
					TSharedPtr<IPropertyHandle> BlendHandle = ChildHandle->GetChildHandle("Blends").ToSharedRef();
					uint32 LayerChildren;
					LayerHandle->GetNumChildren(LayerChildren);
					uint32 BlendChildren;
					BlendHandle->GetNumChildren(BlendChildren);
					if (MaterialEditorInstance->StoredLayerPreviews.Num() != LayerChildren)
					{
						MaterialEditorInstance->StoredLayerPreviews.Empty();
						MaterialEditorInstance->StoredLayerPreviews.AddDefaulted(LayerChildren);
					}
					if (MaterialEditorInstance->StoredBlendPreviews.Num() != BlendChildren)
					{
						MaterialEditorInstance->StoredBlendPreviews.Empty();
						MaterialEditorInstance->StoredBlendPreviews.AddDefaulted(BlendChildren);
					}

					TSharedRef<FSortedParamData> StackProperty = MakeShared<FSortedParamData>();
					StackProperty->StackDataType = EStackDataType::Stack;
					StackProperty->Parameter = Parameter;
					StackProperty->ParameterInfo.Index = LayerChildren - 1;
					StackProperty->NodeKey = FString::FromInt(StackProperty->ParameterInfo.Index);


					TSharedRef<FSortedParamData> ChildProperty = MakeShared<FSortedParamData>();
					ChildProperty->StackDataType = EStackDataType::Asset;
					ChildProperty->Parameter = Parameter;
					ChildProperty->ParameterHandle = LayerHandle->AsArray()->GetElement(LayerChildren - 1);
					ChildProperty->ParameterNode = Generator->FindTreeNode(ChildProperty->ParameterHandle);
					ChildProperty->ParameterInfo.Index = LayerChildren - 1;
					ChildProperty->ParameterInfo.Association = EMaterialParameterAssociation::LayerParameter;
					ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association);

					{
						UObject* AssetObject = nullptr;
						ChildProperty->ParameterHandle->GetValue(AssetObject);
						if (AssetObject)
						{
							if (MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1] == nullptr)
							{
								MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1] = (NewObject<UMaterialInstanceConstant>(MaterialEditorInstance, NAME_None));
							}
							UMaterialInterface* EditedMaterial = Cast<UMaterialFunctionInterface>(AssetObject)->GetPreviewMaterial();
							if (MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1] && MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1]->Parent != EditedMaterial)
							{
								MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1]->SetParentEditorOnly(EditedMaterial);
							}
						}
					}

					StackProperty->Children.Add(ChildProperty);
					LayerProperties.Add(StackProperty);

					if (BlendChildren > 0 && LayerChildren > BlendChildren)
					{
						for (int32 Counter = BlendChildren - 1; Counter >= 0; Counter--)
						{
							ChildProperty = MakeShared<FSortedParamData>();
							ChildProperty->StackDataType = EStackDataType::Asset;
							ChildProperty->Parameter = Parameter;
							ChildProperty->ParameterHandle = BlendHandle->AsArray()->GetElement(Counter);
							ChildProperty->ParameterNode = Generator->FindTreeNode(ChildProperty->ParameterHandle);
							ChildProperty->ParameterInfo.Index = Counter;
							ChildProperty->ParameterInfo.Association = EMaterialParameterAssociation::BlendParameter;
							ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association);
							{
								UObject* AssetObject = nullptr;
								ChildProperty->ParameterHandle->GetValue(AssetObject);
								if (AssetObject)
								{
									if (MaterialEditorInstance->StoredBlendPreviews[Counter] == nullptr)
									{
										MaterialEditorInstance->StoredBlendPreviews[Counter] = (NewObject<UMaterialInstanceConstant>(MaterialEditorInstance, NAME_None));
									}
									UMaterialInterface* EditedMaterial = Cast<UMaterialFunctionInterface>(AssetObject)->GetPreviewMaterial();
									if (MaterialEditorInstance->StoredBlendPreviews[Counter] && MaterialEditorInstance->StoredBlendPreviews[Counter]->Parent != EditedMaterial)
									{
										MaterialEditorInstance->StoredBlendPreviews[Counter]->SetParentEditorOnly(EditedMaterial);
									}
								}
							}
							LayerProperties.Last()->Children.Add(ChildProperty);

							StackProperty = MakeShared<FSortedParamData>();
							StackProperty->StackDataType = EStackDataType::Stack;
							StackProperty->Parameter = Parameter;
							StackProperty->ParameterInfo.Index = Counter;
							StackProperty->NodeKey = FString::FromInt(StackProperty->ParameterInfo.Index);
							LayerProperties.Add(StackProperty);

							ChildProperty = MakeShared<FSortedParamData>();
							ChildProperty->StackDataType = EStackDataType::Asset;
							ChildProperty->Parameter = Parameter;
							ChildProperty->ParameterHandle = LayerHandle->AsArray()->GetElement(Counter);
							ChildProperty->ParameterNode = Generator->FindTreeNode(ChildProperty->ParameterHandle);
							ChildProperty->ParameterInfo.Index = Counter;
							ChildProperty->ParameterInfo.Association = EMaterialParameterAssociation::LayerParameter;
							ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association);
							{
								UObject* AssetObject = nullptr;
								ChildProperty->ParameterHandle->GetValue(AssetObject);
								if (AssetObject)
								{
									if (MaterialEditorInstance->StoredLayerPreviews[Counter] == nullptr)
									{
										MaterialEditorInstance->StoredLayerPreviews[Counter] = (NewObject<UMaterialInstanceConstant>(MaterialEditorInstance, NAME_None));
									}
									UMaterialInterface* EditedMaterial = Cast<UMaterialFunctionInterface>(AssetObject)->GetPreviewMaterial();
									if (MaterialEditorInstance->StoredLayerPreviews[Counter] && MaterialEditorInstance->StoredLayerPreviews[Counter]->Parent != EditedMaterial)
									{
										MaterialEditorInstance->StoredLayerPreviews[Counter]->SetParentEditorOnly(EditedMaterial);
									}
								}
							}
							LayerProperties.Last()->Children.Add(ChildProperty);
						}
					}
				}
				else
				{
					// Skip parameters that shouldn't have widgets here
					if (!FMaterialPropertyHelpers::ShouldCreatePropertyRowForParameter(Parameter))
					{
						continue;
					}

					FUnsortedParamData NonLayerProperty;

					NonLayerProperty.Parameter = Parameter;
					NonLayerProperty.ParameterGroup = ParameterGroup;

					DeferredSearches.Add(ParameterValueProperty);
					NonLayerProperty.UnsortedName = Parameter->ParameterInfo.Name;

					NonLayerProperties.Add(NonLayerProperty);
				}
			}
		}

		checkf(NonLayerProperties.Num() == DeferredSearches.Num(), TEXT("Internal inconsistency: number of node searches does not match the number of properties"));
		TArray<TSharedPtr<IDetailTreeNode>> DeferredResults = Generator->FindTreeNodes(DeferredSearches);
		checkf(NonLayerProperties.Num() == DeferredResults.Num(), TEXT("Internal inconsistency: number of node search results does not match the number of properties"));

		for (int Idx = 0, NumUnsorted = NonLayerProperties.Num(); Idx < NumUnsorted; ++Idx)
		{
			FUnsortedParamData& NonLayerProperty = NonLayerProperties[Idx];
			NonLayerProperty.ParameterNode = DeferredResults[Idx];
			NonLayerProperty.ParameterHandle = NonLayerProperty.ParameterNode->CreatePropertyHandle();
		}

		DeferredResults.Empty();
		DeferredSearches.Empty();

		for (int32 LayerIdx = 0; LayerIdx < LayerProperties.Num(); LayerIdx++)
		{
			for (int32 ChildIdx = 0; ChildIdx < LayerProperties[LayerIdx]->Children.Num(); ChildIdx++)
			{
				ShowSubParameters(LayerProperties[LayerIdx]->Children[ChildIdx]);
			}
		}
	}

	SetParentsExpansionState();
}

bool SMaterialLayersFunctionsInstanceTree::IsLayerVisible(int32 Index) const
{
	if (FunctionParameter.IsValid())
	{
		return FunctionInstance->GetLayerVisibility(Index);
	}
	return false;
}

TSharedRef<SWidget> SMaterialLayersFunctionsInstanceTree::CreateThumbnailWidget(EMaterialParameterAssociation InAssociation, int32 InIndex, float InThumbnailSize)
{
	UObject* ThumbnailObject = nullptr;
	if (InAssociation == EMaterialParameterAssociation::LayerParameter)
	{
		ThumbnailObject = MaterialEditorInstance->StoredLayerPreviews[InIndex];
	}
	else if (InAssociation == EMaterialParameterAssociation::BlendParameter)
	{
		ThumbnailObject = MaterialEditorInstance->StoredBlendPreviews[InIndex];
	}
	const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable(new FAssetThumbnail(ThumbnailObject, InThumbnailSize, InThumbnailSize, GetTreeThumbnailPool()));
	TSharedRef<SWidget> ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();
	ThumbnailWidget->SetOnMouseDoubleClick(FPointerEventHandler::CreateSP(this, &SMaterialLayersFunctionsInstanceTree::OnThumbnailDoubleClick, InAssociation, InIndex));
	return ThumbnailWidget;
}

void SMaterialLayersFunctionsInstanceTree::UpdateThumbnailMaterial(TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 InIndex, bool bAlterBlendIndex)
{
	// Need to invert index b/c layer properties is generated in reverse order
	TArray<TSharedPtr<FSortedParamData>> AssetChildren = LayerProperties[LayerProperties.Num() - 1 - InIndex]->Children;
	UMaterialInstanceConstant* MaterialToUpdate = nullptr;
	int32 ParameterIndex = InIndex;
	if (InAssociation == EMaterialParameterAssociation::LayerParameter)
	{
		MaterialToUpdate = MaterialEditorInstance->StoredLayerPreviews[ParameterIndex];
	}
	if (InAssociation == EMaterialParameterAssociation::BlendParameter)
	{
		if (bAlterBlendIndex)
		{
			ParameterIndex--;
		}
		MaterialToUpdate = MaterialEditorInstance->StoredBlendPreviews[ParameterIndex];
	}

	TArray<FEditorParameterGroup> ParameterGroups;
	for (TSharedPtr<FSortedParamData> AssetChild : AssetChildren)
	{
		for (TSharedPtr<FSortedParamData> Group : AssetChild->Children)
		{
			if (Group->ParameterInfo.Association == InAssociation)
			{
				FEditorParameterGroup DuplicatedGroup = FEditorParameterGroup();
				DuplicatedGroup.GroupAssociation = Group->Group.GroupAssociation;
				DuplicatedGroup.GroupName = Group->Group.GroupName;
				DuplicatedGroup.GroupSortPriority = Group->Group.GroupSortPriority;
				for (UDEditorParameterValue* Parameter : Group->Group.Parameters)
				{
					if (Parameter->ParameterInfo.Index == ParameterIndex)
					{
						DuplicatedGroup.Parameters.Add(Parameter);
					}
				}
				ParameterGroups.Add(DuplicatedGroup);
			}
		}

	

	}
	if (MaterialToUpdate != nullptr)
	{
		FMaterialPropertyHelpers::TransitionAndCopyParameters(MaterialToUpdate, ParameterGroups, true);
	}
}

FReply SMaterialLayersFunctionsInstanceTree::OnThumbnailDoubleClick(const FGeometry& Geometry, const FPointerEvent& MouseEvent, EMaterialParameterAssociation InAssociation, int32 InIndex)
{
	UMaterialFunctionInterface* AssetToOpen = nullptr;
	if (InAssociation == EMaterialParameterAssociation::BlendParameter)
	{
		AssetToOpen = FunctionInstance->Blends[InIndex];
	}
	else if (InAssociation == EMaterialParameterAssociation::LayerParameter)
	{
		AssetToOpen = FunctionInstance->Layers[InIndex];
	}
	if (AssetToOpen != nullptr)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetToOpen);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SMaterialLayersFunctionsInstanceTree::ShowSubParameters(TSharedPtr<FSortedParamData> ParentParameter)
{
	for (FUnsortedParamData Property : NonLayerProperties)
	{
		UDEditorParameterValue* Parameter = Property.Parameter;
		if (Parameter->ParameterInfo.Index == ParentParameter->ParameterInfo.Index
			&& Parameter->ParameterInfo.Association == ParentParameter->ParameterInfo.Association)
		{
			FString GroupNodeKey = FString::FromInt(Parameter->ParameterInfo.Index) + FString::FromInt(Parameter->ParameterInfo.Association) + Property.ParameterGroup.GroupName.ToString();

			bool bAddNewGroup = true;
			for (TSharedPtr<struct FSortedParamData> GroupChild : ParentParameter->Children)
			{
				if (GroupChild->NodeKey == GroupNodeKey)
				{
					bAddNewGroup = false;
				}
			}
			if (bAddNewGroup)
			{
				TSharedPtr<FSortedParamData> GroupProperty(new FSortedParamData());
				GroupProperty->StackDataType = EStackDataType::Group;
				GroupProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
				GroupProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
				GroupProperty->Group = Property.ParameterGroup;
				GroupProperty->NodeKey = GroupNodeKey;

				ParentParameter->Children.Add(GroupProperty);
			}

			TSharedPtr<FSortedParamData> ChildProperty(new FSortedParamData());
			ChildProperty->StackDataType = EStackDataType::Property;
			ChildProperty->Parameter = Parameter;
			ChildProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
			ChildProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
			ChildProperty->ParameterNode = Property.ParameterNode;
			ChildProperty->PropertyName = Property.UnsortedName;
			ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association) +  Property.ParameterGroup.GroupName.ToString() + Property.UnsortedName.ToString();


			if (FMaterialPropertyHelpers::ShouldCreateChildPropertiesForParameter(Parameter))
			{
				TArray<TSharedRef<IDetailTreeNode>> ParamChildren;
				Property.ParameterNode->GetChildren(ParamChildren);
				for (int32 ParamChildIdx = 0; ParamChildIdx < ParamChildren.Num(); ParamChildIdx++)
				{
					TSharedPtr<FSortedParamData> ParamChildProperty(new FSortedParamData());
					ParamChildProperty->StackDataType = EStackDataType::PropertyChild;
					ParamChildProperty->ParameterNode = ParamChildren[ParamChildIdx];
					ParamChildProperty->ParameterHandle = ParamChildProperty->ParameterNode->CreatePropertyHandle();
					ParamChildProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
					ParamChildProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
					ParamChildProperty->Parameter = ChildProperty->Parameter;
					ChildProperty->Children.Add(ParamChildProperty);
				}
			}
			for (TSharedPtr<struct FSortedParamData> GroupChild : ParentParameter->Children)
			{
				if (GroupChild->Group.GroupName == Property.ParameterGroup.GroupName
					&& GroupChild->ParameterInfo.Association == ChildProperty->ParameterInfo.Association
					&&  GroupChild->ParameterInfo.Index == ChildProperty->ParameterInfo.Index)
				{
					GroupChild->Children.Add(ChildProperty);
				}
			}

		}
	}
}

void SMaterialLayersFunctionsInstanceWrapper::Refresh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SMaterialLayersFunctionsInstanceWrapper::Refresh)

	LayerParameter.Reset();

	const bool bHasLayers = HasLayers();

	if (bHasLayers)
	{
		TSharedPtr<SHorizontalBox> HeaderBox;
		NestedTree->CreateGroupsWidget();
		LayerParameter = NestedTree->GetFunctionParameter();
		FOnClicked 	OnChildButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewMaterialInstance, MaterialEditorInstance->GetMaterialInterface(), ImplicitConv<UObject*>(MaterialEditorInstance));
		FOnClicked	OnSiblingButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewMaterialInstance, MaterialEditorInstance->GetParentMaterialInterface(), ImplicitConv<UObject*>(MaterialEditorInstance));

		constexpr float ThumbnailSize = 48.0f;
		TSharedRef<SWidget> ThumbnailWidget = SNullWidget::NullWidget;
		TSharedPtr<SBox> ThumbnailBox = nullptr;
		FText MaterialName = FText::GetEmpty();

		if (LayerParameter != nullptr)
		{

			FOnClicked OnRelinkToParent = FOnClicked::CreateSPLambda(this, [this]()
				{
					return NestedTree->RelinkLayersToParent();
				});

			HeaderBox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(4.0f, 0.0f))
				.HAlign(HAlign_Left)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("MaterialLayers", "Material Layers"))
				];

			this->ChildSlot
				[
					SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
						.AutoHeight()
						[
							HeaderBox.ToSharedRef()
						]
						+ SVerticalBox::Slot()
						.Padding(FMargin(2.0f))
						[
							NestedTree.ToSharedRef()
						]
				];
			if (FMaterialPropertyHelpers::IsOverriddenExpression(NestedTree->GetFunctionParameter()))
			{
				HeaderBox->AddSlot()
					.HAlign(HAlign_Left)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSPLambda(this, [this]()
							{
								NestedTree->AddRootNodeLayer();
							}))
					];
			}

			HeaderBox->AddSlot()
				.FillWidth(1.0f)
				[
					SNullWidget::NullWidget
				];
			HeaderBox->AddSlot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("Relink", "Relink"))
						.HAlign(HAlign_Center)
						.OnClicked(OnRelinkToParent)
						.ToolTipText(LOCTEXT("RelinkToParentLayers", "Relink to Parent Layers and Blends"))
						.Visibility_Lambda([this]()
							{
								return NestedTree->GetRelinkLayersToParentVisibility();
							})
				];
			HeaderBox->AddSlot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("SaveSibling", "Save Sibling"))
						.HAlign(HAlign_Center)
						.OnClicked(OnSiblingButtonClicked)
						.ToolTipText(LOCTEXT("SaveToSiblingInstance", "Save to Sibling Instance"))
				];
			HeaderBox->AddSlot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("SaveChild", "Save Child"))
						.HAlign(HAlign_Center)
						.OnClicked(OnChildButtonClicked)
						.ToolTipText(LOCTEXT("SaveToChildInstance", "Save To Child Instance"))
				];
		}
	}
	else
	{
		this->ChildSlot
			[
				SNew(SBox)
					.Padding(FMargin(10.0f))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("AddLayerParameterPrompt", "Add a Material Attribute Layers parameter to see it here."))
					]
			];
	}
}

bool SMaterialLayersFunctionsInstanceWrapper::HasLayers() const
{
	UMaterialInterface* Material = MaterialEditorInstance->GetMaterialInterface();
	UMaterialInterface* ParentMaterial = MaterialEditorInstance->GetParentMaterialInterface();

	FMaterialLayersFunctions Layers;
	const bool bHasLayers = ParentMaterial ? ParentMaterial->GetMaterialLayers(Layers) : Material->GetMaterialLayers(Layers);

	return bHasLayers;
}

void SMaterialLayersFunctionsInstanceWrapper::Construct(const FArguments& InArgs)
{
	NestedTree = SNew(SMaterialLayersFunctionsInstanceTree)
		.InMaterialEditorInstance(Cast<UMaterialEditorInstanceConstant>(InArgs._InMaterialEditorInstance))
		.InWrapper(this)
		.InShowHiddenDelegate(InArgs._InShowHiddenDelegate)
		.InNotifyHook(InArgs._InNotifyHook);

	LayerParameter = NestedTree->GetFunctionParameter();

	MaterialEditorInstance = InArgs._InMaterialEditorInstance;

	Refresh();
}

void SMaterialLayersFunctionsInstanceWrapper::SetEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance)
{
	NestedTree->SetMaterialEditorInstance(InMaterialEditorInstance);

	const bool bHasLayers = HasLayers();
	if (bHasLayers)
	{
		MaterialEditorInstance->RegenerateArrays();
	}
	Refresh();
}

TSharedPtr<IPropertyRowGenerator> SMaterialLayersFunctionsInstanceWrapper::GetGenerator()
{
	return Generator.Pin();
}


#undef LOCTEXT_NAMESPACE









/////////////// MATERIAL VERSION
#define LOCTEXT_NAMESPACE "MaterialLayerDisplay"

const FSlateBrush* SMaterialLayersFunctionsMaterialTreeItem::GetBorderImage() const
{
	return FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
}

FSlateColor SMaterialLayersFunctionsMaterialTreeItem::GetOuterBackgroundColor(TSharedPtr<FSortedParamData> InParamData) const
{
	if (InParamData->StackDataType == EStackDataType::Stack)
	{
		return FAppStyle::Get().GetSlateColor("Colors.Header");
	}
	else if (IsHovered() || InParamData->StackDataType == EStackDataType::Group)
	{
		return FAppStyle::Get().GetSlateColor("Colors.Header");
	}

	return FAppStyle::Get().GetSlateColor("Colors.Panel");
}

void SMaterialLayersFunctionsMaterialTreeItem::RefreshOnRowChange(const FAssetData& AssetData, SMaterialLayersFunctionsMaterialTree* InTree)
{
	if (SMaterialLayersFunctionsMaterialWrapper* Wrapper = InTree->GetWrapper())
	{
		InTree->CreateGroupsWidget();
	}
}

FText SMaterialLayersFunctionsMaterialTreeItem::GetLayerName(SMaterialLayersFunctionsMaterialTree* InTree, int32 Counter) const
{
	if (InTree->FunctionParameter.IsValid())
	{
		return InTree->FunctionInstance->GetLayerName(Counter);
	}
	return FText();
}

void SMaterialLayersFunctionsMaterialTreeItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	StackParameterData = InArgs._StackParameterData;
	MaterialEditorInstance = InArgs._MaterialEditorInstance;
	Tree = InArgs._InTree;

	TSharedRef<SWidget> LeftSideWidget = SNullWidget::NullWidget;
	TSharedRef<SWidget> RightSideWidget = SNullWidget::NullWidget;
	FText NameOverride;
	TSharedRef<SVerticalBox> WrapperWidget = SNew(SVerticalBox);
	EHorizontalAlignment ValueAlignment = HAlign_Left;
	// STACK --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Stack)
	{
		WrapperWidget->AddSlot()
			.Padding(0.0f)
			.AutoHeight()
			[
				SNullWidget::NullWidget
			];
#if WITH_EDITOR
		NameOverride = Tree->FunctionInstance->GetLayerName(StackParameterData->ParameterInfo.Index);
#endif
		TSharedRef<SHorizontalBox> HeaderRowWidget = SNew(SHorizontalBox);

		const float ThumbnailSize = 24.0f;
		TArray<TSharedPtr<FSortedParamData>> AssetChildren = StackParameterData->Children;
		if (AssetChildren.Num() > 0)
		{
			HeaderRowWidget->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				.AutoWidth()
				[
					SNullWidget::NullWidget
				];
		}
		for (TSharedPtr<FSortedParamData> AssetChild : AssetChildren)
		{
			TSharedPtr<SBox> ThumbnailBox;
			FAssetData AssetData;
			AssetChild->ParameterHandle->GetValue(AssetData);
			int32 PreviewIndex = INDEX_NONE;
			int32 ThumbnailIndex = INDEX_NONE;
			EMaterialParameterAssociation PreviewAssociation = EMaterialParameterAssociation::GlobalParameter;
			if (UObject* AssetObject = AssetData.GetAsset())
			{
				if (Cast<UMaterialFunctionInterface>(AssetObject)->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayer)
				{
					PreviewIndex = StackParameterData->ParameterInfo.Index;
					PreviewAssociation = EMaterialParameterAssociation::LayerParameter;
					Tree->UpdateThumbnailMaterial(PreviewAssociation, PreviewIndex);
					ThumbnailIndex = PreviewIndex;
				}
				if (Cast<UMaterialFunctionInterface>(AssetObject)->GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayerBlend)
				{
					PreviewIndex = StackParameterData->ParameterInfo.Index;
					PreviewAssociation = EMaterialParameterAssociation::BlendParameter;
					Tree->UpdateThumbnailMaterial(PreviewAssociation, PreviewIndex, true);
					ThumbnailIndex = PreviewIndex - 1;
				}
			}
			HeaderRowWidget->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				.MaxWidth(ThumbnailSize)
				[
					SAssignNew(ThumbnailBox, SBox)
					[
						Tree->CreateThumbnailWidget(PreviewAssociation, ThumbnailIndex, ThumbnailSize)
					]
				];
			ThumbnailBox->SetMaxDesiredHeight(ThumbnailSize);
			ThumbnailBox->SetMinDesiredHeight(ThumbnailSize);
			ThumbnailBox->SetMinDesiredWidth(ThumbnailSize);
			ThumbnailBox->SetMaxDesiredWidth(ThumbnailSize);
		}

		if (StackParameterData->ParameterInfo.Index != 0)
		{
			HeaderRowWidget->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5.0f)
				[
					SNew(STextBlock)
					.Text(this, &SMaterialLayersFunctionsMaterialTreeItem::GetLayerName, InArgs._InTree, StackParameterData->ParameterInfo.Index)
				];
				HeaderRowWidget->AddSlot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNullWidget::NullWidget
				];
		}
		else
		{
			HeaderRowWidget->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5.0f)
				[
					SNew(STextBlock)
					.Text(NameOverride)
				];
		}
		LeftSideWidget = HeaderRowWidget;
	}
	// END STACK

	// GROUP --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Group)
	{
		NameOverride = FText::FromName(StackParameterData->Group.GroupName);
		LeftSideWidget = SNew(STextBlock)
			.Text(NameOverride)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.TransformPolicy(ETextTransformPolicy::ToUpper);
	}
	// END GROUP

	// ASSET --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Asset)
	{
		ValueAlignment = HAlign_Fill;
		StackParameterData->ParameterHandle->GetProperty()->SetMetaData(FName(TEXT("DisplayThumbnail")), TEXT("true"));
		FIntPoint ThumbnailOverride;
		if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
		{
			NameOverride = FMaterialPropertyHelpers::LayerID;
			ThumbnailOverride = FIntPoint(64, 64);
		}
		else if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
		{
			NameOverride = FMaterialPropertyHelpers::BlendID;
			ThumbnailOverride = FIntPoint(32, 32);
		}


		const int32 LayerStateIndex = StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter ? StackParameterData->ParameterInfo.Index + 1 : StackParameterData->ParameterInfo.Index;

		IDetailTreeNode& Node = *StackParameterData->ParameterNode;
		FNodeWidgets NodeWidgets = Node.CreateNodeWidgets();

		LeftSideWidget = StackParameterData->ParameterHandle->CreatePropertyNameWidget(NameOverride);

		EMaterialParameterAssociation InAssociation = StackParameterData->ParameterInfo.Association;
		UMaterialFunctionInterface* LocalFunction = nullptr;
		TSharedPtr<SBox> ThumbnailBox;

		if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
		{
			LocalFunction = Tree->FunctionInstance->Layers[StackParameterData->ParameterInfo.Index];
		}
		else if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
		{
			LocalFunction = Tree->FunctionInstance->Blends[StackParameterData->ParameterInfo.Index];
		}

		RightSideWidget = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				.MaxWidth(ThumbnailOverride.X)
				[
					SAssignNew(ThumbnailBox, SBox)
					[
						Tree->CreateThumbnailWidget(StackParameterData->ParameterInfo.Association, StackParameterData->ParameterInfo.Index, ThumbnailOverride.X)
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMaterialFunctionInterface::StaticClass())
					.ObjectPath(this, &SMaterialLayersFunctionsMaterialTreeItem::GetInstancePath, Tree)
					.DisplayCompactSize(true)
				]
			];
		ThumbnailBox->SetMaxDesiredHeight(ThumbnailOverride.Y);
		ThumbnailBox->SetMinDesiredHeight(ThumbnailOverride.Y);
		ThumbnailBox->SetMinDesiredWidth(ThumbnailOverride.X);
		ThumbnailBox->SetMaxDesiredWidth(ThumbnailOverride.X);
	}
	// END ASSET

	// PROPERTY ----------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Property)
	{
		NameOverride = FText::FromName(StackParameterData->Parameter->ParameterInfo.Name);

		IDetailTreeNode& Node = *StackParameterData->ParameterNode;
		TSharedPtr<IPropertyHandle> PropertyHandle = Node.CreatePropertyHandle();
		TSharedPtr<IDetailPropertyRow> GeneratedRow = StaticCastSharedPtr<IDetailPropertyRow>(Node.GetRow());
		IDetailPropertyRow& Row = *GeneratedRow.Get();
		Row
			.DisplayName(NameOverride);

		FMaterialPropertyHelpers::SetPropertyRowParameterWidget(Row, StackParameterData->Parameter, PropertyHandle, MaterialEditorInstance);

		FNodeWidgets NodeWidgets = Node.CreateNodeWidgets();
		LeftSideWidget = NodeWidgets.NameWidget.ToSharedRef();
		RightSideWidget = NodeWidgets.ValueWidget.ToSharedRef();
	}
	// END PROPERTY

	// PROPERTY CHILD ----------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::PropertyChild)
	{
		FNodeWidgets NodeWidgets = StackParameterData->ParameterNode->CreateNodeWidgets();
		LeftSideWidget = NodeWidgets.NameWidget.ToSharedRef();
		RightSideWidget = NodeWidgets.ValueWidget.ToSharedRef();
	}
	// END PROPERTY CHILD

	// FINAL WRAPPER

	LeftSideWidget->SetEnabled(false);
	RightSideWidget->SetEnabled(false);
	if (StackParameterData->StackDataType == EStackDataType::Stack)
	{
		TSharedPtr<SHorizontalBox> FinalStack;
		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(this, &SMaterialLayersFunctionsMaterialTreeItem::GetBorderImage)
				.BorderBackgroundColor(this, &SMaterialLayersFunctionsMaterialTreeItem::GetOuterBackgroundColor, StackParameterData)
				.Padding(0.0f)
				[
					SAssignNew(FinalStack, SHorizontalBox)
				]
			];
		FinalStack->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0f))
			[
				SNew(SExpanderArrow, SharedThis(this))
			];
		FinalStack->AddSlot()
			.Padding(FMargin(2.0f))
			.VAlign(VAlign_Center)
			[
				LeftSideWidget
			];
	}
	else if (StackParameterData->StackDataType == EStackDataType::Group)
	{
		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(this, &SMaterialLayersFunctionsMaterialTreeItem::GetBorderImage)
					.BorderBackgroundColor(this, &SMaterialLayersFunctionsMaterialTreeItem::GetOuterBackgroundColor, StackParameterData)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(3.0f))
						[
							SNew(SExpanderArrow, SharedThis(this))
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(2.0f))
						.VAlign(VAlign_Center)
						[
							LeftSideWidget
						]
					]
				]
			];
	}
	else
	{
		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBox)
					.MinDesiredHeight(26.0f)	// Match PropertyEditorConstants::PropertyRowHeight
					[
						SNew(SBorder)
						.Padding(0.0f)
						.BorderImage(this, &SMaterialLayersFunctionsMaterialTreeItem::GetBorderImage)
						.BorderBackgroundColor(this, &SMaterialLayersFunctionsMaterialTreeItem::GetOuterBackgroundColor, StackParameterData)
						[
							SNew(SSplitter)
							.Style(FAppStyle::Get(), "DetailsView.Splitter")
							.PhysicalSplitterHandleSize(1.0f)
							.HitDetectionSplitterHandleSize(5.0f)
							+ SSplitter::Slot()
							.Value(Tree->ColumnSizeData.GetNameColumnWidth())
							.OnSlotResized(Tree->ColumnSizeData.GetOnNameColumnResized())
							.Value(0.25f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(FMargin(3.0f))
								[
									SNew(SExpanderArrow, SharedThis(this))
								]
								+ SHorizontalBox::Slot()
								.Padding(FMargin(2.0f))
								.VAlign(VAlign_Center)
								[
									LeftSideWidget
								]
							]
							+ SSplitter::Slot()
							.Value(Tree->ColumnSizeData.GetValueColumnWidth())
							.OnSlotResized(Tree->ColumnSizeData.GetOnValueColumnResized())
							[
								SNew(SHorizontalBox)
								.Clipping(EWidgetClipping::OnDemand)
								+ SHorizontalBox::Slot()
								.HAlign(ValueAlignment)
								.VAlign(VAlign_Center)
								.Padding(FMargin(12.0f, 0.0f, 2.0f, 0.0f))	// Match DetailWidgetConstants::RightRowPadding
								[
									RightSideWidget
								]
							]
						]
					]
				]
			];
	}


	this->ChildSlot
		[
			WrapperWidget
		];


	STableRow< TSharedPtr<FSortedParamData> >::ConstructInternal(
		STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false),
		InOwnerTableView
	);
}

FString SMaterialLayersFunctionsMaterialTreeItem::GetInstancePath(SMaterialLayersFunctionsMaterialTree* InTree) const
{
	FString InstancePath;
	if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter && InTree->FunctionInstance->Blends.IsValidIndex(StackParameterData->ParameterInfo.Index))
	{
		InstancePath = InTree->FunctionInstance->Blends[StackParameterData->ParameterInfo.Index]->GetPathName();
	}
	else if (StackParameterData->ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter && InTree->FunctionInstance->Layers.IsValidIndex(StackParameterData->ParameterInfo.Index))
	{
		InstancePath = InTree->FunctionInstance->Layers[StackParameterData->ParameterInfo.Index]->GetPathName();
	}
	return InstancePath;
}

void SMaterialLayersFunctionsMaterialTree::Construct(const FArguments& InArgs)
{
	ColumnSizeData.SetValueColumnWidth(0.5f);

	MaterialEditorInstance = InArgs._InMaterialEditorInstance;
	Wrapper = InArgs._InWrapper;
	CreateGroupsWidget();

#if WITH_EDITOR
	//Fixup for adding new bool arrays to the class
	if (FunctionInstance)
	{
		if (FunctionInstance->Layers.Num() != FunctionInstance->EditorOnly.RestrictToLayerRelatives.Num())
		{
			int32 OriginalSize = FunctionInstance->EditorOnly.RestrictToLayerRelatives.Num();
			for (int32 LayerIt = 0; LayerIt < FunctionInstance->Layers.Num() - OriginalSize; LayerIt++)
			{
				FunctionInstance->EditorOnly.RestrictToLayerRelatives.Add(false);
			}
		}
		if (FunctionInstance->Blends.Num() != FunctionInstance->EditorOnly.RestrictToBlendRelatives.Num())
		{
			int32 OriginalSize = FunctionInstance->EditorOnly.RestrictToBlendRelatives.Num();
			for (int32 BlendIt = 0; BlendIt < FunctionInstance->Blends.Num() - OriginalSize; BlendIt++)
			{
				FunctionInstance->EditorOnly.RestrictToBlendRelatives.Add(false);
			}
		}
	}
#endif

	STreeView<TSharedPtr<FSortedParamData>>::Construct(
		STreeView::FArguments()
		.TreeItemsSource(&LayerProperties)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SMaterialLayersFunctionsMaterialTree::OnGenerateRowMaterialLayersFunctionsTreeView)
		.OnGetChildren(this, &SMaterialLayersFunctionsMaterialTree::OnGetChildrenMaterialLayersFunctionsTreeView)
		.OnExpansionChanged(this, &SMaterialLayersFunctionsMaterialTree::OnExpansionChanged)
	);

	SetParentsExpansionState();
}

TSharedRef< ITableRow > SMaterialLayersFunctionsMaterialTree::OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SMaterialLayersFunctionsMaterialTreeItem > ReturnRow = SNew(SMaterialLayersFunctionsMaterialTreeItem, OwnerTable)
		.StackParameterData(Item)
		.MaterialEditorInstance(MaterialEditorInstance)
		.InTree(this);
	return ReturnRow;
}

void SMaterialLayersFunctionsMaterialTree::OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> InParent, TArray< TSharedPtr<FSortedParamData> >& OutChildren)
{
	OutChildren = InParent->Children;
}


void SMaterialLayersFunctionsMaterialTree::OnExpansionChanged(TSharedPtr<FSortedParamData> Item, bool bIsExpanded)
{
	bool* ExpansionValue = MaterialEditorInstance->PreviewMaterial->LayerParameterExpansion.Find(Item->NodeKey);
	if (ExpansionValue == nullptr)
	{
		MaterialEditorInstance->PreviewMaterial->LayerParameterExpansion.Add(Item->NodeKey, bIsExpanded);
	}
	else if (*ExpansionValue != bIsExpanded)
	{
		MaterialEditorInstance->PreviewMaterial->LayerParameterExpansion.Emplace(Item->NodeKey, bIsExpanded);
	}
	// Expand any children that are also expanded
	for (auto Child : Item->Children)
	{
		bool* ChildExpansionValue = MaterialEditorInstance->PreviewMaterial->LayerParameterExpansion.Find(Child->NodeKey);
		if (ChildExpansionValue != nullptr && *ChildExpansionValue == true)
		{
			SetItemExpansion(Child, true);
		}
	}
}

void SMaterialLayersFunctionsMaterialTree::SetParentsExpansionState()
{
	for (const auto& Pair : LayerProperties)
	{
		if (Pair->Children.Num())
		{
			bool* bIsExpanded = MaterialEditorInstance->PreviewMaterial->LayerParameterExpansion.Find(Pair->NodeKey);
			if (bIsExpanded)
			{
				SetItemExpansion(Pair, *bIsExpanded);
			}
		}
	}
}

TSharedPtr<class FAssetThumbnailPool> SMaterialLayersFunctionsMaterialTree::GetTreeThumbnailPool()
{
	return UThumbnailManager::Get().GetSharedThumbnailPool();
}

void SMaterialLayersFunctionsMaterialTree::CreateGroupsWidget()
{
	check(MaterialEditorInstance);

	NonLayerProperties.Empty();
	LayerProperties.Empty();
	FunctionParameter = nullptr;

	const TArray<TSharedRef<IDetailTreeNode>> TestData = Wrapper->GetGenerator()->GetRootTreeNodes();

	if (TestData.Num() == 0)
	{
		return;
	}

	TSharedPtr<IDetailTreeNode> Category = TestData[0];
	TSharedPtr<IDetailTreeNode> ParameterGroups;
	TArray<TSharedRef<IDetailTreeNode>> Children;
	Category->GetChildren(Children);

	for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ChildIdx++)
	{
		if (Children[ChildIdx]->CreatePropertyHandle().IsValid() &&
			Children[ChildIdx]->CreatePropertyHandle()->GetProperty()->GetName() == "ParameterGroups")
		{
			ParameterGroups = Children[ChildIdx];
			break;
		}
	}

	Children.Empty();
	ParameterGroups->GetChildren(Children);
	// the order should correspond to NonLayerProperty exactly
	TArray<TSharedPtr<IPropertyHandle>> DeferredSearches;
	for (int32 GroupIdx = 0; GroupIdx < Children.Num(); ++GroupIdx)
	{
		TArray<void*> GroupPtrs;
		TSharedPtr<IPropertyHandle> ChildHandle = Children[GroupIdx]->CreatePropertyHandle();
		ChildHandle->AccessRawData(GroupPtrs);
		auto GroupIt = GroupPtrs.CreateConstIterator();
		const FEditorParameterGroup* ParameterGroupPtr = reinterpret_cast<FEditorParameterGroup*>(*GroupIt);
		const FEditorParameterGroup& ParameterGroup = *ParameterGroupPtr;

		for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ParamIdx++)
		{
			UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];

			TSharedPtr<IPropertyHandle> ParametersArrayProperty = ChildHandle->GetChildHandle("Parameters");
			TSharedPtr<IPropertyHandle> ParameterProperty = ParametersArrayProperty->GetChildHandle(ParamIdx);
			TSharedPtr<IPropertyHandle> ParameterValueProperty = ParameterProperty->GetChildHandle("ParameterValue");

			if (Cast<UDEditorMaterialLayersParameterValue>(Parameter))
			{
				if (FunctionParameter == nullptr)
				{
					FunctionParameter = Parameter;
				}
				TArray<void*> StructPtrs;
				ParameterValueProperty->AccessRawData(StructPtrs);
				auto It = StructPtrs.CreateConstIterator();
				FunctionInstance = reinterpret_cast<FMaterialLayersFunctions*>(*It);
				FunctionInstanceHandle = ParameterValueProperty;

				TSharedPtr<IPropertyHandle>	LayerHandle = ChildHandle->GetChildHandle("Layers").ToSharedRef();
				TSharedPtr<IPropertyHandle> BlendHandle = ChildHandle->GetChildHandle("Blends").ToSharedRef();
				uint32 LayerChildren;
				LayerHandle->GetNumChildren(LayerChildren);
				if (LayerChildren > 0)
				{
					uint32 BlendChildren;
					BlendHandle->GetNumChildren(BlendChildren);
					if (MaterialEditorInstance->StoredLayerPreviews.Num() != LayerChildren)
					{
						MaterialEditorInstance->StoredLayerPreviews.Empty();
						MaterialEditorInstance->StoredLayerPreviews.AddDefaulted(LayerChildren);
					}
					if (MaterialEditorInstance->StoredBlendPreviews.Num() != BlendChildren)
					{
						MaterialEditorInstance->StoredBlendPreviews.Empty();
						MaterialEditorInstance->StoredBlendPreviews.AddDefaulted(BlendChildren);
					}

					TSharedRef<FSortedParamData> StackProperty = MakeShared<FSortedParamData>();
					StackProperty->StackDataType = EStackDataType::Stack;
					StackProperty->Parameter = Parameter;
					StackProperty->ParameterInfo.Index = LayerChildren - 1;
					StackProperty->NodeKey = FString::FromInt(StackProperty->ParameterInfo.Index);


					TSharedRef<FSortedParamData> ChildProperty = MakeShared<FSortedParamData>();
					ChildProperty->StackDataType = EStackDataType::Asset;
					ChildProperty->Parameter = Parameter;
					ChildProperty->ParameterHandle = LayerHandle->AsArray()->GetElement(LayerChildren - 1);
					ChildProperty->ParameterNode = Wrapper->GetGenerator()->FindTreeNode(ChildProperty->ParameterHandle);
					ChildProperty->ParameterInfo.Index = LayerChildren - 1;
					ChildProperty->ParameterInfo.Association = EMaterialParameterAssociation::LayerParameter;
					ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association);

					{
						UObject* AssetObject = nullptr;
						ChildProperty->ParameterHandle->GetValue(AssetObject);
						if (AssetObject)
						{
							if (MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1] == nullptr)
							{
								MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1] = (NewObject<UMaterialInstanceConstant>(MaterialEditorInstance, NAME_None));
							}
							UMaterialInterface* EditedMaterial = Cast<UMaterialFunctionInterface>(AssetObject)->GetPreviewMaterial();
							if (MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1] && MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1]->Parent != EditedMaterial)
							{
								MaterialEditorInstance->StoredLayerPreviews[LayerChildren - 1]->SetParentEditorOnly(EditedMaterial);
							}
						}
					}

					StackProperty->Children.Add(ChildProperty);
					LayerProperties.Add(StackProperty);

					if (BlendChildren > 0 && LayerChildren > BlendChildren)
					{
						for (int32 Counter = BlendChildren - 1; Counter >= 0; Counter--)
						{
							ChildProperty = MakeShared<FSortedParamData>();
							ChildProperty->StackDataType = EStackDataType::Asset;
							ChildProperty->Parameter = Parameter;
							ChildProperty->ParameterHandle = BlendHandle->AsArray()->GetElement(Counter);
							ChildProperty->ParameterNode = Wrapper->GetGenerator()->FindTreeNode(ChildProperty->ParameterHandle);
							ChildProperty->ParameterInfo.Index = Counter;
							ChildProperty->ParameterInfo.Association = EMaterialParameterAssociation::BlendParameter;
							ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association);
							{
								UObject* AssetObject = nullptr;
								ChildProperty->ParameterHandle->GetValue(AssetObject);
								if (AssetObject)
								{
									if (MaterialEditorInstance->StoredBlendPreviews[Counter] == nullptr)
									{
										MaterialEditorInstance->StoredBlendPreviews[Counter] = (NewObject<UMaterialInstanceConstant>(MaterialEditorInstance, NAME_None));
									}
									UMaterialInterface* EditedMaterial = Cast<UMaterialFunctionInterface>(AssetObject)->GetPreviewMaterial();
									if (MaterialEditorInstance->StoredBlendPreviews[Counter] && MaterialEditorInstance->StoredBlendPreviews[Counter]->Parent != EditedMaterial)
									{
										MaterialEditorInstance->StoredBlendPreviews[Counter]->SetParentEditorOnly(EditedMaterial);
									}
								}
							}
							LayerProperties.Last()->Children.Add(ChildProperty);

							StackProperty = MakeShared<FSortedParamData>();
							StackProperty->StackDataType = EStackDataType::Stack;
							StackProperty->Parameter = Parameter;
							StackProperty->ParameterInfo.Index = Counter;
							StackProperty->NodeKey = FString::FromInt(StackProperty->ParameterInfo.Index);
							LayerProperties.Add(StackProperty);

							ChildProperty = MakeShared<FSortedParamData>();
							ChildProperty->StackDataType = EStackDataType::Asset;
							ChildProperty->Parameter = Parameter;
							ChildProperty->ParameterHandle = LayerHandle->AsArray()->GetElement(Counter);
							ChildProperty->ParameterNode = Wrapper->GetGenerator()->FindTreeNode(ChildProperty->ParameterHandle);
							ChildProperty->ParameterInfo.Index = Counter;
							ChildProperty->ParameterInfo.Association = EMaterialParameterAssociation::LayerParameter;
							ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association);
							{
								UObject* AssetObject = nullptr;
								ChildProperty->ParameterHandle->GetValue(AssetObject);
								if (AssetObject)
								{
									if (MaterialEditorInstance->StoredLayerPreviews[Counter] == nullptr)
									{
										MaterialEditorInstance->StoredLayerPreviews[Counter] = (NewObject<UMaterialInstanceConstant>(MaterialEditorInstance, NAME_None));
									}
									UMaterialInterface* EditedMaterial = Cast<UMaterialFunctionInterface>(AssetObject)->GetPreviewMaterial();
									if (MaterialEditorInstance->StoredLayerPreviews[Counter] && MaterialEditorInstance->StoredLayerPreviews[Counter]->Parent != EditedMaterial)
									{
										MaterialEditorInstance->StoredLayerPreviews[Counter]->SetParentEditorOnly(EditedMaterial);
									}
								}
							}
							LayerProperties.Last()->Children.Add(ChildProperty);
						}
					}
				}
			}
			else
			{
				// Skip parameters that shouldn't have widgets here
				if (!FMaterialPropertyHelpers::ShouldCreatePropertyRowForParameter(Parameter))
				{
					continue;
				}

				FUnsortedParamData NonLayerProperty;

				NonLayerProperty.Parameter = Parameter;
				NonLayerProperty.ParameterGroup = ParameterGroup;

				DeferredSearches.Add(ParameterValueProperty);
				NonLayerProperty.UnsortedName = Parameter->ParameterInfo.Name;

				NonLayerProperties.Add(NonLayerProperty);
			}
		}
	}

	checkf(NonLayerProperties.Num() == DeferredSearches.Num(), TEXT("Internal inconsistency: number of node searches does not match the number of properties"));
	TArray<TSharedPtr<IDetailTreeNode>> DeferredResults = Wrapper->GetGenerator()->FindTreeNodes(DeferredSearches);
	checkf(NonLayerProperties.Num() == DeferredResults.Num(), TEXT("Internal inconsistency: number of node search results does not match the number of properties"));

	for (int Idx = 0, NumUnsorted = NonLayerProperties.Num(); Idx < NumUnsorted; ++Idx)
	{
		FUnsortedParamData& NonLayerProperty = NonLayerProperties[Idx];
		NonLayerProperty.ParameterNode = DeferredResults[Idx];
		NonLayerProperty.ParameterHandle = NonLayerProperty.ParameterNode->CreatePropertyHandle();
	}

	DeferredResults.Empty();
	DeferredSearches.Empty();

	for (int32 LayerIdx = 0; LayerIdx < LayerProperties.Num(); LayerIdx++)
	{
		for (int32 ChildIdx = 0; ChildIdx < LayerProperties[LayerIdx]->Children.Num(); ChildIdx++)
		{
			ShowSubParameters(LayerProperties[LayerIdx]->Children[ChildIdx]);
		}
	}

	SetParentsExpansionState();
}


TSharedRef<SWidget> SMaterialLayersFunctionsMaterialTree::CreateThumbnailWidget(EMaterialParameterAssociation InAssociation, int32 InIndex, float InThumbnailSize)
{
	UObject* ThumbnailObject = nullptr;
	if (InAssociation == EMaterialParameterAssociation::LayerParameter)
	{
		ThumbnailObject = MaterialEditorInstance->StoredLayerPreviews[InIndex];
	}
	else if (InAssociation == EMaterialParameterAssociation::BlendParameter)
	{
		ThumbnailObject = MaterialEditorInstance->StoredBlendPreviews[InIndex];
	}
	const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable(new FAssetThumbnail(ThumbnailObject, InThumbnailSize, InThumbnailSize, GetTreeThumbnailPool()));
	TSharedRef<SWidget> ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();
	return ThumbnailWidget;
}

void SMaterialLayersFunctionsMaterialTree::UpdateThumbnailMaterial(TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 InIndex, bool bAlterBlendIndex)
{
	// Need to invert index b/c layer properties is generated in reverse order
	TArray<TSharedPtr<FSortedParamData>> AssetChildren = LayerProperties[LayerProperties.Num() - 1 - InIndex]->Children;
	UMaterialInstanceConstant* MaterialToUpdate = nullptr;
	int32 ParameterIndex = InIndex;
	if (InAssociation == EMaterialParameterAssociation::LayerParameter)
	{
		MaterialToUpdate = MaterialEditorInstance->StoredLayerPreviews[ParameterIndex];
	}
	if (InAssociation == EMaterialParameterAssociation::BlendParameter)
	{
		if (bAlterBlendIndex)
		{
			ParameterIndex--;
		}
		MaterialToUpdate = MaterialEditorInstance->StoredBlendPreviews[ParameterIndex];
	}

	TArray<FEditorParameterGroup> ParameterGroups;
	for (TSharedPtr<FSortedParamData> AssetChild : AssetChildren)
	{
		for (TSharedPtr<FSortedParamData> Group : AssetChild->Children)
		{
			if (Group->ParameterInfo.Association == InAssociation)
			{
				FEditorParameterGroup DuplicatedGroup = FEditorParameterGroup();
				DuplicatedGroup.GroupAssociation = Group->Group.GroupAssociation;
				DuplicatedGroup.GroupName = Group->Group.GroupName;
				DuplicatedGroup.GroupSortPriority = Group->Group.GroupSortPriority;
				for (UDEditorParameterValue* Parameter : Group->Group.Parameters)
				{
					if (Parameter->ParameterInfo.Index == ParameterIndex)
					{
						DuplicatedGroup.Parameters.Add(Parameter);
					}
				}
				ParameterGroups.Add(DuplicatedGroup);
			}
		}



	}
	if (MaterialToUpdate != nullptr)
	{
		FMaterialPropertyHelpers::TransitionAndCopyParameters(MaterialToUpdate, ParameterGroups, true);
	}
}

void SMaterialLayersFunctionsMaterialTree::ShowSubParameters(TSharedPtr<FSortedParamData> ParentParameter)
{
	for (FUnsortedParamData Property : NonLayerProperties)
	{
		UDEditorParameterValue* Parameter = Property.Parameter;
		if (Parameter->ParameterInfo.Index == ParentParameter->ParameterInfo.Index
			&& Parameter->ParameterInfo.Association == ParentParameter->ParameterInfo.Association)
		{
			FString GroupNodeKey = FString::FromInt(Parameter->ParameterInfo.Index) + FString::FromInt(Parameter->ParameterInfo.Association) + Property.ParameterGroup.GroupName.ToString();

			bool bAddNewGroup = true;
			for (TSharedPtr<struct FSortedParamData> GroupChild : ParentParameter->Children)
			{
				if (GroupChild->NodeKey == GroupNodeKey)
				{
					bAddNewGroup = false;
				}
			}
			if (bAddNewGroup)
			{
				TSharedPtr<FSortedParamData> GroupProperty(new FSortedParamData());
				GroupProperty->StackDataType = EStackDataType::Group;
				GroupProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
				GroupProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
				GroupProperty->Group = Property.ParameterGroup;
				GroupProperty->NodeKey = GroupNodeKey;

				ParentParameter->Children.Add(GroupProperty);
			}

			TSharedPtr<FSortedParamData> ChildProperty(new FSortedParamData());
			ChildProperty->StackDataType = EStackDataType::Property;
			ChildProperty->Parameter = Parameter;
			ChildProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
			ChildProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
			ChildProperty->ParameterNode = Property.ParameterNode;
			ChildProperty->PropertyName = Property.UnsortedName;
			ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association) + Property.ParameterGroup.GroupName.ToString() + Property.UnsortedName.ToString();


			if (FMaterialPropertyHelpers::ShouldCreateChildPropertiesForParameter(Parameter))
			{
				TArray<TSharedRef<IDetailTreeNode>> ParamChildren;
				Property.ParameterNode->GetChildren(ParamChildren);
				for (int32 ParamChildIdx = 0; ParamChildIdx < ParamChildren.Num(); ParamChildIdx++)
				{
					TSharedPtr<FSortedParamData> ParamChildProperty(new FSortedParamData());
					ParamChildProperty->StackDataType = EStackDataType::PropertyChild;
					ParamChildProperty->ParameterNode = ParamChildren[ParamChildIdx];
					ParamChildProperty->ParameterHandle = ParamChildProperty->ParameterNode->CreatePropertyHandle();
					ParamChildProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
					ParamChildProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
					ParamChildProperty->Parameter = ChildProperty->Parameter;
					ChildProperty->Children.Add(ParamChildProperty);
				}
			}
			for (TSharedPtr<struct FSortedParamData> GroupChild : ParentParameter->Children)
			{
				if (GroupChild->Group.GroupName == Property.ParameterGroup.GroupName
					&& GroupChild->ParameterInfo.Association == ChildProperty->ParameterInfo.Association
					&& GroupChild->ParameterInfo.Index == ChildProperty->ParameterInfo.Index)
				{
					GroupChild->Children.Add(ChildProperty);
				}
			}

		}
	}
}

void SMaterialLayersFunctionsMaterialWrapper::Refresh()
{
	LayerParameter = nullptr;
	TSharedPtr<SHorizontalBox> HeaderBox;
	NestedTree->CreateGroupsWidget();
	LayerParameter = NestedTree->FunctionParameter;

	if (LayerParameter != nullptr)
	{
		this->ChildSlot
			[
				
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(HeaderBox, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(3.0f, 1.0f))
					.HAlign(HAlign_Left)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MaterialLayers", "Material Layers"))
					]
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f))
				[
					NestedTree.ToSharedRef()
				]
			];
		HeaderBox->AddSlot()
			.FillWidth(1.0f)
			[
				SNullWidget::NullWidget
			];
	}
	else
	{
		this->ChildSlot
			[
				SNew(SBorder)
				.Padding(FMargin(10.0f))
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddLayerParameterPrompt", "Add a Material Attribute Layers parameter to see it here."))
				]
			];
	}
}


void SMaterialLayersFunctionsMaterialWrapper::Construct(const FArguments& InArgs)
{
	TSharedPtr<IPropertyRowGenerator> InGenerator = InArgs._InGenerator;
	Generator = InGenerator;


	NestedTree = SNew(SMaterialLayersFunctionsMaterialTree)
		.InMaterialEditorInstance(InArgs._InMaterialEditorInstance)
		.InWrapper(this);


	LayerParameter = NestedTree->FunctionParameter;

	MaterialEditorInstance = InArgs._InMaterialEditorInstance;
	Refresh();

}

void SMaterialLayersFunctionsMaterialWrapper::SetEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance)
{
	NestedTree->MaterialEditorInstance = dynamic_cast<UMaterialEditorPreviewParameters*>(InMaterialEditorInstance);
	Refresh();
}


TSharedPtr<class IPropertyRowGenerator> SMaterialLayersFunctionsMaterialWrapper::GetGenerator()
{
	return Generator.Pin();
}

#undef LOCTEXT_NAMESPACE