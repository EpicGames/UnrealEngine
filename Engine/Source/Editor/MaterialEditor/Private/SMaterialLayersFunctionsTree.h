// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailColumnSizeData.h"
#include "Engine/EngineTypes.h"
#include "IDetailPropertyRow.h"
#include "IDetailTreeNode.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "MaterialPropertyHelpers.h"
#include "Materials/Material.h"
#include "PropertyCustomizationHelpers.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Materials/MaterialExpressionMaterialSample.h"

class IPropertyHandle;
class SMaterialLayersFunctionsInstanceTree;
class UDEditorParameterValue;
class UMaterialEditorInstanceConstant;
class FNotifyHook;

class SMaterialLayersFunctionsInstanceTreeItem : public STableRow< TSharedPtr<FSortedParamData> >, public IDraggableItem
{
public:

	SLATE_BEGIN_ARGS(SMaterialLayersFunctionsInstanceTreeItem)
		: _StackParameterData(nullptr),
		_MaterialEditorInstance(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<FSortedParamData>, StackParameterData)
	SLATE_ARGUMENT(UMaterialEditorInstanceConstant*, MaterialEditorInstance)
	SLATE_ARGUMENT(SMaterialLayersFunctionsInstanceTree*, InTree)
	SLATE_END_ARGS()

	bool bIsBeingDragged;

private:
	bool bIsHoveredDragTarget;


	const FSlateBrush* GetBorderImage() const;

	FSlateColor GetOuterBackgroundColor(TSharedPtr<FSortedParamData> InParamData) const;
public:

	void RefreshOnRowChange(const FAssetData& AssetData, SMaterialLayersFunctionsInstanceTree* InTree);
	bool GetFilterState(SMaterialLayersFunctionsInstanceTree* InTree, TSharedPtr<FSortedParamData> InStackData) const;
	void FilterClicked(const ECheckBoxState NewCheckedState, SMaterialLayersFunctionsInstanceTree* InTree, TSharedPtr<FSortedParamData> InStackData);
	ECheckBoxState GetFilterChecked(SMaterialLayersFunctionsInstanceTree* InTree, TSharedPtr<FSortedParamData> InStackData) const;
	FText GetLayerName(SMaterialLayersFunctionsInstanceTree* InTree, int32 Counter) const;
	void OnNameChanged(const FText& InText, ETextCommit::Type CommitInfo, SMaterialLayersFunctionsInstanceTree* InTree, int32 Counter);


	void OnLayerDragEnter(const FDragDropEvent& DragDropEvent) override
	{
		if (StackParameterData->ParameterInfo.Index != 0)
		{
			bIsHoveredDragTarget = true;
		}
	}

	void OnLayerDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		bIsHoveredDragTarget = false;
	}

	void OnLayerDragDetected() override
	{
		bIsBeingDragged = true;
	}

	FReply OnLayerDrop(const FDragDropEvent& DragDropEvent);
	void OnOverrideParameter(bool NewValue, class UDEditorParameterValue* Parameter);
	void OnOverrideParameter(bool NewValue, TObjectPtr<UDEditorParameterValue> Parameter);
	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	/** The node info to build the tree view row from. */
	TSharedPtr<FSortedParamData> StackParameterData;

	SMaterialLayersFunctionsInstanceTree* Tree;

	UMaterialEditorInstanceConstant* MaterialEditorInstance;

	FString GetInstancePath(SMaterialLayersFunctionsInstanceTree* InTree) const;
};

class SMaterialLayersFunctionsWrapper: public SCompoundWidget
{
public:
	virtual void SetEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance) = 0;
	virtual void Refresh() = 0;
	
};

class SMaterialLayersFunctionsInstanceWrapper : public SMaterialLayersFunctionsWrapper
{
public:
	SLATE_BEGIN_ARGS(SMaterialLayersFunctionsInstanceWrapper)
		: _InMaterialEditorInstance(nullptr),
		_InGenerator(nullptr),
		_InShowHiddenDelegate(),
		_InNotifyHook(nullptr)
	{}

	SLATE_ARGUMENT(UMaterialEditorParameters*, InMaterialEditorInstance)
	SLATE_ARGUMENT(TSharedPtr<class IPropertyRowGenerator>, InGenerator)
	SLATE_ARGUMENT(FGetShowHiddenParameters, InShowHiddenDelegate)
	SLATE_ARGUMENT(FNotifyHook*, InNotifyHook)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	virtual void SetEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance) override;
	virtual void Refresh() override;
	bool HasLayers() const;

	TSharedPtr<class IPropertyRowGenerator> GetGenerator();
	TAttribute<ECheckBoxState> IsParamChecked;
	TWeakObjectPtr<class UDEditorParameterValue> LayerParameter;
	class UMaterialEditorParameters* MaterialEditorInstance;

	TSharedPtr<class SMaterialLayersTree> NestedTree;
	FSimpleDelegate OnLayerPropertyChanged;
	
	
private:
	TWeakPtr<class IPropertyRowGenerator> Generator;
};

class SMaterialLayersTree: public STreeView<TSharedPtr<FSortedParamData>>
{
public:
	virtual void CreateGroupsWidget() = 0;
	virtual TWeakObjectPtr<class UDEditorParameterValue> GetFunctionParameter() = 0;
	virtual TSharedPtr<class FAssetThumbnailPool> GetTreeThumbnailPool() = 0;
	virtual EVisibility GetRelinkLayersToParentVisibility() const = 0;
	virtual FReply RelinkLayersToParent() = 0;
	virtual void SetMaterialEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance) = 0;
	virtual void UpdateThumbnailMaterial(TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 InIndex, bool bAlterBlendIndex = false) = 0;
	virtual void CollectAssetStackItemsRecursively(TSharedPtr<FSortedParamData> Item, TArray<TSharedPtr<FSortedParamData>>& OutGroupsContainer, TArray<uint32>& OutNodeIdsContainer) {}
	virtual void AddRootNodeLayer() {}
};
class SMaterialLayersFunctionsInstanceTree : public SMaterialLayersTree
{
	friend class SMaterialLayersFunctionsInstanceTreeItem;
public:
	SLATE_BEGIN_ARGS(SMaterialLayersFunctionsInstanceTree)
		: _InMaterialEditorInstance(nullptr)
		, _InNotifyHook(nullptr)
	{}

	SLATE_ARGUMENT(UMaterialEditorInstanceConstant*, InMaterialEditorInstance)
	SLATE_ARGUMENT(SMaterialLayersFunctionsInstanceWrapper*, InWrapper)
	SLATE_ARGUMENT(FGetShowHiddenParameters, InShowHiddenDelegate)
	SLATE_ARGUMENT(FNotifyHook*, InNotifyHook)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> InParent, TArray< TSharedPtr<FSortedParamData> >& OutChildren);
	void OnExpansionChanged(TSharedPtr<FSortedParamData> Item, bool bIsExpanded);
	void SetParentsExpansionState();

	void ShowHiddenValues(bool& bShowHiddenParameters) { bShowHiddenParameters = true; }
	TWeakObjectPtr<class UDEditorParameterValue> FunctionParameter;
	struct FMaterialLayersFunctions* FunctionInstance;
	TSharedPtr<IPropertyHandle> FunctionInstanceHandle;
	void RefreshOnAssetChange(const struct FAssetData& InAssetData, int32 Index, EMaterialParameterAssociation MaterialType);
	void ResetAssetToDefault(TSharedPtr<FSortedParamData> InData);
	void AddLayer();
	virtual void AddRootNodeLayer() override { AddLayer(); }
	void RemoveLayer(int32 Index);
	FReply UnlinkLayer(int32 Index);
	virtual FReply RelinkLayersToParent() override;
	EVisibility GetUnlinkLayerVisibility(int32 Index) const;
	virtual EVisibility GetRelinkLayersToParentVisibility() const override;
	virtual void SetMaterialEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance) override { MaterialEditorInstance = Cast<UMaterialEditorInstanceConstant>(InMaterialEditorInstance); } 
	FReply ToggleLayerVisibility(int32 Index);
	virtual TSharedPtr<class FAssetThumbnailPool> GetTreeThumbnailPool() override;

	/** Object that stores all of the possible parameters we can edit */
	UMaterialEditorInstanceConstant* MaterialEditorInstance;

	/** Builds the custom parameter groups category */
	virtual void CreateGroupsWidget() override;
	virtual TWeakObjectPtr<class UDEditorParameterValue> GetFunctionParameter() override{ return FunctionParameter; }
	
	bool IsLayerVisible(int32 Index) const;

	SMaterialLayersFunctionsInstanceWrapper* GetWrapper() { return Wrapper; }



	TSharedRef<SWidget> CreateThumbnailWidget(EMaterialParameterAssociation InAssociation, int32 InIndex, float InThumbnailSize);
	virtual void UpdateThumbnailMaterial(TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 InIndex, bool bAlterBlendIndex = false) override;
	FReply OnThumbnailDoubleClick(const FGeometry& Geometry, const FPointerEvent& MouseEvent, EMaterialParameterAssociation InAssociation, int32 InIndex);
	bool IsOverriddenExpression(class UDEditorParameterValue* Parameter, int32 InIndex);
	bool IsOverriddenExpression(TObjectPtr<UDEditorParameterValue> Parameter, int32 InIndex);

	FGetShowHiddenParameters GetShowHiddenDelegate() const;
protected:

	void ShowSubParameters(TSharedPtr<FSortedParamData> ParentParameter);

private:
	TArray<TSharedPtr<FSortedParamData>> LayerProperties;

	TArray<FUnsortedParamData> NonLayerProperties;

	FDetailColumnSizeData ColumnSizeData;

	SMaterialLayersFunctionsInstanceWrapper* Wrapper;

	TSharedPtr<class IPropertyRowGenerator> Generator;

	bool bLayerIsolated;

	/** Delegate to call to determine if hidden parameters should be shown */
	FGetShowHiddenParameters ShowHiddenDelegate;

	FNotifyHook* NotifyHook = nullptr;
};

class UMaterialEditorPreviewParameters;
class SMaterialLayersFunctionsMaterialTree;
//////////////////////////////// Material version

class SMaterialLayersFunctionsMaterialTreeItem : public STableRow< TSharedPtr<FSortedParamData> >
{
public:

	SLATE_BEGIN_ARGS(SMaterialLayersFunctionsMaterialTreeItem)
		: _StackParameterData(nullptr),
		_MaterialEditorInstance(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<FSortedParamData>, StackParameterData)
	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, MaterialEditorInstance)
	SLATE_ARGUMENT(SMaterialLayersFunctionsMaterialTree*, InTree)
	SLATE_END_ARGS()

	FDetailColumnSizeData ColumnSizeData;
	bool bIsBeingDragged;

private:
	bool bIsHoveredDragTarget;
	const FSlateBrush* GetBorderImage() const;

	FSlateColor GetOuterBackgroundColor(TSharedPtr<FSortedParamData> InParamData) const;
public:

	void RefreshOnRowChange(const FAssetData& AssetData, SMaterialLayersFunctionsMaterialTree* InTree);
	FText GetLayerName(SMaterialLayersFunctionsMaterialTree* InTree, int32 Counter) const;

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	/** The node info to build the tree view row from. */
	TSharedPtr<FSortedParamData> StackParameterData;

	SMaterialLayersFunctionsMaterialTree* Tree;

	UMaterialEditorPreviewParameters* MaterialEditorInstance;

	FString GetInstancePath(SMaterialLayersFunctionsMaterialTree* InTree) const;
};

class SMaterialLayersFunctionsMaterialWrapper : public SMaterialLayersFunctionsWrapper
{
public:
	SLATE_BEGIN_ARGS(SMaterialLayersFunctionsMaterialWrapper)
		: _InMaterialEditorInstance(nullptr)
		, _InGenerator()
	{}

	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, InMaterialEditorInstance)
	SLATE_ARGUMENT(TSharedPtr<class IPropertyRowGenerator>, InGenerator)
	SLATE_END_ARGS()

	virtual void Refresh() override;
	void Construct(const FArguments& InArgs);
	virtual void SetEditorInstance(UMaterialEditorParameters* InMaterialEditorInstance) override;
	TSharedPtr<class IPropertyRowGenerator> GetGenerator();
	TWeakObjectPtr<class UDEditorParameterValue> LayerParameter;
	class UMaterialEditorPreviewParameters* MaterialEditorInstance;
	TSharedPtr<class SMaterialLayersFunctionsMaterialTree> NestedTree;

private:
	TWeakPtr<class IPropertyRowGenerator> Generator;
};

class SMaterialLayersFunctionsMaterialTree : public STreeView<TSharedPtr<FSortedParamData>>
{
	friend class SMaterialLayersFunctionsMaterialTreeItem;
public:
	SLATE_BEGIN_ARGS(SMaterialLayersFunctionsMaterialTree)
		: _InMaterialEditorInstance(nullptr)
	{}

	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, InMaterialEditorInstance)
		SLATE_ARGUMENT(SMaterialLayersFunctionsMaterialWrapper*, InWrapper)
		SLATE_END_ARGS()

		/** Constructs this widget with InArgs */
		void Construct(const FArguments& InArgs);
		TSharedRef< ITableRow > OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable);
		void OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> InParent, TArray< TSharedPtr<FSortedParamData> >& OutChildren);
		void OnExpansionChanged(TSharedPtr<FSortedParamData> Item, bool bIsExpanded);
		void SetParentsExpansionState();
		void ShowHiddenValues(bool& bShowHiddenParameters) { bShowHiddenParameters = true; }
		TWeakObjectPtr<class UDEditorParameterValue> FunctionParameter;
		struct FMaterialLayersFunctions* FunctionInstance;
		TSharedPtr<IPropertyHandle> FunctionInstanceHandle;
		TSharedPtr<class FAssetThumbnailPool> GetTreeThumbnailPool();

		/** Object that stores all of the possible parameters we can edit */
		UMaterialEditorPreviewParameters* MaterialEditorInstance;

		/** Builds the custom parameter groups category */
		void CreateGroupsWidget();

		SMaterialLayersFunctionsMaterialWrapper* GetWrapper() { return Wrapper; }

		TSharedRef<SWidget> CreateThumbnailWidget(EMaterialParameterAssociation InAssociation, int32 InIndex, float InThumbnailSize);
		void UpdateThumbnailMaterial(TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 InIndex, bool bAlterBlendIndex = false);
protected:

	void ShowSubParameters(TSharedPtr<FSortedParamData> ParentParameter);

private:
	TArray<TSharedPtr<FSortedParamData>> LayerProperties;

	TArray<FUnsortedParamData> NonLayerProperties;

	FDetailColumnSizeData ColumnSizeData;

	SMaterialLayersFunctionsMaterialWrapper* Wrapper;
};
