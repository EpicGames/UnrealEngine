// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialNamedReroutesPanel.h"
#include "MaterialEditor.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Colors/SColorBlock.h"

#define LOCTEXT_NAMESPACE "MaterialNamedReroutesPanel"

//////////////////////////////////////////////////////////////////////////
// SMaterialNamedRerouteItem

void SMaterialNamedRerouteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData->Action.IsValid());

	ActionPtr = InCreateData->Action;

	// Try to get the node color from the declaration
	FLinearColor SwatchColor = FLinearColor::White;
	if (InCreateData->Action->GetTypeId() == FMaterialGraphSchemaAction_NewNamedRerouteUsage::StaticGetTypeId())
	{
		const TSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage> RerouteAction = StaticCastSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage>(InCreateData->Action);
		if (RerouteAction->Declaration)
		{
			SwatchColor = RerouteAction->Declaration->NodeColor;
		}
	}

	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, false);

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		// Color swatch
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2, 0)
		[
			SNew(SColorBlock)
			.Color(SwatchColor)
			.Size(FVector2D(12.0f, 12.0f))
			.CornerRadius(FVector4(2.0f, 2.0f, 2.0f, 2.0f))
		]
		// Name slot
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(3, 0)
		[
			NameSlotWidget
		]
	];
}

FText SMaterialNamedRerouteItem::GetItemTooltip() const
{
	return ActionPtr.IsValid() ? ActionPtr.Pin()->GetTooltipDescription() : FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////
// SMaterialNamedReroutesPanel

void SMaterialNamedReroutesPanel::Construct(const FArguments& InArgs, TWeakPtr<FMaterialEditor> InMaterialEditorPtr)
{
	MaterialEditorPtr = InMaterialEditorPtr;

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(2.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionDragged(this, &SMaterialNamedReroutesPanel::OnActionDragged)
				.OnCreateWidgetForAction(this, &SMaterialNamedReroutesPanel::OnCreateWidgetForAction)
				.OnCollectAllActions(this, &SMaterialNamedReroutesPanel::CollectAllActions)
				.OnActionDoubleClicked(this, &SMaterialNamedReroutesPanel::OnActionDoubleClicked)
				.OnContextMenuOpening(this, &SMaterialNamedReroutesPanel::OnContextMenuOpening)
				.AutoExpandActionMenu(true)
			]
		]
	];
}

TSharedRef<SWidget> SMaterialNamedReroutesPanel::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SMaterialNamedRerouteItem, InCreateData);
}

void SMaterialNamedReroutesPanel::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	TSharedPtr<FMaterialEditor> MaterialEditor = MaterialEditorPtr.Pin();
	if (!MaterialEditor.IsValid())
	{
		return;
	}

	UMaterial* Material = MaterialEditor->Material;
	if (!Material || !Material->MaterialGraph)
	{
		return;
	}

	UMaterialGraphSchema::GetNamedRerouteActionsForGraph(OutAllActions, Material->MaterialGraph);
}

void SMaterialNamedReroutesPanel::OnActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions)
{
	TSharedPtr<FMaterialEditor> MaterialEditor = MaterialEditorPtr.Pin();
	if (!MaterialEditor.IsValid())
	{
		return;
	}

	for (const TSharedPtr<FEdGraphSchemaAction>& Action : InActions)
	{
		if (Action.IsValid() && Action->GetTypeId() == FMaterialGraphSchemaAction_NewNamedRerouteUsage::StaticGetTypeId())
		{
			const TSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage> RerouteAction = StaticCastSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage>(Action);
			if (RerouteAction->Declaration && RerouteAction->Declaration->GraphNode)
			{
				MaterialEditor->JumpToNode(RerouteAction->Declaration->GraphNode);
			}
		}
	}
}

void SMaterialNamedReroutesPanel::RequestRefresh()
{
	RefreshActionsList(true);
}

TSharedPtr<SWidget> SMaterialNamedReroutesPanel::OnContextMenuOpening()
{
	TSharedPtr<FMaterialEditor> MaterialEditor = MaterialEditorPtr.Pin();
	if (!MaterialEditor.IsValid())
	{
		return nullptr;
	}

	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	if (SelectedActions.Num() != 1 || !SelectedActions[0].IsValid())
	{
		return nullptr;
	}

	if (SelectedActions[0]->GetTypeId() != FMaterialGraphSchemaAction_NewNamedRerouteUsage::StaticGetTypeId())
	{
		return nullptr;
	}

	TSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage> RerouteAction = StaticCastSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage>(SelectedActions[0]);

	if (!RerouteAction->Declaration)
	{
		return nullptr;
	}

	TWeakObjectPtr<UMaterialExpressionNamedRerouteDeclaration> WeakDeclaration = RerouteAction->Declaration;
	TWeakPtr<FMaterialEditor> WeakEditor = MaterialEditorPtr;

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/ true, /*InCommandList=*/ nullptr);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SelectNamedRerouteUsages", "Select Named Reroute Usages"),
		LOCTEXT("SelectNamedRerouteUsagesTooltip", "Find all usage nodes of this named reroute and show them in Find Results"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([WeakEditor, WeakDeclaration]()
		{
			if (TSharedPtr<FMaterialEditor> Editor = WeakEditor.Pin())
			{
				if (UMaterialExpressionNamedRerouteDeclaration* Declaration = WeakDeclaration.Get())
				{
					Editor->SelectNamedRerouteUsages(Declaration);
				}
			}
		}))
	);

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
