// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphPalette.h"

class FMaterialEditor;

/** Widget for displaying a single named reroute item with a color swatch */
class SMaterialNamedRerouteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SMaterialNamedRerouteItem) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

private:
	virtual FText GetItemTooltip() const override;
};

//////////////////////////////////////////////////////////////////////////

/** Panel listing all Named Reroute Declaration nodes in the current material */
class SMaterialNamedReroutesPanel : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS(SMaterialNamedReroutesPanel) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FMaterialEditor> InMaterialEditorPtr);

	/** Force a full refresh of the list (e.g. when declarations are added/removed or properties change) */
	void RequestRefresh();

protected:
	// SGraphPalette Interface
	virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	// End of SGraphPalette Interface

	/** Called when an action is selected in the list - navigates to the declaration node */
	void OnActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions);

private:
	/** Creates the right-click context menu for panel items */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Pointer back to the material editor that owns us */
	TWeakPtr<FMaterialEditor> MaterialEditorPtr;
};
