// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialPalette.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SOverlay.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Materials/MaterialFunction.h"
#include "MaterialGraph/MaterialGraphSchema.h"

#include "Materials/MaterialExpressionComment.h"

#include "EditorWidgetsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MaterialEditor.h"
#include "MaterialEditorActions.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "MaterialPalette"

void SMaterialPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData->Action.IsValid());

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;

	// Get the Hotkey chord if one exists for this action
	TSharedPtr<const FInputChord> HotkeyChord;

	if ( FMaterialEditorSpawnNodeCommands::IsRegistered() )
	{
		if ( GraphAction->GetTypeId() == FMaterialGraphSchemaAction_NewNode::StaticGetTypeId() )
		{
			UClass* MaterialExpressionClass = StaticCastSharedPtr<FMaterialGraphSchemaAction_NewNode>(GraphAction)->MaterialExpressionClass;
			HotkeyChord = FMaterialEditorSpawnNodeCommands::Get().GetChordByClass(MaterialExpressionClass);
		}
		else if ( GraphAction->GetTypeId() == FMaterialGraphSchemaAction_NewComment::StaticGetTypeId() )
		{
			HotkeyChord = FMaterialEditorSpawnNodeCommands::Get().GetChordByClass(UMaterialExpressionComment::StaticClass());
		}
	}

	// Find icons
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor IconColor = FSlateColor::UseForeground();
	FText IconToolTip = GraphAction->GetTooltipDescription();
	bool bIsReadOnly = false;

	TSharedRef<SWidget> IconWidget = CreateIconWidget( IconToolTip, IconBrush, IconColor );
	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, bIsReadOnly );
	TSharedRef<SWidget> HotkeyDisplayWidget = CreateHotkeyDisplayWidget(HotkeyChord );

	// Create the actual widget
	this->ChildSlot
	[
		SNew(SHorizontalBox)
		// Icon slot
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			IconWidget
		]
		// Name slot
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(3,0)
		[
			NameSlotWidget
		]
		// Hotkey slot
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			HotkeyDisplayWidget
		]
	];
}

TSharedRef<SWidget> SMaterialPaletteItem::CreateHotkeyDisplayWidget(const TSharedPtr<const FInputChord> HotkeyChord)
{
	FText HotkeyText;
	if (HotkeyChord.IsValid())
	{
		HotkeyText = HotkeyChord->GetInputText();
	}
	return SNew(STextBlock)
		.Text(HotkeyText);
}

FText SMaterialPaletteItem::GetItemTooltip() const
{
	return ActionPtr.Pin()->GetTooltipDescription();
}

//////////////////////////////////////////////////////////////////////////

void SMaterialPalette::Construct(const FArguments& InArgs, TWeakPtr<FMaterialEditor> InMaterialEditorPtr)
{
	MaterialEditorPtr = InMaterialEditorPtr;

	// Create the asset discovery indicator
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_Vertical);

	CategoryNames.Add(MakeShareable(new FString(TEXT("All"))));
	CategoryNames.Add(MakeShareable(new FString(TEXT("Expressions"))));
	CategoryNames.Add(MakeShareable(new FString(TEXT("Functions"))));

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(2.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			// Filter UI
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Comment
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Category", "Category: "))
				]

				// Combo button to select a class
				+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SAssignNew(CategoryComboBox, STextComboBox)
						.OptionsSource(&CategoryNames)
						.OnSelectionChanged(this, &SMaterialPalette::CategorySelectionChanged)
						.InitiallySelectedItem(CategoryNames[0])
					]
			]

			// Content list
			+SVerticalBox::Slot()
				[
					SNew(SOverlay)

					+SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						// Old Expression and Function lists were auto expanded so do the same here for now
						SAssignNew(GraphActionMenu, SGraphActionMenu)
						.OnActionDragged(this, &SMaterialPalette::OnActionDragged)
						.OnCreateWidgetForAction(this, &SMaterialPalette::OnCreateWidgetForAction)
						.OnCollectAllActions(this, &SMaterialPalette::CollectAllActions)
						.AutoExpandActionMenu(true)
					]

					+SOverlay::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Bottom)
						.Padding(FMargin(24, 0, 24, 0))
						[
							// Asset discovery indicator
							AssetDiscoveryIndicator
						]
				]

		]
	];

	// Register with the Asset Registry to be informed when it is done loading up files.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SMaterialPalette::AddAssetFromAssetRegistry);
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &SMaterialPalette::RemoveAssetFromRegistry);
	AssetRegistryModule.Get().OnAssetRenamed().AddSP(this, &SMaterialPalette::RenameAssetFromRegistry);
}

TSharedRef<SWidget> SMaterialPalette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return	SNew(SMaterialPaletteItem, InCreateData);
}

void SMaterialPalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	const UMaterialGraphSchema* Schema = GetDefault<UMaterialGraphSchema>();

	FGraphActionMenuBuilder ActionMenuBuilder;

	// Determine all possible actions
	Schema->GetPaletteActions(ActionMenuBuilder, GetFilterCategoryName(), MaterialEditorPtr.Pin()->OriginalMaterialObject);

	//@TODO: Avoid this copy
	OutAllActions.Append(ActionMenuBuilder);
}

FString SMaterialPalette::GetFilterCategoryName() const
{
	if (CategoryComboBox.IsValid())
	{
		return *CategoryComboBox->GetSelectedItem();
	}
	else
	{
		return TEXT("All");
	}
}

void SMaterialPalette::CategorySelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	RefreshActionsList(true);
}

void SMaterialPalette::AddAssetFromAssetRegistry(const FAssetData& InAddedAssetData)
{
	RefreshAssetInRegistry(InAddedAssetData);
}

void SMaterialPalette::RemoveAssetFromRegistry(const FAssetData& InAddedAssetData)
{
	RefreshAssetInRegistry(InAddedAssetData);
}

void SMaterialPalette::RenameAssetFromRegistry(const FAssetData& InAddedAssetData, const FString& InNewName)
{
	RefreshAssetInRegistry(InAddedAssetData);
}

void SMaterialPalette::RefreshAssetInRegistry(const FAssetData& InAddedAssetData)
{
	if (InAddedAssetData.IsInstanceOf(UMaterialFunction::StaticClass()))
	{
		bNeedRefresh = true;
	}
}

void SMaterialPalette::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bNeedRefresh)
	{
		RefreshActionsList(true);
		bNeedRefresh = false;
	}
}

#undef LOCTEXT_NAMESPACE
