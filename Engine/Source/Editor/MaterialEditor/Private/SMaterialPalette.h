// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/InputChord.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphPalette.h"

class FMaterialEditor;

/** Widget for displaying a single item  */
class SMaterialPaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS( SMaterialPaletteItem ) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

private:
	/* Create the hotkey display widget */
	TSharedRef<SWidget> CreateHotkeyDisplayWidget(const TSharedPtr<const FInputChord> HotkeyChord);

	virtual FText GetItemTooltip() const override;
};

//////////////////////////////////////////////////////////////////////////

class SMaterialPalette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS( SMaterialPalette ) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FMaterialEditor> InMaterialEditorPtr);

protected:
	// SGraphPalette Interface
	virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	// End of SGraphPalette Interface

	/** Get the currently selected category name */
	FString GetFilterCategoryName() const;

	/** Callback for when the selected category changes */
	void CategorySelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	/** Callback from the Asset Registry when a new asset is added. */
	void AddAssetFromAssetRegistry(const FAssetData& InAddedAssetData);

	/** Callback from the Asset Registry when an asset is removed. */
	void RemoveAssetFromRegistry(const FAssetData& InAddedAssetData);

	/** Callback from the Asset Registry when an asset is renamed. */
	void RenameAssetFromRegistry(const FAssetData& InAddedAssetData, const FString& InNewName);

	void RefreshAssetInRegistry(const FAssetData& InAddedAssetData);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	/** Pointer back to the material editor that owns us */
	TWeakPtr<FMaterialEditor> MaterialEditorPtr;

	/** List of available Category Names */
	TArray< TSharedPtr<FString> > CategoryNames;

	/** Combo box used to select category */
	TSharedPtr<STextComboBox> CategoryComboBox;

	bool bNeedRefresh = false;
};
