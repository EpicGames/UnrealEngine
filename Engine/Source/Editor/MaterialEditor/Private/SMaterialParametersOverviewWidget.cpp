// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialParametersOverviewWidget.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Materials/Material.h"
#include "PropertyHandle.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Styling/AppStyle.h"
#include "IPropertyRowGenerator.h"
#include "Widgets/Views/STreeView.h"
#include "IDetailTreeNode.h"
#include "AssetThumbnail.h"
#include "MaterialPropertyHelpers.h"
#include "DetailWidgetRow.h"
#include "EditorSupportDelegates.h"
#include "Widgets/Images/SImage.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "Widgets/Input/SButton.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Framework/Application/SlateApplication.h"

#include "Widgets/Views/SExpanderArrow.h"
#include "MaterialEditor/DEditorRuntimeVirtualTextureParameterValue.h"
#include "MaterialEditor/DEditorSparseVolumeTextureParameterValue.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Styling/StyleColors.h"



#define LOCTEXT_NAMESPACE "MaterialLayerCustomization"

const FSlateBrush* SMaterialParametersOverviewTreeItem::GetBorderImage() const
{
	return FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
}

FSlateColor SMaterialParametersOverviewTreeItem::GetOuterBackgroundColor(TSharedPtr<FSortedParamData> InParamData) const
{
	if (IsHovered() || InParamData->StackDataType == EStackDataType::Group)
	{
		return FAppStyle::Get().GetSlateColor("Colors.Header");
	}

	return FAppStyle::Get().GetSlateColor("Colors.Panel");
}

void SMaterialParametersOverviewTreeItem::RefreshOnRowChange(const FAssetData& AssetData, TSharedPtr<SMaterialParametersOverviewTree> InTree)
{
	if (InTree.IsValid())
	{
		InTree->CreateGroupsWidget();
	}
}

void SMaterialParametersOverviewTreeItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	StackParameterData = InArgs._StackParameterData;
	MaterialEditorInstance = InArgs._MaterialEditorInstance;
	Tree = InArgs._InTree;

	TSharedRef<SWidget> LeftSideWidget = SNullWidget::NullWidget;
	TSharedRef<SWidget> RightSideWidget = SNullWidget::NullWidget;
	FText NameOverride;
	TSharedRef<SVerticalBox> WrapperWidget = SNew(SVerticalBox);
	FMargin RightSidePadding;

// GROUP --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Group)
	{
		NameOverride = FText::FromName(StackParameterData->Group.GroupName);
		LeftSideWidget = SNew(STextBlock)
			.TransformPolicy(ETextTransformPolicy::ToUpper)
			.Text(NameOverride)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle");
	}
// END GROUP

// PROPERTY ----------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Property)
	{
		NameOverride = FText::FromName(StackParameterData->Parameter->ParameterInfo.Name);

		IDetailTreeNode& Node = *StackParameterData->ParameterNode;
		TSharedPtr<IPropertyHandle> PropertyHandle = Node.CreatePropertyHandle();
		TSharedPtr<IDetailPropertyRow> GeneratedRow = StaticCastSharedPtr<IDetailPropertyRow>(Node.GetRow());
		IDetailPropertyRow& Row = *GeneratedRow.Get();
		Row.DisplayName(NameOverride);

		FMaterialPropertyHelpers::SetPropertyRowParameterWidget(Row, StackParameterData->Parameter, PropertyHandle, MaterialEditorInstance);

		FNodeWidgets NodeWidgets = Node.CreateNodeWidgets();
		LeftSideWidget = NodeWidgets.NameWidget.ToSharedRef();
		RightSideWidget = NodeWidgets.ValueWidget.ToSharedRef();

		LeftSideWidget->SetToolTipText(FMaterialPropertyHelpers::GetParameterTooltip(StackParameterData->Parameter, MaterialEditorInstance->PreviewMaterial));

		RightSidePadding = FMargin(12.0f, 0.0f, 2.0f, 0.0f);	// Match DetailWidgetConstants::RightRowPadding
	}
// END PROPERTY

// PROPERTY CHILD ----------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::PropertyChild)
	{
		FNodeWidgets NodeWidgets = StackParameterData->ParameterNode->CreateNodeWidgets();
		LeftSideWidget = NodeWidgets.NameWidget.ToSharedRef();
		RightSideWidget = NodeWidgets.ValueWidget.ToSharedRef();

		RightSidePadding = FMargin(12.0f, 0.0f, 2.0f, 0.0f);	// Match DetailWidgetConstants::RightRowPadding
	}
// END PROPERTY CHILD

// FINAL WRAPPER

	{
		FDetailColumnSizeData& ColumnSizeData = InArgs._InTree->GetColumnSizeData();

		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(this, &SMaterialParametersOverviewTreeItem::GetBorderImage)
					.BorderBackgroundColor(this, &SMaterialParametersOverviewTreeItem::GetOuterBackgroundColor, StackParameterData)
					[
						SNew(SBox)
						.MinDesiredHeight(26.0f)	// Match PropertyEditorConstants::PropertyRowHeight
						[
							SNew(SSplitter)
							.Style(FAppStyle::Get(), "DetailsView.Splitter")
							.PhysicalSplitterHandleSize(1.0f)
							.HitDetectionSplitterHandleSize(5.0f)
							+ SSplitter::Slot()
							.Value(ColumnSizeData.GetNameColumnWidth())
							.OnSlotResized(ColumnSizeData.GetOnNameColumnResized())
							.Value(0.25f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(FMargin(3.0f))
								[
									SNew(SExpanderArrow, SharedThis(this))
								]
								+ SHorizontalBox::Slot()
								.Padding(FMargin(2.0f))
								.VAlign(VAlign_Center)
								[
									LeftSideWidget
								]
							]
							+ SSplitter::Slot()
							.Value(ColumnSizeData.GetValueColumnWidth())
							.OnSlotResized(ColumnSizeData.GetOnValueColumnResized())
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(RightSidePadding)
								[
									RightSideWidget
								]
							]
						]
					]
				]
			];
	}

	this->ChildSlot
		[
			WrapperWidget
		];

	STableRow< TSharedPtr<FSortedParamData> >::ConstructInternal(
		STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false),
		InOwnerTableView
	);
}

void SMaterialParametersOverviewTree::Construct(const FArguments& InArgs)
{
	bHasAnyParameters = false;
	MaterialEditorInstance = InArgs._InMaterialEditorInstance;
	Owner = InArgs._InOwner;
	SelectMaterialNodeDelegate = InArgs._SelectMaterialNode;
	CreateGroupsWidget();

	STreeView<TSharedPtr<FSortedParamData>>::Construct(
		STreeView::FArguments()
		.TreeItemsSource(&SortedParameters)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SMaterialParametersOverviewTree::OnGenerateRowMaterialLayersFunctionsTreeView)
		.OnGetChildren(this, &SMaterialParametersOverviewTree::OnGetChildrenMaterialLayersFunctionsTreeView)
		.OnExpansionChanged(this, &SMaterialParametersOverviewTree::OnExpansionChanged)
		.ExternalScrollbar(InArgs._InScrollbar)
	);
}

TSharedRef< ITableRow > SMaterialParametersOverviewTree::OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SMaterialParametersOverviewTreeItem > ReturnRow = SNew(SMaterialParametersOverviewTreeItem, OwnerTable)
		.StackParameterData(Item)
		.MaterialEditorInstance(MaterialEditorInstance)
		.InTree(SharedThis(this));
	return ReturnRow;
}

void SMaterialParametersOverviewTree::OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> InParent, TArray< TSharedPtr<FSortedParamData> >& OutChildren)
{
	OutChildren = InParent->Children;
}

void SMaterialParametersOverviewTree::OnExpansionChanged(TSharedPtr<FSortedParamData> Item, bool bIsExpanded)
{
	bool* ExpansionValue = MaterialEditorInstance->OriginalMaterial->ParameterOverviewExpansion.Find(Item->NodeKey);
	if (ExpansionValue == nullptr)
	{
		MaterialEditorInstance->OriginalMaterial->ParameterOverviewExpansion.Add(Item->NodeKey, bIsExpanded);
	}
	else if (*ExpansionValue != bIsExpanded)
	{
		MaterialEditorInstance->OriginalMaterial->ParameterOverviewExpansion.Emplace(Item->NodeKey, bIsExpanded);
	}
	// Expand any children that are also expanded
	for (auto Child : Item->Children)
	{
		bool* ChildExpansionValue = MaterialEditorInstance->OriginalMaterial->ParameterOverviewExpansion.Find(Child->NodeKey);
		if (ChildExpansionValue != nullptr && *ChildExpansionValue == true)
		{
			SetItemExpansion(Child, true);
		}
	}
}

void SMaterialParametersOverviewTree::SetParentsExpansionState()
{
	for (const auto& Pair : SortedParameters)
	{
		if (Pair->Children.Num())
		{
			bool* bIsExpanded = MaterialEditorInstance->OriginalMaterial->ParameterOverviewExpansion.Find(Pair->NodeKey);
			if (bIsExpanded)
			{
				SetItemExpansion(Pair, *bIsExpanded);
			}
			else
			{
				SetItemExpansion(Pair, true);
			}
		}
	}
}

TSharedPtr<class FAssetThumbnailPool> SMaterialParametersOverviewTree::GetTreeThumbnailPool()
{
	return UThumbnailManager::Get().GetSharedThumbnailPool();
}

void SMaterialParametersOverviewTree::CreateGroupsWidget()
{
	check(MaterialEditorInstance);
	UnsortedParameters.Reset();
	SortedParameters.Reset();

	const TArray<TSharedRef<IDetailTreeNode>> TestData = GetOwner().Pin()->GetGenerator()->GetRootTreeNodes();
	if (TestData.Num() == 0)
	{
		return;
	}
	TSharedPtr<IDetailTreeNode> Category = TestData[0];
	TSharedPtr<IDetailTreeNode> ParameterGroups;
	TArray<TSharedRef<IDetailTreeNode>> Children;
	Category->GetChildren(Children);

	for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ChildIdx++)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = Children[ChildIdx]->CreatePropertyHandle();
		if (PropertyHandle.IsValid() && PropertyHandle->GetProperty() && PropertyHandle->GetProperty()->GetName() == "ParameterGroups")
		{
			ParameterGroups = Children[ChildIdx];
			break;
		}
	}

	Children.Empty();
	// the order should correspond to UnsortedParameters exactly
	TArray<TSharedPtr<IPropertyHandle>> DeferredSearches;

	if (ParameterGroups.IsValid())
	{
		ParameterGroups->GetChildren(Children);
		for (int32 GroupIdx = 0; GroupIdx < Children.Num(); ++GroupIdx)
		{
			TArray<void*> GroupPtrs;
			TSharedPtr<IPropertyHandle> ChildHandle = Children[GroupIdx]->CreatePropertyHandle();
			ChildHandle->AccessRawData(GroupPtrs);
			auto GroupIt = GroupPtrs.CreateConstIterator();
			const FEditorParameterGroup* ParameterGroupPtr = reinterpret_cast<FEditorParameterGroup*>(*GroupIt);
			if (!ParameterGroupPtr)
			{
				continue;
			}

			const FEditorParameterGroup& ParameterGroup = *ParameterGroupPtr;
			for (int32 ParamIdx = 0; ParamIdx < ParameterGroup.Parameters.Num(); ParamIdx++)
			{
				UDEditorParameterValue* Parameter = ParameterGroup.Parameters[ParamIdx];
				if (Parameter->ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter)
				{
					// Skip parameters that shouldn't have widgets here
					if (!FMaterialPropertyHelpers::ShouldCreatePropertyRowForParameter(Parameter))
					{
						continue;
					}

					bHasAnyParameters = true;
					TSharedPtr<IPropertyHandle> ParametersArrayProperty = ChildHandle->GetChildHandle("Parameters");
					TSharedPtr<IPropertyHandle> ParameterProperty = ParametersArrayProperty->GetChildHandle(ParamIdx);
					TSharedPtr<IPropertyHandle> ParameterValueProperty = ParameterProperty->GetChildHandle("ParameterValue");


					FUnsortedParamData NonLayerProperty;
					NonLayerProperty.Parameter = Parameter;
					NonLayerProperty.ParameterGroup = ParameterGroup;
					NonLayerProperty.UnsortedName = Parameter->ParameterInfo.Name;

					DeferredSearches.Add(ParameterValueProperty);
					UnsortedParameters.Add(NonLayerProperty);
				}
			}
		}
	}

	checkf(UnsortedParameters.Num() == DeferredSearches.Num(), TEXT("Internal inconsistency: number of node searches does not match the number of properties"));
	TArray<TSharedPtr<IDetailTreeNode>> DeferredResults = GetOwner().Pin()->GetGenerator()->FindTreeNodes(DeferredSearches);
	checkf(UnsortedParameters.Num() == DeferredResults.Num(), TEXT("Internal inconsistency: number of node search results does not match the number of properties"));

	for (int Idx = 0, NumUnsorted = UnsortedParameters.Num(); Idx < NumUnsorted; ++Idx)
	{
		FUnsortedParamData& NonLayerProperty = UnsortedParameters[Idx];
		NonLayerProperty.ParameterNode = DeferredResults[Idx];
		NonLayerProperty.ParameterHandle = NonLayerProperty.ParameterNode->CreatePropertyHandle();
	}

	ShowSubParameters();
	RequestTreeRefresh();
	SetParentsExpansionState();
}


bool SMaterialParametersOverviewTree::Private_OnItemDoubleClicked(TSharedPtr<FSortedParamData> TheItem)
{
	if (STreeView<TSharedPtr<FSortedParamData>>::Private_OnItemDoubleClicked(TheItem))
	{
		return true;
	}

	if (!SelectMaterialNodeDelegate.IsBound() || !TheItem.IsValid() || !TheItem->Parameter || 
		!TheItem->Parameter->ExpressionId.IsValid())
	{
		return false;
	}

	SelectMaterialNodeDelegate.Execute(TheItem->Parameter->ExpressionId);

	return true;
}

void SMaterialParametersOverviewTree::ShowSubParameters()
{
	for (FUnsortedParamData Property : UnsortedParameters)
	{
		UDEditorParameterValue* Parameter = Property.Parameter;
		{
			FString GroupNodeKey = FString::FromInt(Parameter->ParameterInfo.Index) + FString::FromInt(Parameter->ParameterInfo.Association) + Property.ParameterGroup.GroupName.ToString();

			bool bAddNewGroup = true;
			for (TSharedPtr<struct FSortedParamData> GroupChild : SortedParameters)
			{
				if (GroupChild->NodeKey == GroupNodeKey)
				{
					bAddNewGroup = false;
				}
			}
			if (bAddNewGroup)
			{
				TSharedPtr<FSortedParamData> GroupProperty(new FSortedParamData());
				GroupProperty->StackDataType = EStackDataType::Group;
				GroupProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
				GroupProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
				GroupProperty->Group = Property.ParameterGroup;
				GroupProperty->NodeKey = GroupNodeKey;

				SortedParameters.Add(GroupProperty);
			}

			TSharedPtr<FSortedParamData> ChildProperty(new FSortedParamData());
			ChildProperty->StackDataType = EStackDataType::Property;
			ChildProperty->Parameter = Parameter;
			ChildProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
			ChildProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
			ChildProperty->ParameterNode = Property.ParameterNode;
			ChildProperty->PropertyName = Property.UnsortedName;
			ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association) +  Property.ParameterGroup.GroupName.ToString() + Property.UnsortedName.ToString();


			// No children for masks
			if (FMaterialPropertyHelpers::ShouldCreateChildPropertiesForParameter(Parameter))
			{
				TArray<TSharedRef<IDetailTreeNode>> ParamChildren;
				Property.ParameterNode->GetChildren(ParamChildren);
				for (int32 ParamChildIdx = 0; ParamChildIdx < ParamChildren.Num(); ParamChildIdx++)
				{
					TSharedPtr<FSortedParamData> ParamChildProperty(new FSortedParamData());
					ParamChildProperty->StackDataType = EStackDataType::PropertyChild;
					ParamChildProperty->ParameterNode = ParamChildren[ParamChildIdx];
					ParamChildProperty->ParameterHandle = ParamChildProperty->ParameterNode->CreatePropertyHandle();
					ParamChildProperty->ParameterInfo.Index = Parameter->ParameterInfo.Index;
					ParamChildProperty->ParameterInfo.Association = Parameter->ParameterInfo.Association;
					ParamChildProperty->Parameter = ChildProperty->Parameter;
					ChildProperty->Children.Add(ParamChildProperty);
				}
			}

			UDEditorRuntimeVirtualTextureParameterValue* VTParameter = Cast<UDEditorRuntimeVirtualTextureParameterValue>(Parameter);
			UDEditorSparseVolumeTextureParameterValue* SVTParameter = Cast<UDEditorSparseVolumeTextureParameterValue>(Parameter);

			// Don't add child property to this group if parameter is of type 'Virtual Texture' or 'Sparse Volume Texture'
			if (!VTParameter && !SVTParameter)
			{
				for (TSharedPtr<struct FSortedParamData> GroupChild : SortedParameters)
				{
					if (GroupChild->Group.GroupName == Property.ParameterGroup.GroupName
						&& GroupChild->ParameterInfo.Association == ChildProperty->ParameterInfo.Association
						&&  GroupChild->ParameterInfo.Index == ChildProperty->ParameterInfo.Index)
					{
						GroupChild->Children.Add(ChildProperty);
					}
				}
			}
		}
	}
}

const FSlateBrush* SMaterialParametersOverviewPanel::GetBackgroundImage() const
{
	return FAppStyle::GetBrush("DetailsView.CategoryTop");
}

int32 SMaterialParametersOverviewPanel::GetPanelIndex() const
{
	return NestedTree && NestedTree->HasAnyParameters() ? 1 : 0;
}

void SMaterialParametersOverviewPanel::Refresh()
{
	TSharedPtr<SHorizontalBox> HeaderBox;
	NestedTree->CreateGroupsWidget();

	FOnClicked 	OnChildButtonClicked = FOnClicked();
	if (MaterialEditorInstance->OriginalFunction)
	{
		OnChildButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewFunctionInstance, ImplicitConv<UMaterialFunctionInterface*>(MaterialEditorInstance->OriginalFunction), ImplicitConv<UMaterialInterface*>(MaterialEditorInstance->PreviewMaterial), ImplicitConv<UObject*>(MaterialEditorInstance));
	}
	else
	{
		OnChildButtonClicked = FOnClicked::CreateStatic(&FMaterialPropertyHelpers::OnClickedSaveNewMaterialInstance, ImplicitConv<UMaterialInterface*>(MaterialEditorInstance->OriginalMaterial), ImplicitConv<UObject*>(MaterialEditorInstance));
	}

	if (NestedTree->HasAnyParameters())
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f)
			[
				
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(HeaderBox, SHorizontalBox)
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					[
						NestedTree.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(16.0f)
						[
							ExternalScrollbar.ToSharedRef()
						]
					]
					
				]
				
			]
		];

		HeaderBox->AddSlot()
			.FillWidth(1.0f)
			[
				SNullWidget::NullWidget
			];

		if (NestedTree->HasAnyParameters())
		{
			HeaderBox->AddSlot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("SaveChild", "Save Child"))
					.HAlign(HAlign_Center)
					.OnClicked(OnChildButtonClicked)
					.ToolTipText(LOCTEXT("SaveToChildInstance", "Save To Child Instance"))
				];
		}
	}
	else
	{
		this->ChildSlot
			[
				SNew(SBox)
				.Padding(10.0f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ConnectMaterialParametersToFillList", "Connect a parameter to see it here."))
				]
			];
	}

	
}


void SMaterialParametersOverviewPanel::Construct(const FArguments& InArgs)
{
	ExternalScrollbar = SNew(SScrollBar);
	TSharedPtr<IPropertyRowGenerator> InGenerator = InArgs._InGenerator;
	Generator = InGenerator;

	NestedTree = SNew(SMaterialParametersOverviewTree)
		.InMaterialEditorInstance(InArgs._InMaterialEditorInstance)
		.InOwner(SharedThis(this))
		.InScrollbar(ExternalScrollbar)
		.SelectMaterialNode(InArgs._SelectMaterialNode);

	MaterialEditorInstance = InArgs._InMaterialEditorInstance;
	Refresh();
}

void SMaterialParametersOverviewPanel::UpdateEditorInstance(UMaterialEditorPreviewParameters* InMaterialEditorInstance)
{
	NestedTree->MaterialEditorInstance = InMaterialEditorInstance;
	Refresh();
}


TSharedPtr<class IPropertyRowGenerator> SMaterialParametersOverviewPanel::GetGenerator()
{
	 return Generator.Pin();
}

#undef LOCTEXT_NAMESPACE
