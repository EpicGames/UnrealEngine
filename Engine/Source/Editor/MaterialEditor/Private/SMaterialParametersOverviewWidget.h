// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailColumnSizeData.h"
#include "Engine/EngineTypes.h"
#include "IDetailTreeNode.h"
#include "MaterialEditorDelegates.h"
#include "MaterialPropertyHelpers.h"
#include "Materials/Material.h"
#include "PropertyCustomizationHelpers.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class IPropertyHandle;
class SMaterialParametersOverviewTree;
class UMaterialEditorPreviewParameters;

// ********* SMaterialParametersOverviewTreeItem *******
class SMaterialParametersOverviewTreeItem : public STableRow< TSharedPtr<FSortedParamData> >
{
public:

	SLATE_BEGIN_ARGS(SMaterialParametersOverviewTreeItem)
		: _StackParameterData(nullptr),
		_MaterialEditorInstance(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<FSortedParamData>, StackParameterData)
	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, MaterialEditorInstance)
	SLATE_ARGUMENT(TSharedPtr<SMaterialParametersOverviewTree>, InTree)
	SLATE_END_ARGS()

	void RefreshOnRowChange(const FAssetData& AssetData, TSharedPtr<SMaterialParametersOverviewTree> InTree);

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

private:
	const FSlateBrush* GetBorderImage() const;

	FSlateColor GetOuterBackgroundColor(TSharedPtr<FSortedParamData> InParamData) const;
private:

	/** The node info to build the tree view row from. */
	TSharedPtr<FSortedParamData> StackParameterData;

	/** The tree that contains this item */
	TWeakPtr<SMaterialParametersOverviewTree> Tree;

	/** The set of material parameters this is associated with */
	UMaterialEditorPreviewParameters* MaterialEditorInstance;
};

// ********* SMaterialParametersOverviewPanel *******
class SMaterialParametersOverviewPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialParametersOverviewPanel)
		: _InMaterialEditorInstance(nullptr)
		, _InGenerator()
	{}

	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, InMaterialEditorInstance)
	SLATE_ARGUMENT(TSharedPtr<class IPropertyRowGenerator>, InGenerator)
	SLATE_EVENT(FSelectMaterialNode, SelectMaterialNode)
	SLATE_END_ARGS()
	void Refresh();
	void Construct(const FArguments& InArgs);
	void UpdateEditorInstance(UMaterialEditorPreviewParameters* InMaterialEditorInstance);
	TSharedPtr<class IPropertyRowGenerator> GetGenerator();

private:
	const FSlateBrush* GetBackgroundImage() const;

	int32 GetPanelIndex() const;

private:
	/** The set of material parameters this is associated with */
	class UMaterialEditorPreviewParameters* MaterialEditorInstance;

	/** The tree contained in this item */
	TSharedPtr<class SMaterialParametersOverviewTree> NestedTree;

	TSharedPtr<class SScrollBar> ExternalScrollbar;
	TWeakPtr<class IPropertyRowGenerator> Generator;
	
};

// ********* SMaterialParametersOverviewTree *******
class SMaterialParametersOverviewTree : public STreeView<TSharedPtr<FSortedParamData>>
{
	friend class SMaterialParametersOverviewTreeItem;

public:

	SLATE_BEGIN_ARGS(SMaterialParametersOverviewTree)
		: _InMaterialEditorInstance(nullptr)
		, _InOwner(nullptr)
		, _InScrollbar()
	{}

	SLATE_ARGUMENT(UMaterialEditorPreviewParameters*, InMaterialEditorInstance)
	SLATE_ARGUMENT(TSharedPtr<SMaterialParametersOverviewPanel>, InOwner)
	SLATE_ARGUMENT(TSharedPtr<SScrollBar>, InScrollbar)
	SLATE_EVENT(FSelectMaterialNode, SelectMaterialNode)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FSortedParamData> InParent, TArray< TSharedPtr<FSortedParamData> >& OutChildren);
	void OnExpansionChanged(TSharedPtr<FSortedParamData> Item, bool bIsExpanded);
	void SetParentsExpansionState();

	TSharedPtr<class FAssetThumbnailPool> GetTreeThumbnailPool();

	/** Object that stores all of the possible parameters we can edit */
	UMaterialEditorPreviewParameters* MaterialEditorInstance;

	/** Builds the custom parameter groups category */
	void CreateGroupsWidget();

	TWeakPtr<SMaterialParametersOverviewPanel> GetOwner() { return Owner; }
	bool HasAnyParameters() const { return bHasAnyParameters; }

	FDetailColumnSizeData& GetColumnSizeData() { return ColumnSizeData; }

	virtual bool Private_OnItemDoubleClicked(TSharedPtr<FSortedParamData> TheItem) override;

protected:

	void ShowSubParameters();

private:

	TArray<TSharedPtr<FSortedParamData>> SortedParameters;

	TArray<FUnsortedParamData> UnsortedParameters;

	TWeakPtr<SMaterialParametersOverviewPanel> Owner;

	bool bHasAnyParameters;
	
	FDetailColumnSizeData ColumnSizeData;

	/** Called when a node should be selected in the material editor. */
	FSelectMaterialNode SelectMaterialNodeDelegate;
};
