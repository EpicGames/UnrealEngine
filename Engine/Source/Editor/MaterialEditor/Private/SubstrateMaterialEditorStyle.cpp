// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubstrateMaterialEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "DetailLayoutBuilder.h"
#include "MaterialEditorModule.h"
#include "Engine/Texture2D.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH(RelativePath, ...) FSlateImageBrush(FSubstrateMaterialEditorStyle::InContent(RelativePath, ".png"), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(FSubstrateMaterialEditorStyle::InContent(RelativePath, ".svg"), __VA_ARGS__)
#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FSubstrateMaterialEditorStyle::StyleInstance = nullptr;

void FSubstrateMaterialEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FSubstrateMaterialEditorStyle::Shutdown()
{
	if (StyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
		ensure(StyleInstance.IsUnique());
		StyleInstance.Reset();
	}
}

FName FSubstrateMaterialEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("SubstrateMaterialEditorStyle"));
	return StyleSetName;
}

const ISlateStyle& FSubstrateMaterialEditorStyle::Get()
{
	if (!StyleInstance.IsValid())
	{
		Initialize();
	}
	return *StyleInstance;
}

FLinearColor FSubstrateMaterialEditorStyle::GetColor(const FName& InName)
{
	return FSubstrateMaterialEditorStyle::Get().GetColor(InName);
}

const FSlateBrush* FSubstrateMaterialEditorStyle::GetBrush(const FName& InName)
{
	return FSubstrateMaterialEditorStyle::Get().GetBrush(InName);
}


const FVector2D Icon8x8(8.f, 8.f);
const FVector2D Icon12x12(12.f, 12.f);
const FVector2D Icon16x16(16.f, 16.f);
const FVector2D Icon20x20(20.f, 20.f);
const FVector2D Icon24x24(24.f, 24.f);
const FVector2D Icon32x32(32.f, 32.f);
const FVector2D Icon40x40(40.f, 40.f);

FLinearColor ReplaceColorAlpha(const FLinearColor& InColor, const float InNewAlpha)
{
	FLinearColor OutColor(InColor);
	OutColor.A = InNewAlpha;
	return OutColor;
}
TSharedRef<FSlateStyleSet> FSubstrateMaterialEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>(TEXT("SubstrateMaterial"));

	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/DynamicMaterial"));
	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Color Styles
	const FLinearColor EngineSelectColor = FStyleColors::Primary.GetSpecifiedColor();
	const FLinearColor EngineSelectHoverColor = FStyleColors::PrimaryHover.GetSpecifiedColor();
	const FLinearColor EngineSelectPressColor = FStyleColors::PrimaryPress.GetSpecifiedColor();

	const FLinearColor SelectColor = ReplaceColorAlpha(FStyleColors::Select.GetSpecifiedColor(), 0.9f);
	const FLinearColor SelectHoverColor = FStyleColors::Select.GetSpecifiedColor();
	const FLinearColor SelectPressColor = EngineSelectPressColor;

	Style->Set("Color.Select", SelectColor);
	Style->Set("Color.Select.Hover", SelectHoverColor);
	Style->Set("Color.Select.Press", SelectPressColor);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Brush Styles
	Style->Set("Icons.Menu.Dropdown", new IMAGE_BRUSH_SVG("Icons/MenuDropdown", Icon16x16));
	
	Style->Set("Icons.Type.None", new IMAGE_BRUSH("Icons/ValueTypes/None", Icon12x12));
	Style->Set("Icons.Type.Bool", new IMAGE_BRUSH("Icons/ValueTypes/Bool", Icon12x12));
	Style->Set("Icons.Type.Float1", new IMAGE_BRUSH("Icons/ValueTypes/Float1", Icon12x12));
	Style->Set("Icons.Type.Float2", new IMAGE_BRUSH("Icons/ValueTypes/Float2", Icon12x12));
	Style->Set("Icons.Type.Float3_RPY", new IMAGE_BRUSH("Icons/ValueTypes/Float3_RPY", Icon12x12));
	Style->Set("Icons.Type.Float3_RGB", new IMAGE_BRUSH("Icons/ValueTypes/Float3_RGB", Icon12x12));
	Style->Set("Icons.Type.Float3_XYZ", new IMAGE_BRUSH("Icons/ValueTypes/Float3_XYZ", Icon12x12));
	Style->Set("Icons.Type.Float4_RGBA", new IMAGE_BRUSH("Icons/ValueTypes/Float4_RGBA", Icon12x12));
	Style->Set("Icons.Type.Float_Any", new IMAGE_BRUSH("Icons/ValueTypes/Float_Any", Icon12x12));
	Style->Set("Icons.Type.Texture", new IMAGE_BRUSH("Icons/ValueTypes/Texture", Icon12x12));
	
	Style->Set("Icons.Material.DefaultLit", new IMAGE_BRUSH("Icons/EditorIcons/MaterialTypeDefaultLit", Icon32x32));
	Style->Set("Icons.Material.Unlit", new IMAGE_BRUSH("Icons/EditorIcons/MaterialTypeUnlit", Icon32x32));
	
	Style->Set("Icons.Lock", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Lock", Icon16x16));
	Style->Set("Icons.Unlock", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Unlock", Icon16x16));
	
	Style->Set("Icons.Remove", new IMAGE_BRUSH("Icons/EditorIcons/Remove_16px", Icon16x16));
	
	Style->Set("Icons.Normalize", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Normalize", Icon16x16));
	
	Style->Set("Icons.Stage.EnabledButton", new IMAGE_BRUSH("Icons/EditorIcons/WhiteBall", Icon8x8));
	Style->Set("Icons.Stage.BaseToggle", new IMAGE_BRUSH("Icons/EditorIcons/BaseToggle_16x", Icon16x16));
	Style->Set("Icons.Stage.MaskToggle", new IMAGE_BRUSH("Icons/EditorIcons/MaskToggle_16x", Icon16x16));
	Style->Set("Icons.Stage.Enabled", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Enable", Icon24x24));
	Style->Set("Icons.Stage.Disabled", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Disable", Icon24x24));
	
	Style->Set("Icons.Stage.ChainLinked", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainLinked", Icon16x16));
	Style->Set("Icons.Stage.ChainUnlinked", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainUnlinked", Icon16x16));
	Style->Set("Icons.Stage.ChainLinked.Horizontal", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainLinked_Horizontal", Icon24x24));
	Style->Set("Icons.Stage.ChainUnlinked.Horizontal", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainUnlinked_Horizontal", Icon24x24));
	Style->Set("Icons.Stage.ChainLinked.Vertical", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainLinked_Vertical", Icon24x24));
	Style->Set("Icons.Stage.ChainUnlinked.Vertical", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainUnlinked_Vertical", Icon24x24));
	
	Style->Set("ImageBorder", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, 0.0f, 
		FStyleColors::Panel.GetSpecifiedColor(), 6.0f));
	
	Style->Set("Border.SinglePixel", new BORDER_BRUSH(TEXT("Images/Borders/Border_SinglePixel"), FMargin(1.0f / 4.0f)));
	Style->Set("Border.LeftTopRight", new BORDER_BRUSH(TEXT("Images/Borders/Border_LeftTopRight"), FMargin(1.0f / 4.0f, 1.0f / 2.0f)));
	Style->Set("Border.LeftBottomRight", new BORDER_BRUSH(TEXT("Images/Borders/Border_LeftBottomRight"), FMargin(1.0f / 4.0f, 1.0f / 2.0f)));
	Style->Set("Border.TopLeftBottom", new BORDER_BRUSH(TEXT("Images/Borders/Border_TopLeftBottom"), FMargin(1.0f / 2.0f, 1.0f / 4.0f)));
	Style->Set("Border.TopRightBottom", new BORDER_BRUSH(TEXT("Images/Borders/Border_TopRightBottom"), FMargin(1.0f / 2.0f, 1.0f / 4.0f)));
	Style->Set("Border.Top", new BORDER_BRUSH(TEXT("Images/Borders/Border_Top"), FMargin(0.0f, 1.0f / 2.0f)));
	Style->Set("Border.Bottom", new BORDER_BRUSH(TEXT("Images/Borders/Border_Bottom"), FMargin(0.0f, 1.0f / 2.0f)));
	Style->Set("Border.Left", new BORDER_BRUSH(TEXT("Images/Borders/Border_Left"), FMargin(1.0f / 2.0f, 0.0f)));
	Style->Set("Border.Right", new BORDER_BRUSH(TEXT("Images/Borders/Border_Right"), FMargin(1.0f / 2.0f, 0.0f)));

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Button Styles
	Style->Set("HoverHintOnly", FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));
	
	Style->Set("HoverHintOnly.Bordered", FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 4.0f, FLinearColor(1, 1, 1, 0.25f), 1.0f))
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f, FLinearColor(1, 1, 1, 0.4f), 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f, FLinearColor(1, 1, 1, 0.5f), 1.0f))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));
	
	Style->Set("HoverHintOnly.Bordered.Dark", FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 4.0f, FStyleColors::InputOutline.GetSpecifiedColor(), 1.0f))
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f, FLinearColor(1, 1, 1, 0.4f), 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f, FLinearColor(1, 1, 1, 0.5f), 1.0f))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));
	
	SetupLayerViewStyles(Style);
	SetupTextStyles(Style);
	
	//////////////////////////////////////////////////////
	/// Editable TextBox Style
	FEditableTextBoxStyle InlineEditableTextBoxStyle;
	InlineEditableTextBoxStyle.SetPadding(FMargin(0));
	InlineEditableTextBoxStyle.SetBackgroundColor(FSlateColor(FLinearColor::Transparent));
	
	Style->Set("InlineEditableTextBoxStyle", InlineEditableTextBoxStyle);

	return Style;
}

void FSubstrateMaterialEditorStyle::SetupLayerViewStyles(const TSharedRef<FSlateStyleSet>& Style)
{
	Style->Set("LayerView.Background", new FSlateRoundedBoxBrush(
		FStyleColors::Recessed, 6.0f, FStyleColors::Recessed, 0.0f));

	Style->Set("LayerView", FTableViewStyle()
		.SetBackgroundBrush(*Style->GetBrush("LayerView.Background"))
	);

	constexpr float LayerViewItemCornerRadius = 10.0f;
	constexpr float LayerViewItemBorderWidth = 1.0f;
	constexpr float DropZoneMargin = 0.25f;

	const FLinearColor LayerViewItem1FillColor = FStyleColors::Dropdown.GetSpecifiedColor();
	const FLinearColor LayerViewItem2FillColor = FStyleColors::DropdownOutline.GetSpecifiedColor();
	constexpr FLinearColor LayerViewItemBorderColor = FLinearColor(0, 0, 0, 1.f);

	const FLinearColor LayerItemHoverFillColor = FStyleColors::Hover.GetSpecifiedColor();
	constexpr FLinearColor LayerItemHoverBorderColor = FLinearColor(1, 1, 1, 0.2f);

	const FLinearColor LayerItemSelectBorderColor = ReplaceColorAlpha(FStyleColors::Select.GetSpecifiedColor(), 0.9f);

	Style->Set("LayerView.Row1.Item", new FSlateRoundedBoxBrush(
		LayerViewItem1FillColor, LayerViewItemCornerRadius));
	Style->Set("LayerView.Row2.Item", new FSlateRoundedBoxBrush(
				LayerViewItem2FillColor, LayerViewItemCornerRadius,
				LayerViewItemBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row.Hovered", new FSlateRoundedBoxBrush(
		LayerItemHoverFillColor, LayerViewItemCornerRadius,
		LayerItemHoverBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row1.ActiveBrush", new FSlateRoundedBoxBrush(
		LayerViewItem1FillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row1.ActiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerViewItem1FillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row1.InactiveBrush", new FSlateRoundedBoxBrush(
		LayerViewItem1FillColor, LayerViewItemCornerRadius));
	Style->Set("LayerView.Row1.InactiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerViewItem1FillColor, LayerViewItemCornerRadius));

	Style->Set("LayerView.Row2.ActiveBrush", new FSlateRoundedBoxBrush(
		LayerViewItem2FillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row2.ActiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerViewItem2FillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row2.InactiveBrush", new FSlateRoundedBoxBrush(
		LayerViewItem2FillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Style->Set("LayerView.Row2.InactiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerViewItem2FillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));

	Style->Set("LayerView.DropIndicatorOnto", new FSlateBoxBrush(
		FPaths::EngineContentDir() + TEXT("Editor/Slate/Common/DropZoneIndicator_Onto.png"),
		FMargin(4.0f / 16.0f), Style->GetColor(TEXT("Color.Select.Hover"))));
	Style->Set("LayerView.DropIndicatorAbove", new FSlateBorderBrush(
		FPaths::EngineContentDir() + TEXT("Editor/Slate/Common/LayersDropZoneDashed_Above.png"), 
		FMargin(DropZoneMargin, DropZoneMargin, 0.f, 0.f), Style->GetColor(TEXT("Color.Select.Hover"))));
	Style->Set("LayerView.DropIndicatorBelow", new FSlateBorderBrush(
		FPaths::EngineContentDir() + TEXT("Editor/Slate/Common/LayersDropZoneDashed_Below.png"), 
		FMargin(DropZoneMargin, 0, 0, DropZoneMargin), Style->GetColor(TEXT("Color.Select.Hover"))));

	Style->Set("LayerView.Row1", FTableRowStyle()
		.SetTextColor(FStyleColors::Foreground)
		.SetSelectedTextColor(FStyleColors::ForegroundHover)
		.SetEvenRowBackgroundBrush(*Style->GetBrush(TEXT("LayerView.Row1.Item")))
		.SetEvenRowBackgroundHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row.Hovered")))
		.SetOddRowBackgroundBrush(*Style->GetBrush(TEXT("LayerView.Row1.Item")))
		.SetOddRowBackgroundHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row.Hovered")))
		.SetSelectorFocusedBrush(*Style->GetBrush(TEXT("LayerView.Row1.ActiveBrush")))
		.SetActiveBrush(*Style->GetBrush(TEXT("LayerView.Row1.ActiveBrush")))
		.SetActiveHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row1.ActiveHoveredBrush")))
		.SetInactiveBrush(*Style->GetBrush(TEXT("LayerView.Row1.InactiveBrush")))
		.SetInactiveHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row1.InactiveHoveredBrush")))
		.SetDropIndicator_Onto(*Style->GetBrush(TEXT("LayerView.DropIndicatorOnto")))
		.SetDropIndicator_Above(*Style->GetBrush(TEXT("LayerView.DropIndicatorAbove")))
		.SetDropIndicator_Below(*Style->GetBrush(TEXT("LayerView.DropIndicatorBelow")))
	);
	Style->Set("LayerView.Row2", FTableRowStyle()
		.SetTextColor(FStyleColors::Foreground)
		.SetSelectedTextColor(FStyleColors::ForegroundHover)
		.SetEvenRowBackgroundBrush(*Style->GetBrush(TEXT("LayerView.Row2.Item")))
		.SetEvenRowBackgroundHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row.Hovered")))
		.SetOddRowBackgroundBrush(*Style->GetBrush(TEXT("LayerView.Row2.Item")))
		.SetOddRowBackgroundHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row.Hovered")))
		.SetSelectorFocusedBrush(*Style->GetBrush(TEXT("LayerView.Row2.ActiveBrush")))
		.SetActiveBrush(*Style->GetBrush(TEXT("LayerView.Row2.ActiveBrush")))
		.SetActiveHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row2.ActiveHoveredBrush")))
		.SetInactiveBrush(*Style->GetBrush(TEXT("LayerView.Row2.InactiveBrush")))
		.SetInactiveHoveredBrush(*Style->GetBrush(TEXT("LayerView.Row2.InactiveHoveredBrush")))
		.SetDropIndicator_Onto(*Style->GetBrush(TEXT("LayerView.DropIndicatorOnto")))
		.SetDropIndicator_Above(*Style->GetBrush(TEXT("LayerView.DropIndicatorAbove")))
		.SetDropIndicator_Below(*Style->GetBrush(TEXT("LayerView.DropIndicatorBelow")))
	);
	Style->Set("LayerView.AddIcon", new IMAGE_BRUSH("Icons/EditorIcons/LayerAdd", Icon16x16));
	Style->Set("LayerView.DuplicateIcon", new IMAGE_BRUSH("Icons/EditorIcons/Duplicate_40x", Icon40x40));
	Style->Set("LayerView.RemoveIcon", new IMAGE_BRUSH("Icons/EditorIcons/LayerRemove", Icon16x16));

	Style->Set("LayerView.Row.Handle", new IMAGE_BRUSH_SVG("Icons/DragHandle", Icon16x16));

	Style->Set("LayerView.Row.OuterRoundBrush", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 15.0f, FStyleColors::Recessed, 1.0f));
}

void FSubstrateMaterialEditorStyle::SetupTextStyles(const TSharedRef<FSlateStyleSet>& Style)
{
	const FTextBlockStyle NormalTextStyle(FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText")));

	const FLinearColor LayerViewItemTextShadowColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);

	FFontOutlineSettings HandleFontOutline;
	HandleFontOutline.OutlineColor = LayerViewItemTextShadowColor;
	HandleFontOutline.OutlineSize = 1;

	FTextBlockStyle SmallTextStyle(NormalTextStyle);
	SmallTextStyle.SetFont(DEFAULT_FONT("Regular", 8));
	Style->Set("SmallFont", SmallTextStyle);

	FTextBlockStyle RegularTextStyle(NormalTextStyle);
	RegularTextStyle.SetFont(DEFAULT_FONT("Regular", 10));
	Style->Set("RegularFont", RegularTextStyle);

	FTextBlockStyle BoldTextStyle(NormalTextStyle);
	BoldTextStyle.SetFont(DEFAULT_FONT("Bold", 10));
	Style->Set("BoldFont", BoldTextStyle);

	Style->Set("ActorName", RegularTextStyle);

	FTextBlockStyle ActorNameBigTextStyle(NormalTextStyle);
	ActorNameBigTextStyle.SetFont(DEFAULT_FONT("Regular", 14));
	Style->Set("ActorNameBig", ActorNameBigTextStyle);

	FTextBlockStyle ComponentNameBigTextStyle(NormalTextStyle);
	ComponentNameBigTextStyle.SetFont(DEFAULT_FONT("Regular", 12));
	Style->Set("ComponentNameBig", ComponentNameBigTextStyle);

	FTextBlockStyle SlotLayerInfoTextStyle(NormalTextStyle);
	SlotLayerInfoTextStyle.SetFont(DEFAULT_FONT("Italic", 8));
	Style->Set("SlotLayerInfo", SlotLayerInfoTextStyle);

	FSlateFontInfo LayerViewItemFont = DEFAULT_FONT("Bold", 12);
	LayerViewItemFont.OutlineSettings = HandleFontOutline;
	Style->Set("LayerView.Row.Font", LayerViewItemFont);

	Style->Set("LayerView.Row.HandleFont", RegularTextStyle);

	FTextBlockStyle LayerViewItemTextStyle(NormalTextStyle);

	Style->Set("LayerView.Row.HeaderText",
		FTextBlockStyle(LayerViewItemTextStyle)
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetFont(LayerViewItemFont));

	Style->Set("LayerView.Row.HeaderText.Small",
		FTextBlockStyle(LayerViewItemTextStyle)
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetFont(RegularTextStyle.Font));

	FTextBlockStyle StagePropertyDetailsTextStyle(NormalTextStyle);
	StagePropertyDetailsTextStyle.SetFont(DEFAULT_FONT("Regular", 12));
	Style->Set("Font.Stage.Details", StagePropertyDetailsTextStyle);

	Style->Set("Font.Stage.Details.Bold", BoldTextStyle);

	FTextBlockStyle StagePropertyDetailsSmallTextStyle(NormalTextStyle);
	StagePropertyDetailsSmallTextStyle.SetFont(IDetailLayoutBuilder::GetDetailFont());
	Style->Set("Font.Stage.Details.Small", StagePropertyDetailsSmallTextStyle);

	FTextBlockStyle StagePropertyDetailsSmallBoldTextStyle(NormalTextStyle);
	StagePropertyDetailsSmallBoldTextStyle.SetFont(IDetailLayoutBuilder::GetDetailFontBold());
	Style->Set("Font.Stage.Details.Small.Bold", StagePropertyDetailsSmallBoldTextStyle);

	Style->Set("Font.Stage.Details.Small.Bold", FTextBlockStyle(NormalTextStyle)
		.SetFont(IDetailLayoutBuilder::GetDetailFontBold()));

	Style->Set("LayerView.Row.SlabWithoutAttributes",
		FTextBlockStyle(ComponentNameBigTextStyle)
		.SetColorAndOpacity(FStyleColors::SelectInactive));
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_PLUGIN_BRUSH_SVG
#undef RootToContentDir
