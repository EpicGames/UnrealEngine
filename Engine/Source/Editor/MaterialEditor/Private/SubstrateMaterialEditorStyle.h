// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FSubstrateMaterialEditorStyle
{
public:
	static void Initialize();
	static void Shutdown();

	static FName GetStyleSetName();
	static const ISlateStyle& Get();

	static FLinearColor GetColor(const FName& InName);
	static const FSlateBrush* GetBrush(const FName& InName);

private:
	static TSharedPtr<FSlateStyleSet> StyleInstance;

	static TSharedRef<FSlateStyleSet> Create();

	static void SetupLayerViewStyles(const TSharedRef<FSlateStyleSet>& Style);
	static void SetupTextStyles(const TSharedRef<FSlateStyleSet>& Style);
};
