// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tabs/MaterialEditorTabFactories.h"
#include "MaterialEditorTabs.h"
#include "MaterialEditor.h"
#include "MaterialGraph/MaterialGraph.h"

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/NamePermissionList.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "MaterialWorkflowTabFactory"

/////////////////////////////////////////////////////
// FMaterialGraphEditorSummoner

void FMaterialGraphEditorSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	MaterialEditorPtr.Pin()->OnGraphEditorFocused(GraphEditor);
}

void FMaterialGraphEditorSummoner::OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	MaterialEditorPtr.Pin()->OnGraphEditorBackgrounded(GraphEditor);
}

void FMaterialGraphEditorSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	GraphEditor->NotifyGraphChanged();
}

void FMaterialGraphEditorSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());

	FVector2f ViewLocation;
	float ZoomAmount;
	GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);

	UEdGraph* Graph = Payload->IsValid() ? FTabPayload_UObject::CastChecked<UEdGraph>(Payload) : nullptr;
}

FMaterialGraphEditorSummoner::FMaterialGraphEditorSummoner(TSharedPtr<class FMaterialEditor> InMaterialEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback) : FDocumentTabFactoryForObjects<UEdGraph>("Document", InMaterialEditorPtr)
, MaterialEditorPtr(InMaterialEditorPtr)
, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{
}

FTabSpawnerEntry& FMaterialGraphEditorSummoner::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* CurrentApplicationMode) const
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.TabManager = InTabManager;
	SpawnInfo.Payload = FTabPayload_UObject::Make(MaterialEditorPtr.Pin()->Material->MaterialGraph);

	TWeakPtr<FTabManager> WeakTabManager(InTabManager);
	FTabSpawnerEntry& SpawnerEntry = InTabManager->RegisterTabSpawner(GetIdentifier(), FOnSpawnTab::CreateSP(this, &FMaterialGraphEditorSummoner::OnSpawnTab, WeakTabManager), FCanSpawnTab::CreateSP(this, &FMaterialGraphEditorSummoner::CanSpawnTab, WeakTabManager))
		.SetDisplayName(LOCTEXT("GraphTabName", "Graph"));

	if (CurrentApplicationMode)
	{
		SpawnerEntry.SetGroup(CurrentApplicationMode->GetWorkspaceMenuCategory());
	}

	// Add the tab icon to the menu entry if one was provided
	const FSlateIcon& TabSpawnerIcon = GetTabSpawnerIcon(SpawnInfo);
	if (TabSpawnerIcon.IsSet())
	{
		SpawnerEntry.SetIcon(TabSpawnerIcon);
	}

	return SpawnerEntry;
}

TSharedRef<SDockTab> FMaterialGraphEditorSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FDocumentTracker> DocumentManager = MaterialEditorPtr.Pin()->DocumentManager;

	// Call 'CreateTabBodyForObject' indirectly by making the material document manager open the tab, this preserves history while letting us manage the tab checkmark
	return DocumentManager->GetActiveTab().IsValid()
		? DocumentManager->GetActiveTab().ToSharedRef()
		: DocumentManager->OpenDocument(Info.Payload, FDocumentTracker::SpawnManagedDocument).ToSharedRef();
}

TSharedRef<SDockTab> FMaterialGraphEditorSummoner::OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.TabManager = WeakTabManager.Pin();
	SpawnInfo.Payload = FTabPayload_UObject::Make(MaterialEditorPtr.Pin()->Material->MaterialGraph);

	return SpawnInfo.TabManager.IsValid() ? SpawnTab(SpawnInfo) : SNew(SDockTab);
}

TSharedRef<SWidget> FMaterialGraphEditorSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	check(Info.TabInfo.IsValid());
	return OnCreateGraphEditorWidget.Execute(Info.TabInfo.ToSharedRef(), DocumentID);
}

const FSlateBrush* FMaterialGraphEditorSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_16x"));
}

TSharedRef<FGenericTabHistory> FMaterialGraphEditorSummoner::CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload)
{
	return MakeShareable(new FMaterialGraphTabHistory(SharedThis(this), Payload));
}

/////////////////////////////////////////////////////
// FMaterialGraphTabHistory

void FMaterialGraphTabHistory::EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches)
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.Payload = Payload;
	SpawnInfo.TabInfo = InTabInfo;

	if(bPrevTabMatches)
	{
		TSharedPtr<SDockTab> DockTab = InTabInfo->GetTab().Pin();
		GraphEditor = StaticCastSharedRef<SGraphEditor>(DockTab->GetContent());
	}
	else
	{
		TSharedRef< SGraphEditor > GraphEditorRef = StaticCastSharedRef< SGraphEditor >(FactoryPtr.Pin()->CreateTabBody(SpawnInfo));
		GraphEditor = GraphEditorRef;
		FactoryPtr.Pin()->UpdateTab(InTabInfo->GetTab().Pin(), SpawnInfo, GraphEditorRef);
	}
}

void FMaterialGraphTabHistory::SaveHistory()
{
	if (IsHistoryValid())
	{
		check(GraphEditor.IsValid());
		GraphEditor.Pin()->GetViewLocation(SavedLocation, SavedZoomAmount);
		GraphEditor.Pin()->GetViewBookmark(SavedBookmarkId);
	}
}

void FMaterialGraphTabHistory::RestoreHistory()
{
	if (IsHistoryValid())
	{
		check(GraphEditor.IsValid());
		GraphEditor.Pin()->SetViewLocation(SavedLocation, SavedZoomAmount, SavedBookmarkId);
	}
}

#undef LOCTEXT_NAMESPACE