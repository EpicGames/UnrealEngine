// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"

#include "GraphEditor.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

#define LOCTEXT_NAMESPACE "MaterialEditorTabFactories"

class FAssetEditorToolkit;
class SDockTab;

/////////////////////////////////////////////////////
// FMaterialGraphEditorSummoner

struct FMaterialGraphEditorSummoner : public FDocumentTabFactoryForObjects<UEdGraph>
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, TSharedRef<FTabInfo>, UEdGraph*);
public:
	FMaterialGraphEditorSummoner(TSharedPtr<class FMaterialEditor> InMaterialEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;

	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

	virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const override;

protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UEdGraph* DocumentID) const override
	{
		auto GetGraphDisplayName = [](UEdGraph* Graph)
		{
			if (Graph)
			{
				if (const UEdGraphSchema* Schema = Graph->GetSchema())
				{
					FGraphDisplayInfo Info;
					Schema->GetGraphDisplayInformation(*Graph, /*out*/ Info);

					return Info.DisplayName;
				}
				else
				{
					// if we don't have a schema, we're dealing with a malformed (or incomplete graph)...
					// possibly in the midst of some transaction - here we return the object's outer path 
					// so we can at least get some context as to which graph we're referring
					return FText::FromString(Graph->GetPathName());
				}
			}

			return LOCTEXT("UnknownGraphName", "UNKNOWN");
		};

		return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(GetGraphDisplayName, DocumentID));
	}

	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

	virtual TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const override;

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual TSharedRef<FGenericTabHistory> CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload) override;

protected:
	TWeakPtr<class FMaterialEditor> MaterialEditorPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};


/////////////////////////////////////////////////////
// FMaterialGraphTabHistory

struct FMaterialGraphTabHistory : public FGenericTabHistory
{
public:
	/**
	 * @param InFactory		The factory used to regenerate the content
	 * @param InPayload		The payload object used to regenerate the content
	 */
	FMaterialGraphTabHistory(TSharedPtr<FDocumentTabFactory> InFactory, TSharedPtr<FTabPayload> InPayload)
		: FGenericTabHistory(InFactory, InPayload)
		, SavedLocation(FVector2f::ZeroVector)
		, SavedZoomAmount(INDEX_NONE)
	{

	}

	virtual void EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches) override;

	virtual void SaveHistory() override;

	virtual void RestoreHistory() override;

private:
	/** The graph editor represented by this history node. While this node is inactive, the graph editor is invalid */
	TWeakPtr< class SGraphEditor > GraphEditor;
	/** Saved location the graph editor was at when this history node was last visited */
	FDeprecateSlateVector2D SavedLocation;
	/** Saved zoom the graph editor was at when this history node was last visited */
	float SavedZoomAmount;
	/** Saved bookmark ID the graph editor was at when this history node was last visited */
	FGuid SavedBookmarkId;
};


#undef LOCTEXT_NAMESPACE
