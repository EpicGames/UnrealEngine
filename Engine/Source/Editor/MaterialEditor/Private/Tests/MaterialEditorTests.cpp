// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Materials/Material.h"
#include "MaterialGraph/MaterialGraph.h"
#include "Engine/Texture.h"
#include "AssetRegistry/AssetData.h"


//Automation
#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationEditorPromotionCommon.h"

//Assets

//Materials
#include "MaterialEditor.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Subsystems/AssetEditorSubsystem.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "EditorMaterialEditorPromotionTests"

DEFINE_LOG_CATEGORY_STATIC(LogEditorMaterialEditorPromotionTests, Log, All);

/**
* Helper functions used by the build promotion automation test
*/
namespace MaterialEditorPromotionTestUtils
{
	/** 
	* Gets saved settings for Material Editor promotion 
	*/
	static FMaterialEditorPromotionSettings TestSettings()
	{
		UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
		check(AutomationTestSettings);

		return AutomationTestSettings->MaterialEditorPromotionTest;
	}


	/**
	* Assigns a normal map to a material
	*
	* @param InNormalTexture - The texture to use as the normal map
	* @param InMaterial - The material to modify
	*/
	static bool AssignNormalToMaterial(UTexture* InNormalTexture, UMaterial* InMaterial)
	{
		IAssetEditorInstance* OpenEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InMaterial, true);
		IMaterialEditor* CurrentMaterialEditor = (IMaterialEditor*)OpenEditor;

		//Create the texture sample and auto assign the normal texture
		UMaterialExpressionTextureSample* NewTextureSampleExpression = Cast<UMaterialExpressionTextureSample>(CurrentMaterialEditor->CreateNewMaterialExpression(UMaterialExpressionTextureSample::StaticClass(), FVector2D(100.f, 200.f), true, true));
		if (NewTextureSampleExpression)
		{
			UE_LOGF(LogEditorMaterialEditorPromotionTests, Display, "Created a new texture sample expression");
			NewTextureSampleExpression->Texture = InNormalTexture;
			NewTextureSampleExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Normal;
			UE_LOGF(LogEditorMaterialEditorPromotionTests, Display, "Assigned the normal map texture to the new node");
			UMaterial* EditorMaterial = Cast<UMaterial>(CurrentMaterialEditor->GetMaterialInterface());
			UMaterialGraph* MaterialGraph = EditorMaterial->MaterialGraph;
			EditorMaterial->GetEditorOnlyData()->Normal.Connect(0, NewTextureSampleExpression);
			UE_LOGF(LogEditorMaterialEditorPromotionTests, Display, "Connected the new node to the normal pin");
			MaterialGraph->LinkGraphNodesFromMaterial();

			return true;
		}
		else
		{
			UE_LOGF(LogEditorMaterialEditorPromotionTests, Error, "Could not add a texture sample to %ls", *InMaterial->GetName());
			return true;
		}
	}

	template<class AssetType> 
	static AssetType* GetAssetFromPackagePath(const FString& PackagePath)
	{
		FAssetData AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(PackagePath);
		return Cast<AssetType>(AssetData.GetAsset());
	}

}


/**
* Container for items related to the create asset test
*/
namespace MaterialEditorPromotionTestHelper
{
	struct FMaterialEditorPromotionTest
	{
		/** Pointer to the execution info of this test */
		FAutomationTestExecutionInfo* TestExecutionInfo;

		/** Function definition for the test stage functions */
		typedef bool(MaterialEditorPromotionTestHelper::FMaterialEditorPromotionTest::*TestStageFn)();

		/** List of test stage functions */
		TArray<TestStageFn> TestStages;

		/** The index of the test stage we are on */
		int32 CurrentStage;

		// Test assets
		UTexture* DiffuseTexture;
		UTexture* NormalTexture;

		/** Material created from the "Creating a Material" stage */
		UMaterial* CreatedMaterial;

		/**
		* Constructor
		*/
		FMaterialEditorPromotionTest(FAutomationTestExecutionInfo* InExecutionInfo) :
			CurrentStage(0)
		{
			FMemory::Memzero(this, sizeof(FMaterialEditorPromotionTest));

			TestExecutionInfo = InExecutionInfo;

			//Add test stage functions to array in order of execution
			TestStages.Add(&MaterialEditorPromotionTestHelper::FMaterialEditorPromotionTest::ContentBrowser_CreateAMaterial_Part1);
			TestStages.Add(&MaterialEditorPromotionTestHelper::FMaterialEditorPromotionTest::ContentBrowser_CreateAMaterial_Part2);
		}
		/**
		* Runs the current test stage
		*/
		bool Update()
		{
			bool bStageComplete = (this->*TestStages[CurrentStage])();
			if (bStageComplete)
			{
				CurrentStage++;
			}
			return CurrentStage >= TestStages.Num();
		}

	private:

		/**
		* ContentBrowser Test Stage: Creating a material (Part 1)
		*    Creates a material from the diffuse provided in the AutomationTestSettings and opens the material editor
		*/
		bool ContentBrowser_CreateAMaterial_Part1()
		{
			FString DiffuseTexturePackagePath = MaterialEditorPromotionTestUtils::TestSettings().DefaultDiffuseTexture.FilePath;
			if (!(DiffuseTexturePackagePath.Len() > 0))
			{
				UE_LOGF(LogEditorMaterialEditorPromotionTests, Warning, "Skipping material creation test: No texture asset defined.");
				return true;
			}
			DiffuseTexture = MaterialEditorPromotionTestUtils::GetAssetFromPackagePath<UTexture>(DiffuseTexturePackagePath);

			if (DiffuseTexture)
			{
				// @TODO: Break out TEST 1: Create material from texture
				CreatedMaterial = FEditorPromotionTestUtilities::CreateMaterialFromTexture(DiffuseTexture);
				if (CreatedMaterial)
				{
					UE_LOGF(LogEditorMaterialEditorPromotionTests, Display, "Created new material (%ls) from texture (%ls)", *CreatedMaterial->GetName(), *DiffuseTexture->GetName());
					// @TODO: Break out TEST 2: Open a material
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CreatedMaterial);
					UE_LOGF(LogEditorMaterialEditorPromotionTests, Display, "Opened the material editor");
				}
				else
				{
					UE_LOGF(LogEditorMaterialEditorPromotionTests, Error, "Failed to create material fom texture");
				}
			}
			else
			{
				UE_LOGF(LogEditorMaterialEditorPromotionTests, Error, "Failed to load texture asset");
			}

			return true;
		}

		/**
		* ContentBrowser Test Stage: Creating a material (Part 2)
		*    Adds the normal map texture to the material and updates the shader
		*/
		bool ContentBrowser_CreateAMaterial_Part2()
		{ 
			if (CreatedMaterial)
			{
				FString NormalTexturePackagePath = MaterialEditorPromotionTestUtils::TestSettings().DefaultNormalTexture.FilePath;
				if (!(NormalTexturePackagePath.Len() > 0))
				{
					UE_LOGF(LogEditorMaterialEditorPromotionTests, Warning, "Skipping material change test: No normal texture asset defined.");
					return true;
				}
				NormalTexture = MaterialEditorPromotionTestUtils::GetAssetFromPackagePath<UTexture>(NormalTexturePackagePath);

				if (NormalTexture)
				{
					IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(CreatedMaterial, true);
					FMaterialEditor* MaterialEditor = (FMaterialEditor*)AssetEditor;

					// @TODO: Break out TEST 3: Add normal map to a material
					if (MaterialEditorPromotionTestUtils::AssignNormalToMaterial(NormalTexture, CreatedMaterial))
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(CreatedMaterial, true);
						// @TODO: Break out TEST 4: Compile a material
						MaterialEditor->UpdateMaterialAfterGraphChange();
						MaterialEditor->bMaterialDirty = false; // Remove dirty flag so window closes without prompt
						UE_LOGF(LogEditorMaterialEditorPromotionTests, Display, "Compiled the new material");
						MaterialEditor->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);

					}
				}
				else
				{
					UE_LOGF(LogEditorMaterialEditorPromotionTests, Error, "Failed to load normal texture asset");
				}
			}
			else
			{
				UE_LOGF(LogEditorMaterialEditorPromotionTests, Warning, "Skipping material change test: Previous test step either did not run, or did not succeed.");
			}

			return true;
		}
	};
}


/**
* Latent command to run the main build promotion test
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FRunMaterialEditorPromotionTestCommand, TSharedPtr<MaterialEditorPromotionTestHelper::FMaterialEditorPromotionTest>, MaterialEditorPromotionTest);
bool FRunMaterialEditorPromotionTestCommand::Update()
{
	return MaterialEditorPromotionTest->Update();
}


/**
* Automation test that handles the build promotion process
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMaterialEditorPromotionTest, "System.Promotion.Editor.Material Editor", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FMaterialEditorPromotionTest::RunTest(const FString& Parameters)
{
	TSharedPtr<MaterialEditorPromotionTestHelper::FMaterialEditorPromotionTest> MaterialEditorPromotionTest = MakeShareable(new MaterialEditorPromotionTestHelper::FMaterialEditorPromotionTest(&ExecutionInfo));
	ADD_LATENT_AUTOMATION_COMMAND(FRunMaterialEditorPromotionTestCommand(MaterialEditorPromotionTest));
	return true;
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_DEV_AUTOMATION_TESTS
