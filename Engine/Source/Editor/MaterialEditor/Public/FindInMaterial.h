// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Editor/MaterialEditor/Private/MaterialEditor.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

/** Item that matched the search results */
class FFindInMaterialResult
{
public:
	/** Create a root (or only text) result */
	FFindInMaterialResult(const FString& InValue);

	/** Create a listing for a search result*/
	FFindInMaterialResult(const FString& InValue, TSharedPtr<FFindInMaterialResult>& InParent, UClass* InClass, int InDuplicationIndex);

	/** Create a listing for a pin result */
	FFindInMaterialResult(const FString& InValue, TSharedPtr<FFindInMaterialResult>& InParent, UEdGraphPin* InPin);

	/** Create a listing for a node result */
	FFindInMaterialResult(const FString& InValue, TSharedPtr<FFindInMaterialResult>& InParent, UEdGraphNode* InNode);

	/** Called when user clicks on the search item */
	FReply OnClick(TWeakPtr<class FMaterialEditor> MaterialEditor);

	/* Get Category for this search result */
	FText GetCategory() const;

	/** Create an icon to represent the result */
	TSharedRef<SWidget>	CreateIcon() const;

	/** Gets the comment on this node if any */
	FString GetCommentText() const;

	/** Gets the value of the pin if any */
	FString GetValueText() const;

	/** Any children listed under this category */
	TArray< TSharedPtr<FFindInMaterialResult> > Children;

	/** Search result Parent */
	TWeakPtr<FFindInMaterialResult> Parent;

	/*The meta string that was stored in the asset registry for this item */
	FString Value;

	/*The graph may have multiple instances of whatever we are looking for, this tells us which instance # we refer to*/
	int	DuplicationIndex;

	/*The class this item refers to */
	UClass* Class;

	/** The pin that this search result refers to */
	FEdGraphPinReference Pin;

	/** The graph node that this search result refers to (if not by asset registry or UK2Node) */
	TWeakObjectPtr<UEdGraphNode> GraphNode;

	/** Display text for comment information */
	FString CommentText;

	/** Boolean for marking that the result was found embedded in another node e.g. cascading MFs*/
	bool bEmbeddedViaNode = false;
};


/** Widget for searching for items that are part of a UEdGraph */
class SFindInMaterial : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFindInMaterial){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class FMaterialEditor> InMaterialEditor);

	/** Focuses this widget's search box */
	void FocusForUse();

	/** Clears out the existing search items and repopulates the list with these nodes */
	void PopulateSearchItems(const TArray<UEdGraphNode*>& Nodes);

protected:
	typedef TSharedPtr<FFindInMaterialResult> FSearchResult;
	typedef STreeView<FSearchResult> STreeViewType;

	/** Called when user changes the text they are searching for */
	void OnSearchTextChanged(const FText& Text);

	/** Called when user changes commits text to the search box */
	void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/** Get the children of a row */
	void OnGetChildren(FSearchResult InItem, TArray<FSearchResult>& OutChildren);

	/** Called when user clicks on a new result */
	void OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type SelectInfo);

	/** Called when user double clicks on a new result */
	void OnTreeSelectionDoubleClick(FSearchResult Item);

	/** Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Begins the search based on the SearchValue */
	void InitiateSearch();

	/** Find any results that contain all of the tokens */
	void MatchTokens(const TArray<FString>& Tokens);

	/** Find any results that contain all of the tokens in provided graph and subgraphs */
	void MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens);

	/** Determines if a string matches the search tokens */
	static bool StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString);

protected:
	/** Pointer back to the Material editor that owns us */
	TWeakPtr<class FMaterialEditor> MaterialEditorPtr;

	/** The tree view displays the results */
	TSharedPtr<STreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<class SSearchBox> SearchTextField;

	/** This buffer stores the currently displayed results */
	TArray<FSearchResult> ItemsFound;

	/** we need to keep a handle on the root result, because it won't show up in the tree */
	FSearchResult RootSearchResult;

	/** The string to highlight in the results */
	FText HighlightText;

	/** The string to search for */
	FString	SearchValue;
};
