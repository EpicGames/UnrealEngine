// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class FCanvas;
class FMaterialRenderProxy;
class FViewport;
class IAssetReferenceFilter;
class IMaterialEditorModule;
class UMaterialExpression;
class UMaterialExpressionComment;
class UMaterialExpressionComposite;
class UMaterialGraphNode;
class UMaterialInterface;

/**
 * Public interface to Material Editor
 */
class IMaterialEditor : public FWorkflowCentricApplication, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/**
	 * Creates a new material expression of the specified class.
	 *
	 * @param	NewExpressionClass		The type of material expression to add.  Must be a child of UMaterialExpression.
	 * @param	NodePos					Position of the new node.
	 * @param	bAutoSelect				If true, deselect all expressions and select the newly created one.
	 * @param	bAutoAssignResource		If true, assign resources to new expression.
	 * @param	Graph            		Graph to create new expression within. 
	 *
	 * @return	UMaterialExpression*	Newly created material expression
	 */
	virtual UMaterialExpression* CreateNewMaterialExpression(UClass* NewExpressionClass, const FVector2D& NodePos, bool bAutoSelect, bool bAutoAssignResource, const class UEdGraph* Graph = nullptr) {return nullptr;}

	/**
	 * Creates a new material expression composite, along with related child pinbase expressions
	 *
	 * @param	NodePos							Position of the new composite
	 * @param	Graph							Graph to create new expression within.
	 *
	 * @return	UMaterialExpressionComposite*	Newly created material expression composite
	 */
	virtual UMaterialExpressionComposite* CreateNewMaterialExpressionComposite(const FVector2D& NodePos, const class UEdGraph* Graph = nullptr) { return nullptr; }

	/**
	 * Creates a new material expression comment
	 *
	 * @param	NodePos						Position of the new comment
	 *
	 * @return	UMaterialExpressionComment*	Newly created material expression comment
	 * @param	Graph            			Graph to create new expression within.
	 */
	virtual UMaterialExpressionComment* CreateNewMaterialExpressionComment(const FVector2D& NodePos, const class UEdGraph* Graph = nullptr) {return nullptr;}

	/**
	 * Refreshes all material expression previews, regardless of whether or not realtime previews are enabled.
	 */
	virtual void ForceRefreshExpressionPreviews() {};

	/**
	 * Add the specified material expression's graph node to the list of selected nodes
	 */
	virtual void AddToSelection(UMaterialExpression* Expression) {};

	/**
	 * Jumps to the node for this expression, if possible
	 */
	virtual void JumpToExpression(UMaterialExpression* Expression) {};

	/**
	 * Disconnects and removes the selected material graph nodes.
	 */
	virtual void DeleteSelectedNodes() {};

	/** 
	 * Delete an array of Material Graph Nodes and their corresponding expressions/comments
	*/
	virtual void DeleteNodes(const TArray<class UEdGraphNode*>& NodesToDelete) {}

	/**
	 * Gets the name of the material or material function that we are editing
	 */
	virtual FText GetOriginalObjectName() const {return FText::GetEmpty();}

	/**
	 * Creates an IAssetReferenceFilter that can be used to check if an asset is valid to be used with this material.
	 */
	virtual TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const { return {}; }

	/**
	 * Re-links the material and updates its representation in the editor,
	 * used when graph is changed outside of editor code.
	 */
	virtual void UpdateMaterialAfterGraphChange() {};

	/** Marks the material as dirty. */
	virtual void MarkMaterialDirty() {};

	virtual void UpdateDetailView() {}

	/**
	 * Jumps to a specific node or graph
	 *
	 * @param	ObjectReference			Object to jump to
	 */
	virtual void JumpToHyperlink(const UObject* ObjectReference) {};

	/** Checks whether nodes can currently be pasted */
	virtual bool CanPasteNodes() const {return false;}

	/** Paste nodes at a specific location */
	virtual void PasteNodesHere(const FVector2D& Location, const class UEdGraph* Graph = nullptr) {};

	/** Gets the number of selected nodes */
	virtual int32 GetNumberOfSelectedNodes() const {return 0;}

	/** Get the currently selected set of nodes */
	virtual TSet<UObject*> GetSelectedNodes() const {return TSet<UObject*>();}

	/** Get the visible bounds of the given graph node */
	virtual void GetBoundsForNode(const UEdGraphNode* InNode, class FSlateRect& OutRect, float InPadding) const {};

	/**
	 * Gets the preview for an expression
	 *
	 * @param	InExpression			The expression to preview
	 *
	 * @return	FMaterialRenderProxy*	The expression preview
	 */
	virtual FMaterialRenderProxy* GetExpressionPreview(UMaterialExpression* InExpression) {return NULL;}

	/**
	 * Updates the SearchResults array based on the search query
	 *
	 * @param	bQueryChanged		Indicates whether the update is because of a query change or a potential material content change.
	 */
	virtual void UpdateSearch( bool bQueryChanged ) {};

	/** The material instance applied to the preview mesh. */
	virtual UMaterialInterface* GetMaterialInterface() const = 0;

	/** Allows editor to veto the setting of a preview mesh */
	virtual bool ApproveSetPreviewAsset(UObject* InAsset) = 0;

	/** Draws messages on the specified viewport and canvas. */
	virtual void DrawMessages( FViewport* Viewport, FCanvas* Canvas ) = 0;

	/** Delegate to be called when the tabs are being registered **/
	DECLARE_EVENT_OneParam(IMaterialEditorModule, FRegisterTabSpawnersEvent, const TSharedRef<class FTabManager>&);
	virtual FRegisterTabSpawnersEvent& OnRegisterTabSpawners() { return RegisterTabSpawnersEvent; };

	/** Delegate to be called when the tabs are being unregistered **/
	DECLARE_EVENT_OneParam(IMaterialEditorModule, FUnregisterTabSpawnersEvent, const TSharedRef<class FTabManager>&);
	virtual FUnregisterTabSpawnersEvent& OnUnregisterTabSpawners() { return UnregisterTabSpawnersEvent; };

	/** Delegate to be called when this IMaterialEditor is about to be destroyed **/
	DECLARE_EVENT(IMaterialEditorModule, FMaterialEditorClosedEvent);
	virtual FMaterialEditorClosedEvent& OnMaterialEditorClosed() { return MaterialEditorClosedEvent; };

	/** call this to notify the editor that the edited material changed from outside */
	virtual void NotifyExternalMaterialChange() {}

	/** Rebuild the preview + graph view from the asset under edit (Material or MaterialFunction). */
	virtual void RefreshGraphFromOriginal() {}

	/** Fills inheritance drop down menu */
	virtual void GenerateInheritanceMenu(class UToolMenu* Menu) {};

	/** Updates materials used in material stats */
	virtual void RefreshStatsMaterials() {};

	/** Start/Stop Previewing the specified material graph node */
	virtual void PreviewNode(UMaterialGraphNode* InGraphNode) {};

	/** Set the custom primitive data to use in the preview display area */
	virtual void SetCustomPrimitiveData(TConstArrayView<float> CustomFloats) {}

private:
	FMaterialEditorClosedEvent MaterialEditorClosedEvent;
	FRegisterTabSpawnersEvent RegisterTabSpawnersEvent;
	FUnregisterTabSpawnersEvent UnregisterTabSpawnersEvent;
};


