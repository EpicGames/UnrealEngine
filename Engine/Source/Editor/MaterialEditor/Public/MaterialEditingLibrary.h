// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialExpression.h"
#include "SceneTypes.h"

#include "MaterialEditingLibrary.generated.h"

#define UE_API MATERIALEDITOR_API

class UMaterialFunction;
class UMaterialInstance;
class FMaterialUpdateContext;
class UMaterialInstanceConstant;
class URuntimeVirtualTexture;

USTRUCT(BlueprintType)
struct FMaterialStatistics
{
	GENERATED_BODY()

	/** Number of instructions used by most expensive vertex shader in the material */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Statistics)
	int32 NumVertexShaderInstructions = 0;

	/** Number of instructions used by most expensive pixel shader in the material */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Statistics)
	int32 NumPixelShaderInstructions = 0;

	/** Number of samplers required by the material */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Statistics)
	int32 NumSamplers = 0;

	/** Number of textures sampled by the vertex shader */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Statistics)
	int32 NumVertexTextureSamples = 0;

	/** Number of textures sampled by the pixel shader */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Statistics)
	int32 NumPixelTextureSamples = 0;

	/** Number of virtual textures sampled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Statistics)
	int32 NumVirtualTextureSamples = 0;

	/** Number of interpolator scalars required for UVs */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Statistics)
	int32 NumUVScalars = 0;

	/** Number of interpolator scalars required for user-defined interpolators */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Statistics)
	int32 NumInterpolatorScalars = 0;
};

/** This struct represents a single shader in a material. */
USTRUCT(BlueprintType)
struct FDebugShaderInfo
{
	GENERATED_BODY()

	/** Name of the vertex factory that this shader belongs to.  Could be NAME_None for material-only shaders. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Statistics)
	FName VertexFactoryName;

	/** Name of the shader type for this shader */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Statistics)
	FName ShaderTypeName;
};

/** Blueprint library for creating/editing Materials */
UCLASS(MinimalAPI)
class UMaterialEditingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	*	Create a new material expression node within the supplied material, optionally specifying asset to use
	*	@note	If a MaterialFunction and Material are specified, expression is added to Material and not MaterialFunction, assuming Material is a preview that will be copied to Function later by user.
	*	@param	Material					Material asset to add an expression to
	*	@param	MaterialFunction			Specified if adding an expression to a MaterialFunction, used as Outer for new expression object
	*	@param	SelectedAsset				If specified, new node will attempt to use this asset, if of the appropriate type (e.g. Texture for a TextureSampler)
	*	@param	ExpressionClass				Class of expression to add
	*	@param	NodePosX					X position of new expression node
	*	@param	NodePosY					Y position of new expression node
	*	@param	bAllowMarkingPackageDirty	Packages can't be marked dirty outside of the game thread. If this is false, package will need to be dirtied through other means. 
	*/
	static UE_API UMaterialExpression* CreateMaterialExpressionEx(UMaterial* Material, UMaterialFunction* MaterialFunction, TSubclassOf<UMaterialExpression> ExpressionClass,
		UObject* SelectedAsset = nullptr, int32 NodePosX = 0, int32 NodePosY = 0, bool bAllowMarkingPackageDirty = true);

	/**
	*	Rebuilds dependent Material Instance Editors
	*	@param	BaseMaterial	Material that MaterialInstance must be based on for Material Instance Editor to be rebuilt
	*/
	static UE_API void RebuildMaterialInstanceEditors(UMaterial* BaseMaterial);

	/**
	*	Rebuilds dependent Material Instance Editors
	*	@param	BaseMaterial	Material that MaterialInstance must be based on for Material Instance Editor to be rebuilt
	*/
	static UE_API void RebuildMaterialInstanceEditors(UMaterialFunction* BaseFunction);

	/**
	 *	Refresh an open Material Editor's preview from its source asset. Call after script-driven edits
	 *	so the editor reflects the change. No-op if no editor is open for the asset. Discards any
	 *	in-editor edits not yet applied back to the asset.
	 *	@param	Material	Material whose open editor should be refreshed.
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void RefreshMaterialEditor(UMaterial* Material);

	/**
	 *	Refresh an open Material Function Editor's preview from its source asset. Same semantics as
	 *	RefreshMaterialEditor.
	 *	@param	MaterialFunction	MaterialFunction whose open editor should be refreshed.
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void RefreshMaterialFunctionEditor(UMaterialFunction* MaterialFunction);

	/**
	 *	Refresh an open Material Instance Editor's parameter and inheritance views. Call after
	 *	script-driven edits so the editor reflects the change. No-op if no editor is open for
	 *	the instance.
	 *	@param	Instance	Material Instance whose open editor should be refreshed.
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void RefreshMaterialInstanceEditor(UMaterialInstanceConstant* Instance);

	///////////// MATERIAL EDITING

	/** returns the Expressions contained in this Material Function. Does not include nested Expressions. */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<UMaterialExpression*> GetMaterialFunctionExpressions(const UMaterialFunction* MaterialFunction);

	/** returns the Expressions contained in this Material. Does not include nested Expressions. */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<UMaterialExpression*> GetMaterialExpressions(const UMaterial* Material);

	/** Returns number of material expressions in the supplied material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API int32 GetNumMaterialExpressions(const UMaterial* Material);


	/** Delete all material expressions in the supplied material */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void DeleteAllMaterialExpressions(UMaterial* Material);

	/** Delete a specific expression from a material. Will disconnect from other expressions. */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void DeleteMaterialExpression(UMaterial* Material, UMaterialExpression* Expression);

	/** Delete all expressions not reachable from any material output. Call recompile afterwards. */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void DeleteUnusedExpressions(UMaterial* Material);

	/** 
	 *	Create a new material expression node within the supplied material 
	 *	@param	Material			Material asset to add an expression to
	 *	@param	ExpressionClass		Class of expression to add
	 *	@param	NodePosX			X position of new expression node
	 *	@param	NodePosY			Y position of new expression node
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API UMaterialExpression* CreateMaterialExpression(UMaterial* Material, TSubclassOf<UMaterialExpression> ExpressionClass, int32 NodePosX=0, int32 NodePosY=0);

	/** 
	 *	Duplicates the provided material expression adding it to the same material / material function, and copying parameters.
	 *  Note: Does not duplicate transient properties (Ex: GraphNode).
	 *
	 *	@param	Material			Material asset to add an expression to
	 *	@param	MaterialFunction	Specified if adding an expression to a MaterialFunction, used as Outer for new expression object
	 *	@param	SourceExpression	Expression to be duplicated
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API UMaterialExpression* DuplicateMaterialExpression(UMaterial* Material, UMaterialFunction* MaterialFunction, UMaterialExpression* Expression);

	/** 
	 *	Enable a particular usage for the supplied material (e.g. SkeletalMesh, ParticleSprite etc)
	 *	@param	Material			Material to change usage for
	 *	@param	Usage				New usage type to enable for this material
	 *	@param	bNeedsRecompile		Returned to indicate if material needs recompiling after this change
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing", meta = (DeprecatedFunction, DeprecationMessage = "Use SetBaseMaterialUsage instead"))
	static UE_API bool SetMaterialUsage(UMaterial* Material, EMaterialUsage Usage, bool& bNeedsRecompile);

	/**
	 *	Check if a particular usage is enabled for the supplied material (e.g. SkeletalMesh, ParticleSprite etc)
	 *	@param	Material			Material to check usage for
	 *	@param	Usage				Usage type to check for this material
	 */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API bool HasMaterialUsage(UMaterialInterface* Material, EMaterialUsage Usage);

	/** 
	 *	Enable or disable a particular usage for the supplied material
	 *	@param	Material			Material to change usage for
	 *	@param	Usage				Usage type
	 *	@param	bValue				Whether to enable the usage
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void SetBaseMaterialUsage(UMaterial* Material, EMaterialUsage Usage, bool bValue = true);

	/**
	 *	Check if a if the supplied material instance has an override for the particular usage
	 *	@param	MaterialInstance	Material instance to check override for
	 *	@param	Usage				Usage type to check for this material
	 */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static bool HasMaterialUsageOverride(UMaterialInstanceConstant* MaterialInstance, EMaterialUsage Usage);

	/** 
	 *	Set the material usage override for the supplied material instance
	 *	@param	MaterialInstance	Material instance to change usage for
	 *	@param	Usage				Usage type
	 *	@param	bOverride			Whether to enable the override (set to false to default to Parent setting)
	 *	@param	bValue				Whether to enable the usage (only if bOverride is true)
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void SetMaterialUsageOverride(UMaterialInstanceConstant* MaterialInstance, EMaterialUsage Usage, bool bOverride = true, bool bValue = true);

	/** 
	 *	Connect a material expression output to one of the material property inputs (e.g. diffuse color, opacity etc)
	 *	@param	FromExpression		Expression to make connection from
	 *	@param	FromOutputName		Name of output of FromExpression to make connection from
	 *	@param	Property			Property input on material to make connection to
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool ConnectMaterialProperty(UMaterialExpression* FromExpression, FString FromOutputName, EMaterialProperty Property);

	/**
	 *	Create connection between two material expressions
	 *	@param	FromExpression		Expression to make connection from
	 *	@param	FromOutputName		Name of output of FromExpression to make connection from. Leave empty to use first output.
	 *	@param	ToExpression		Expression to make connection to
	 *	@param	ToInputName			Name of input of ToExpression to make connection to. Leave empty to use first input.
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool ConnectMaterialExpressions(UMaterialExpression* FromExpression, FString FromOutputName, UMaterialExpression* ToExpression, FString ToInputName);

	/**
	 *	Disconnect the input pin of a material expression, removing whatever is currently connected to it.
	 *	@param	ToExpression	Expression whose input pin should be disconnected
	 *	@param	ToInputName		Name of the input pin to disconnect.
	 *	@return	true if a connection was broken, false if the pin was already disconnected or not found
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool DisconnectMaterialExpressions(UMaterialExpression* ToExpression, FString ToInputName);

	/**
	 *	Disconnect the expression node currently connected to a material property input (e.g. base color, roughness).
	 *	@param	Material	Material asset to modify
	 *	@param	Property	Property input on material to disconnect
	 *	@return	true if a connection was broken, false if the input was already disconnected
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool DisconnectMaterialProperty(UMaterial* Material, EMaterialProperty Property);

	/**
	 *	Trigger a recompile of a material. Must be performed after making changes to the graph to have changes reflected.
	 *  @return Any compiler error messages produced during compilation. Empty if compilation succeeded.
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API TArray<FString> RecompileMaterial(UMaterial* Material);

	/** Item complete callback for use when processing arrays of items. Allows higher level to tick UI etc. */
	DECLARE_DELEGATE(FOnItemComplete)

	/** 
	 *	Trigger a recompile of an array of materials.
	 * 	Faster than individual calls to RecompileMaterial().
	 *  Calls OnItemComplete after each material is processed.
	 */
	static UE_API void RecompileMaterials(TArray<UMaterial*>& Materials, FOnItemComplete const& OnItemComplete);

	/**
	 * Flattens shader type info into a flat list of individual shader entries,
	 * stripping the per-type grouping from FDebugShaderTypeInfo.
	 *
	 * @param InShaderTypeInfos  Grouped shader type information (e.g., from GetDebugShaderTypeInfo)
	 * @return Flat array of all individual FDebugShaderInfo across all types
	 */
	static UE_API TArray<FDebugShaderInfo> GetShaderInfoFromTypeInfo(const TArray<FDebugShaderTypeInfo>& InShaderTypeInfos);

	/**
	 *	Layouts the expressions in a grid pattern
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void LayoutMaterialExpressions(UMaterial* Material);

	/** Get the default scalar (float) parameter value from a Material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API float GetMaterialDefaultScalarParameterValue(UMaterial* Material, FName ParameterName);


	/** Get the default texture parameter value from a Material  */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API UTexture* GetMaterialDefaultTextureParameterValue(UMaterial* Material, FName ParameterName);

	/** Get the default vector parameter value from a Material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API FLinearColor GetMaterialDefaultVectorParameterValue(UMaterial* Material, FName ParameterName);

	/** Get the default static switch parameter value from a Material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API bool GetMaterialDefaultStaticSwitchParameterValue(UMaterial* Material, FName ParameterName);

	/** Get the set of selected nodes from an active material editor */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TSet<UObject*> GetMaterialSelectedNodes(UMaterial* Material);

	/** Get the node providing the output for a given material property from an active material editor */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API UMaterialExpression* GetMaterialPropertyInputNode(UMaterial* Material, EMaterialProperty Property);

	/** Get the node output name providing the output for a given material property from an active material editor */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API FString GetMaterialPropertyInputNodeOutputName(UMaterial* Material, EMaterialProperty Property);

	/** Get the array of input pin names for a material expression */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<FString> GetMaterialExpressionInputNames(UMaterialExpression* MaterialExpression);

	/** Get the array of input pin types for a material expression */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<int32> GetMaterialExpressionInputTypes(UMaterialExpression* MaterialExpression);

	/** Get the array of output pin names for a material expression */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<FString> GetMaterialExpressionOutputNames(UMaterialExpression* MaterialExpression);

	/** Get the set of nodes acting as inputs to a node from an active material editor */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<UMaterialExpression*> GetInputsForMaterialExpression(UMaterial* Material, UMaterialExpression* MaterialExpression);

	/** Get the set of nodes acting as inputs to a node in a material function graph */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<UMaterialExpression*> GetInputsForMaterialFunctionExpression(UMaterialFunction* MaterialFunction, UMaterialExpression* MaterialExpression);

	/** Get the output name of input node connected to MaterialExpression from an active material editor */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API bool GetInputNodeOutputNameForMaterialExpression(UMaterialExpression* MaterialExpression, UMaterialExpression* InputNode, FString& OutputName);

	/** Get the position of the MaterialExpression node. */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API void GetMaterialExpressionNodePosition(UMaterialExpression* MaterialExpression, int32& NodePosX, int32& NodePosY);

	/** Get the list of textures used by a material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing", meta = (DeprecatedFunction, DeprecationMessage = "Use GetMaterialUsedTextures instead since it works with Material Instances."))
	static UE_API TArray<UTexture*> GetUsedTextures(UMaterial* Material);

	/** Get the list of textures used by a material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API TArray<UTexture*> GetMaterialUsedTextures(UMaterialInterface* MaterialInterface);
	
	//////// MATERIAL FUNCTION EDITING

	/** Returns number of material expressions in the supplied material */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API int32 GetNumMaterialExpressionsInFunction(const UMaterialFunction* MaterialFunction);

	/**
	*	Create a new material expression node within the supplied material function
	*	@param	MaterialFunction	Material function asset to add an expression to
	*	@param	ExpressionClass		Class of expression to add
	*	@param	NodePosX			X position of new expression node
	*	@param	NodePosY			Y position of new expression node
	*/
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API UMaterialExpression* CreateMaterialExpressionInFunction(UMaterialFunction* MaterialFunction, TSubclassOf<UMaterialExpression> ExpressionClass, int32 NodePosX = 0, int32 NodePosY = 0);

	/** Delete all material expressions in the supplied material function */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void DeleteAllMaterialExpressionsInFunction(UMaterialFunction* MaterialFunction);

	/** Delete a specific expression from a material function. Will disconnect from other expressions. */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void DeleteMaterialExpressionInFunction(UMaterialFunction* MaterialFunction, UMaterialExpression* Expression);

	/**
	 *	Update a Material Function after edits have been made.
	 *	Will recompile any Materials that use the supplied Material Function.
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing", meta = (HidePin = "PreviewMaterial"))
	static UE_API void UpdateMaterialFunction(UMaterialFunctionInterface* MaterialFunction, UMaterial* PreviewMaterial = nullptr);

	/**
	 *	Update an array of Material Functions after edits have been made.
	 *	Will recompile any Materials that use the supplied Material Function.
	 *  Faster than individual calls to UpdateMaterialFunction().
	 *  Calls OnItemComplete after each material function is processed.
	 */
	static UE_API void UpdateMaterialFunctions(TArray<UMaterialFunctionInterface*>& MaterialFunctions, FOnItemComplete const& OnItemComplete);

	/**
	 *	Layouts the expressions in a grid pattern
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void LayoutMaterialFunctionExpressions(UMaterialFunction* MaterialFunction);

	//////// MATERIAL INSTANCE CONSTANT EDITING

	/** Set the parent Material or Material Instance to use for this Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void SetMaterialInstanceParent(UMaterialInstanceConstant* Instance, UMaterialInterface* NewParent);

	/** Clears all material parameters set by this Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void ClearAllMaterialInstanceParameters(UMaterialInstanceConstant* Instance);


	/** Get the current scalar (float) parameter value from a Material Instance */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API float GetMaterialInstanceScalarParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Set the scalar (float) parameter value for a Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceScalarParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, float Value, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);


	/** Get the current texture parameter value from a Material Instance */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API UTexture* GetMaterialInstanceTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Set the texture parameter value for a Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, UTexture* Value, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);


	/** Get the current texture parameter value from a Material Instance */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API URuntimeVirtualTexture* GetMaterialInstanceRuntimeVirtualTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Set the texture parameter value for a Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceRuntimeVirtualTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, URuntimeVirtualTexture* Value, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);


	/** Get the current texture parameter value from a Material Instance */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API USparseVolumeTexture* GetMaterialInstanceSparseVolumeTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Set the texture parameter value for a Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceSparseVolumeTextureParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, USparseVolumeTexture* Value, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);


	/** Get the current vector parameter value from a Material Instance */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API FLinearColor GetMaterialInstanceVectorParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Set the vector parameter value for a Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceVectorParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, FLinearColor Value, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Get the current static switch parameter value from a Material Instance */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API bool GetMaterialInstanceStaticSwitchParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Set the static switch parameter value for a Material Instance */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceStaticSwitchParameterValue(UMaterialInstanceConstant* Instance, FName ParameterName, bool Value, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter, bool bUpdateMaterialInstance = true);

	/**
	 *	Check whether a parameter is overridden in this Material Instance.
	 *	@param	Instance		Material Instance to query
	 *	@param	ParameterName	Name of the parameter
	 *	@param	Association		Parameter association type
	 *	@return Whether the parameter is overridden in this instance
	 */
	UFUNCTION(BlueprintPure, Category = "MaterialEditing")
	static UE_API bool IsMaterialInstanceParameterOverridden(UMaterialInstanceConstant* Instance, FName ParameterName, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/**
	 *	Set whether a parameter override is enabled in this Material Instance.
	 *	@param	Instance		Material Instance to modify
	 *	@param	ParameterName	Name of the parameter
	 *	@param	bOverride		Whether the parameter should be overridden
	 *	@param	Association		Parameter association type
	 *	@return Whether the operation succeeded (false if Instance is null or the parameter was not found)
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool SetMaterialInstanceParameterOverride(UMaterialInstanceConstant* Instance, FName ParameterName, bool bOverride, EMaterialParameterAssociation Association = EMaterialParameterAssociation::GlobalParameter);

	/** Called after making modifications to a Material Instance to recompile shaders etc. */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void UpdateMaterialInstance(UMaterialInstanceConstant* Instance);

	/** Gets all direct child mat instances */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void GetChildInstances(UMaterialInterface* Parent, TArray<FAssetData>& ChildInstances);

	/** Gets all materials that use a material function */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void GetMaterialsReferencingFunction(UMaterialFunction* InFunction, TArray<FAssetData>& OutMaterials);

	/** Gets all scalar parameter names */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void GetScalarParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames);

	/** Gets all vector parameter names */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void GetVectorParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames);

	/** Gets all texture parameter names */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void GetTextureParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames);

	/** Gets all static switch parameter names */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API void GetStaticSwitchParameterNames(UMaterialInterface* Material, TArray<FName>& ParameterNames);
	
	/**
	*	Returns the path of the asset where the parameter originated, as well as true/false if it was found
	*	@param	Material	The material or material instance you want to look up a parameter from
	*	@param	ParameterName		The parameter name
	*	@param	ParameterSource		The soft object path of the asset the parameter originates in 
	*	@return	Whether or not the parameter was found in this material
	*/
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool GetScalarParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource);

	/**
	*	Returns the path of the asset where the parameter originated, as well as true/false if it was found
	*	@param	Material	The material or material instance you want to look up a parameter from
	*	@param	ParameterName		The parameter name
	*	@param	ParameterSource		The soft object path of the asset the parameter originates in
	*	@return	Whether or not the parameter was found in this material
	*/
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool GetVectorParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource);

	/**
	*	Returns the path of the asset where the parameter originated, as well as true/false if it was found
	*	@param	Material	The material or material instance you want to look up a parameter from
	*	@param	ParameterName		The parameter name
	*	@param	ParameterSource		The soft object path of the asset the parameter originates in
	*	@return	Whether or not the parameter was found in this material
	*/
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool GetTextureParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource);

	/**
	 * Renames a material parameter group within the specified material.
	 *
	 * This function allows you to rename an existing parameter group in a material.
	 * It iterates all parameters within the material, finds all the one belonging
	 * to the `OldGroupName` group and switches those parameters to be in the `NewGroupName`
	 * group.  This function only affects parameters that belong to the specified group.   
	 * To remove the groups from the parameters the new group name can be 'None'.  If the 
	 * OldGroupName does not exist in the material, the function will return false.   If 
	 * the NewGroupName already exists, the parameters will be "merged" into the existing group.
	 *
	 * @param Material       The material asset in which the parameter group resides.
	 * @param OldGroupName   The current name of the parameter group to rename.
	 * @param NewGroupName   The new name to assign to the parameter group.
	 *
	 * @return true if the rename operation was successful; false otherwise.
	 * 
	 * @see RenameMaterialFunctionParameterGroup
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool RenameMaterialParameterGroup(UMaterial* Material, const FName OldGroupName, const FName NewGroupName);

	/**
	 * Renames a material parameter group within the specified material function.
	 *
	 * This function allows you to rename an existing parameter group in a material function.
	 * It iterates all parameters within the material function, finds all the one belonging
	 * to the `OldGroupName` group and switches those parameters to be in the `NewGroupName`
	 * group.  This function only affects parameters that belong to the specified group.
	 * To remove the groups from the parameters the new group name can be 'None'.  If the 
	 * OldGroupName does not exist in the material, the function will return false.   If the 
	 * NewGroupName already exists, the parameters will be "merged" into the existing group.
	 *
	 * @param MaterialFunction	The material function asset in which the parameter group resides.
	 * @param OldGroupName		The current name of the parameter group to rename.
	 * @param NewGroupName		The new name to assign to the parameter group.
	 *
	 * @return true if the rename operation was successful; false otherwise.
	 *
	 * @see RenameMaterialParameterGroup
	 */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool RenameMaterialFunctionParameterGroup(UMaterialFunctionInterface* MaterialFunction, const FName OldGroupName, const FName NewGroupName);

	/**
	*	Returns the path of the asset where the parameter originated, as well as true/false if it was found
	*	@param	Material	The material or material instance you want to look up a parameter from
	*	@param	ParameterName		The parameter name
	*	@param	ParameterSource		The soft object path of the asset the parameter originates in
	*	@return	Whether or not the parameter was found in this material
	*/
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API bool GetStaticSwitchParameterSource(UMaterialInterface* Material, const FName ParameterName, FSoftObjectPath& ParameterSource);

	/** Returns the number of shader types (including pipeline shader types) for the given material. */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API int32 GetNumShaderTypes(UMaterialInterface* Material);

	/** Returns a list of all the shaders contained in this material. */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API TArray<FDebugShaderInfo> ListShaders(UMaterialInterface* Material);
	
	/** Returns statistics about the given material */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API FMaterialStatistics GetStatistics(UMaterialInterface* Material);

	/** Returns any nanite override material for the given material */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	static UE_API UMaterialInterface* GetNaniteOverrideMaterial(UMaterialInterface* Material);

private:
	static void RecompileMaterialInternal(const TArray<UMaterial*>& Material, const FOnItemComplete* OnItemComplete);
	static void UpdateMaterialFunctionInternal(FMaterialUpdateContext& UpdateContext, UMaterialFunctionInterface* MaterialFunction, UMaterial* PreviewMaterial);
};

#undef UE_API
