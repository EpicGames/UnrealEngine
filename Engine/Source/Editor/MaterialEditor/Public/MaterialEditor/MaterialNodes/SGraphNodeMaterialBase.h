// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Rendering/RenderingCommon.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateShaderResource.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API MATERIALEDITOR_API

class FMaterialRenderProxy;
class FRHICommandListImmediate;
class FSlateRect;
class FWidgetStyle;
class SGraphPin;
class SOverlay;
class SVerticalBox;
class SWidget;
class UMaterialGraphNode;
struct FGeometry;
struct FSlateBrush;

typedef TSharedPtr<class FPreviewElement, ESPMode::ThreadSafe> FThreadSafePreviewPtr;

class FPreviewViewport : public ISlateViewport
{
public:
	FPreviewViewport(class UMaterialGraphNode* InNode);
	~FPreviewViewport();

	// ISlateViewport interface
	virtual TUniquePtr<UE::Slate::FPaintScope> Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled) override;
	virtual FIntPoint GetSize() const override;
	virtual class FSlateShaderResource* GetViewportRenderTargetTexture() const override {return NULL;}
	virtual bool RequiresVsync() const override {return false;}
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/** Material node to get expression preview from */
	UMaterialGraphNode* MaterialNode;
	/** Custom Slate Element to display the preview */
	FThreadSafePreviewPtr PreviewElement;
	/** The current UV we are hovered over, if any. */
	TOptional<FVector2D> HoveredUV;
private:
	/**
	 * Updates the expression preview render proxy from a graph node
	 */
	void UpdatePreviewNodeRenderProxy();
};

class FPreviewElement : public ICustomSlateElement
{
public:
	FPreviewElement();
	virtual ~FPreviewElement() = default;

	/**
	 * Sets up the canvas for rendering
	 *
	 * @param	InCanvasRect			Size of the canvas tile
	 * @param	InClippingRect			How to clip the canvas tile
	 * @param	InGraphNode				The graph node for the material preview
	 * @param	bInIsRealtime			Whether preview is using realtime values
	 * @param	bInShouldReadback		Whether we should readback the render target data or not
	 *
	 * @return	Whether there is anything to render
	 */
	bool BeginRenderingCanvas(const FIntRect& InCanvasRect, const FIntRect& InClippingRect, UMaterialGraphNode* InGraphNode, bool bInIsRealtime, bool bInShouldReadback);

	/**
	 * Updates the expression preview render proxy from a graph node on the render thread
	 */
	void UpdateExpressionPreview(UMaterialGraphNode* PreviewNode);

	/**
	 * Returns the readback value at the provided coordinated, unset if no readback data existss.
	 */
	 TOptional<FLinearColor> GetReadbackValue(const FVector2D& UV) const;

	 /**
	  * Clear any pending readbacks that may be in flight to ensure we don't read stale data later.
	  */
	 void ClearReadback();

private:
	/**
	 * ICustomSlateElement interface 
	 */
	virtual void Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs) override;

	void Readback_RenderThread(FRDGBuilder& GraphBuilder);

private:
	/** Render target that the canvas renders to */
	TUniquePtr<class FSlateMaterialPreviewRenderTarget> RenderTarget;
	/** Render proxy for the expression preview */
	FMaterialRenderProxy* ExpressionPreview = nullptr;
	/** Whether preview is using realtime values */
	bool bIsRealtime = false;

	/** Used on the render thread to determine if we should readback the data or not. */
	bool bShouldReadback = false;
	/** Used on the render thread to determine if we have a readback in flight or not. */
	bool bReadbackPending = false;
	/** Readback render target, different from the regular render target as we need float range */
	TUniquePtr<class FSlateMaterialPreviewRenderTarget> ReadbackRenderTarget;
	/** Readback primitive */
	TUniquePtr<class FRHIGPUTextureReadback> GPUTextureReadback;

	static constexpr int32 ReadbackDimension = 128;

	mutable UE::FMutex ReadbackDataGuard;
	TArray<FFloat16Color> ReadbackData;
};

template<>
struct TWidgetTypeTraits<class SGraphNodeMaterialBase>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeMaterialBase : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialBase){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode);

	// SGraphNode interface
	UE_API virtual void CreatePinWidgets() override;
	// End of SGraphNode interface

	// SNodePanel::SNode interface
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	// End of SNodePanel::SNode interface

	UMaterialGraphNode* GetMaterialGraphNode() const {return MaterialNode;}

	/* Populate a meta data tag with information about this graph node */
	UE_API virtual void PopulateMetaTag(class FGraphNodeMetaData* TagMeta) const override;

protected:
	// SGraphNode interface
	UE_API virtual void AddPin( const TSharedRef<SGraphPin>& PinToAdd ) override;
	UE_API virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	UE_API virtual TSharedRef<SWidget> CreateTitleRightWidget() override;

	UE_API virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	UE_API virtual void OnAdvancedViewChanged(const ECheckBoxState NewCheckedState) override;
	// End of SGraphNode interface

	/** Creates a preview viewport if necessary */
	UE_API TSharedRef<SWidget> CreatePreviewWidget();

	/** Returns visibility of Expression Preview viewport */
	UE_API EVisibility ExpressionPreviewVisibility() const;

	/** Returns text to over lay over the expression preview viewport */
	UE_API FText ExpressionPreviewOverlayText() const;

	/** Returns text to over lay as the tooltip value text */
	UE_API FText ExpressionPreviewValueText() const;

	/** Called when we click to preview node button */
	UE_API FReply OnTogglePreviewClicked();

	/** Show/hide Expression Preview */
	UE_API void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);

	/** hidden == unchecked, shown == checked */
	UE_API ECheckBoxState IsExpressionPreviewChecked() const;

	/** Up when shown, down when hidden */
	UE_API const FSlateBrush* GetExpressionPreviewArrow() const;

protected:
	/** Slate viewport for rendering preview via custom slate element */
	TSharedPtr<FPreviewViewport> PreviewViewport;

	/** Cached material graph node pointer to avoid casting */
	UMaterialGraphNode* MaterialNode;
};

#undef UE_API
