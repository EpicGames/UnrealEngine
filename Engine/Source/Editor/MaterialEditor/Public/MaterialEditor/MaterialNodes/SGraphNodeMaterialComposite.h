// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"
#include "Math/Vector2D.h"
#include "SNodePanel.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API MATERIALEDITOR_API

class SToolTip;
class SWidget;
class UEdGraph;
class UMaterialGraphNode_Composite;

//@TODO: This class is mostly C&P from UK2Node_Composite, consolidate composites to not be BP specific.
class SGraphNodeMaterialComposite : public SGraphNodeMaterialBase
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialComposite){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UMaterialGraphNode_Composite* InNode);

	// SGraphNode interface
	UE_API virtual void UpdateGraphNode() override;
	UE_API virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	// End of SGraphNode interface

	// SNodePanel::SNode interface
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	// End of SNodePanel::SNode interface
protected:
	UE_API virtual UEdGraph* GetInnerGraph() const;

private:
	UE_API FText GetPreviewCornerText() const;
	UE_API FText GetTooltipTextForNode() const;

	UE_API TSharedRef<SWidget> CreateNodeBody();

	/** Cached material graph node pointer to avoid casting */
	UMaterialGraphNode_Composite* CompositeNode;
};

#undef UE_API
