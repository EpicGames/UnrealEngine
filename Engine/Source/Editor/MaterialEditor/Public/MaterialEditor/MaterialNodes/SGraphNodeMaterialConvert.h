// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"
#include "SGraphPin.h"
#include "Types/SlateVector2.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MATERIALEDITOR_API

class FConvertDragDropOp;
class SConvertInnerPin;
class SGraphNodeMaterialConvert;
class SImage;
enum class EMaterialExpressionConvertType : uint8;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FConvertDragDropOp: Drag and Drop Operation used to form connections within the convert node
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FConvertDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FConvertDragDropOp, FDragDropOperation)

		FConvertDragDropOp(TSharedPtr<SConvertInnerPin> InSourcePin);

	//~ Begin FDragDropOperation interface
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	//~ End FDragDropOperation interface

	TSharedPtr<SConvertInnerPin> GetSourcePin() const;
	UE::Slate::FDeprecateVector2DResult GetScreenPosition() const;

protected:
	TSharedPtr<SConvertInnerPin> SourcePin;
	FVector2f ScreenPosition;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// SMaterialExpressionConvertGraphPin: The Outer pins that form connections to other material graph nodes
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class SMaterialExpressionConvertGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SMaterialExpressionConvertGraphPin){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

	void CreateInnerPins(TSharedRef<SGraphNodeMaterialConvert> InOwningGraphNodeWidget);

	const TArray<TSharedRef<SConvertInnerPin>>& GetInnerPins() const { return InnerPins; }

	bool IsHoveredOverPrimaryPin() const;
private:

	// Inner Pins, one for each component
	TArray<TSharedRef<SConvertInnerPin>> InnerPins;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// SConvertInnerPin: The inner pins used to route values with the convert node
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Class for the inner pins we'll use to route values in SGraphNodeMaterialConvert */
class SConvertInnerPin : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConvertInnerPin) {}
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		TSharedPtr<SGraphNodeMaterialConvert> InOwningNode,
		TSharedPtr<SMaterialExpressionConvertGraphPin> InOwningPin,
		bool bInIsInputPin, 
		int32 InPinIndex, 
		int32 InComponentIndex
	);

	//~ Begin SWidget interface
	virtual int32 OnPaint(
		const FPaintArgs& Args, 
		const FGeometry& AllottedGeometry, 
		const FSlateRect& MyCullingRect, 
		FSlateWindowElementList& OutDrawElements, 
		int32 LayerId, 
		const FWidgetStyle& InWidgetStyle, 
		bool bParentEnabled
	) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End SWidget interface

	const FSlateBrush* GetPinBrush() const;
	void CancelDragDrop();
	UE::Slate::FDeprecateVector2DResult GetPinCenterAbsolute() const;

	/** Adds a connection to WeakConnectedPins */
	void AddConnection(TSharedPtr<SConvertInnerPin> InOtherPin);

	/** Removes a connection from WeakConnectedPins */
	void RemoveConnection(TSharedPtr<SConvertInnerPin> InOtherPin);

	/** Clears WeakConnectedPins */
	void RemoveAllConnections();

	/** Calls into the owning node to break connections and refresh the material node */
	void BreakConnections();

	EVisibility GetDefaultValueVisibility() const;
	TOptional<float> GetDefaultValue() const;
	void SetDefaultValue(float InDefaultValue, ETextCommit::Type CommitType);

	EVisibility GetPinNameVisibility() const;
	FText GetPinName() const;

	bool IsInputPin() const { return bIsInputPin; }
	int32 GetPinIndex() const { return PinIndex; }
	int32 GetComponentIndex() const { return ComponentIndex; }

protected:
	// Input Data
	TWeakPtr<SGraphNodeMaterialConvert> WeakOwningNode;
	TWeakPtr<SMaterialExpressionConvertGraphPin> WeakOwningPin;
	bool bIsInputPin = false;
	int32 PinIndex = INDEX_NONE;
	int32 ComponentIndex = INDEX_NONE;

	TSharedPtr<SImage> PinImage;
	TSet<TWeakPtr<SConvertInnerPin>> WeakConnectedPins;
	mutable FVector2f CenterAbsolute;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// SGraphNodeMaterialConvert: Custom Slate widget for UMaterialExpressionConvert
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class SGraphNodeMaterialConvert : public SGraphNodeMaterialBase
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialConvert){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode);

	// Begin SGraphNode interface
	UE_API virtual void CreatePinWidgets() override;
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	UE_API virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;

	UE_API virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
	UE_API virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	UE_API virtual EVisibility IsAddPinButtonVisible() const override;
	// End SGraphNode interface

	UE_API FReply OnAddInputPinClicked();
	UE_API FReply OnAddOutputPinClicked();

	UE_API TSharedRef<SWidget> CreateAddPinContextMenu(bool bInputPin);
	UE_API void AddNewPin(bool bInputPin, EMaterialExpressionConvertType ConvertType);

	// Begin SWidget Interface
	UE_API virtual int32 OnPaint(
		const FPaintArgs& Args, 
		const FGeometry& AllottedGeometry, 
		const FSlateRect& MyCullingRect, 
		FSlateWindowElementList& OutDrawElements, 
		int32 LayerId, 
		const FWidgetStyle& InWidgetStyle, 
		bool bParentEnabled
	) const override;
	// End SWidget Interface

	UE_API void FormConnection(TSharedPtr<SConvertInnerPin> InnerPinA, TSharedPtr<SConvertInnerPin> InnerPinB);
	UE_API void BreakConnections(TSharedPtr<SConvertInnerPin> InnerPin);

	UE_API TOptional<float> GetDefaultValue(TSharedPtr<const SConvertInnerPin> InnerPin) const;
	UE_API void SetDefaultValue(TSharedPtr<SConvertInnerPin> InnerPin, const float InDefaultValue);

	void SetCurrentDragDropOp(TSharedPtr<FConvertDragDropOp> InDragDropOp) { CurrentDragDropOp = InDragDropOp; }
	TSharedPtr<FConvertDragDropOp> GetCurrentDragDropOp() const { return CurrentDragDropOp; }

protected:
	UE_API TSharedPtr<SConvertInnerPin> GetInnerPin(bool bInputPin, const int32 InPinIndex, const int32 InComponentIndex) const;
	UE_API void MakeConnectionCurve(
		const FGeometry& InAllottedGeometry,
		FSlateWindowElementList& OutDrawElements,
		int32& InOutLayerId, 
		const FVector2f& InCurveStart, 
		const FVector2f& InCurveEnd
	) const;

private:
	TSharedPtr<FConvertDragDropOp> CurrentDragDropOp;
};

#undef UE_API
