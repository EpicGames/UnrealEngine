// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"
#include "Text/HLSLSyntaxHighlighterMarshaller.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

class UMaterialExpressionCustom;

#define UE_API MATERIALEDITOR_API

template <>
struct TWidgetTypeTraits<class SGraphNodeMaterialCustom>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeMaterialCustom : public SGraphNodeMaterialBase
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialCustom){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode);

	UMaterialGraphNode* GetMaterialGraphNode() const {return MaterialNode;}

	UE_API FText GetHlslText() const;
	UE_API void OnCustomHlslTextCommited(const FText& InText, ETextCommit::Type InType);

protected:
	// SGraphNode interface
	UE_API virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	UE_API virtual void CreateAdvancedViewArrow(TSharedPtr<SVerticalBox> MainBox) override;
	UE_API virtual EVisibility AdvancedViewArrowVisibility() const override;
	UE_API virtual void OnAdvancedViewChanged( const ECheckBoxState NewCheckedState ) override;
	UE_API virtual ECheckBoxState IsAdvancedViewChecked() const override;
	UE_API virtual const FSlateBrush* GetAdvancedViewArrow() const override;
	// End of SGraphNode interface

private:
	UE_API EVisibility CodeVisibility() const;
	UE_API EVisibility PixelDiscardVisibility() const;

protected:
	/** Shader text box */
	TSharedPtr<FHLSLSyntaxHighlighterMarshaller> SyntaxHighlighter;
	TSharedPtr<SMultiLineEditableTextBox> ShaderTextBox;
	TWeakObjectPtr<UMaterialExpressionCustom> WeakCustomExpression;
};

#undef UE_API
