// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"
#include "Materials/MaterialExpressionOperator.h"

#include "Components/HorizontalBox.h"
#include "KismetPins/SGraphPinNum.h"
#include "Widgets/Input/SButton.h"

#define UE_API MATERIALEDITOR_API

template<typename NumericType>
/** Custom SGraphPin for handling array of inputs */
class SMaterialExpressionOperatorGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SMaterialExpressionOperatorGraphPin) {}
		SLATE_ARGUMENT(int32, ArrayIndex)
		SLATE_ARGUMENT(UMaterialExpressionOperator*, OperatorExpression)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override;
	TOptional<NumericType> GetNumericValue() const;
	void SetNumericValue(NumericType NewValue, ETextCommit::Type CommitInfo);

private:
	int32 ArrayIndex;
	UMaterialExpressionOperator* OperatorExpression;
};


class SGraphNodeMaterialOperator : public SGraphNodeMaterialBase
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialOperator) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UMaterialGraphNode* InNode);

protected:
	// SGraphNode interface
	UE_API virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin) const override;
	UE_API virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	// End of SGraphNode interface

private:
	/** Get the Material Expression */
	UE_API UMaterialExpressionOperator* GetMaterialExpression() const;

	/** Event for adding a pin */
	UE_API FReply OnAddPinClicked();

	/** Button on Operator Graph Node for Operator Selection Menu */
	TWeakPtr<SButton> OperatorButton;

	/** Event for Operator Selection Menu */
	UE_API FReply OnOperatorMenuClicked();

	/** Create Operator Selection Menu */
	UE_API TSharedRef<SWidget> BuildOperatorMenu();

	/** Event for selecting operator */
	UE_API void OnOperatorMenuSelected(EMaterialExpressionOperatorKind NewKind);

	/** Get Operator Display Name */
	UE_API FText GetOperatorDisplayName() const;
};

#undef UE_API
