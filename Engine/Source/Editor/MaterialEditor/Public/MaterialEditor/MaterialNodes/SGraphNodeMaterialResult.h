// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API MATERIALEDITOR_API

class FGraphNodeMetaData;
class UMaterialGraphNode_Root;

template <>
struct TWidgetTypeTraits<class SGraphNodeMaterialResult>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeMaterialResult : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialResult){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UMaterialGraphNode_Root* InNode);

	// SGraphNode interface
	UE_API virtual void CreatePinWidgets() override;
	UE_API virtual void PopulateMetaTag(FGraphNodeMetaData* TagMeta) const override;
	// End of SGraphNode interface

	// SNodePanel::SNode interface
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	// End of SNodePanel::SNode interface

private:
	UMaterialGraphNode_Root* RootNode;
};

#undef UE_API
