// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API MATERIALEDITOR_API

class UEdGraphPin;

template <>
struct TWidgetTypeTraits<class SGraphPinMaterialInput>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinMaterialInput : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinMaterialInput) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual FSlateColor GetPinColor() const override;
	//~ End SGraphPin Interface

};

#undef UE_API
