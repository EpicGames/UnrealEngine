// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Rendering/RenderingCommon.h"
#include "SGraphPin.h"

#define UE_API MATERIALEDITOR_API

class UMaterialGraphNode;
struct FSubstrateMaterialCompilationOutput;

struct FSubstrateWidget
{
	static UE_API const TSharedRef<SWidget> ProcessOperator(const FSubstrateMaterialCompilationOutput& CompilationOutput);
	static UE_API const TSharedRef<SWidget> ProcessOperator(const FSubstrateMaterialCompilationOutput& CompilationOutput, const TArray<FGuid>& InGuid);
	static UE_API void GetPinColor(TSharedPtr<SGraphPin>& Out, const UMaterialGraphNode* InNode);
	static UE_API FLinearColor GetConnectionColor();
	static UE_API bool HasInputSubstrateType(const UEdGraphPin* InPin);
	static UE_API bool HasOutputSubstrateType(const UEdGraphPin* InPin);
};

#undef UE_API
