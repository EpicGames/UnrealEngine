// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;
class UClass;
class UEdGraph;
struct FEdGraphSchemaAction;
struct FInputChord;

UE_DECLARE_TCOMMANDS(class FMaterialEditorCommands, MATERIALEDITOR_API)

/**
 * Unreal material editor actions
 */
class FMaterialEditorCommands : public TCommands<FMaterialEditorCommands>
{

public:
	FMaterialEditorCommands() : TCommands<FMaterialEditorCommands>
	(
		"MaterialEditor", // Context name for fast lookup
		NSLOCTEXT("Contexts", "MaterialEditor", "Material Editor"), // Localized context name for displaying
		NAME_None, FAppStyle::GetAppStyleSetName()
	)
	{
	}
	
	/**
	 * Material Editor Commands
	 */
	
	/** Applys the following material to the world */
	TSharedPtr< FUICommandInfo > Apply;
	
	/** Flattens the material to a texture for mobile devices */
	TSharedPtr< FUICommandInfo > Flatten;

	/**
	 * Material Instance Editor Commands
	 */

	/** Toggles between showing all the material parameters or not */
	TSharedPtr< FUICommandInfo > ShowAllMaterialParameters;

	/**
	 * Preview Pane Commands
	 */

	/** Sets the preview mesh to a cylinder */
	TSharedPtr< FUICommandInfo > SetCylinderPreview;
	
	/** Sets the preview mesh to a sphere */
	TSharedPtr< FUICommandInfo > SetSpherePreview;
	
	/** Sets the preview mesh to a plane */
	TSharedPtr< FUICommandInfo > SetPlanePreview;
	
	/** Sets the preview mesh to a cube */
	TSharedPtr< FUICommandInfo > SetCubePreview;
	
	/** Sets the preview mesh to a shader ball */
	TSharedPtr< FUICommandInfo > SetShaderBallPreview;
	
	/** Sets the preview mesh to the current selection in the level editor */
	TSharedPtr< FUICommandInfo > SetPreviewMeshFromSelection;

	/** Set the material slots of the PreviewMesh to apply this material to */
	TSharedPtr< FUICommandInfo > SelectPreviewMeshMaterialSlots;
	
	/** Toggles the preview pane's background */
	TSharedPtr< FUICommandInfo > TogglePreviewBackground;


	/**
	 * Canvas Commands
	 */
	
	/** Moves the canvas camera to the home position */
	TSharedPtr< FUICommandInfo > CameraHome;
	
	/** Removes any unused nodes */
	TSharedPtr< FUICommandInfo > CleanUnusedExpressions;

	/** Fixes up broken sampler nodes */
	TSharedPtr<FUICommandInfo> FixupMismatchedSamplers;
	
	/** Shows or hides unused connectors */
	TSharedPtr< FUICommandInfo > ShowHideConnectors;
	
	/** Toggles live updating of the preview material. */
	TSharedPtr< FUICommandInfo > ToggleLivePreview;
	
	/** Fade nodes which are not connected to the selected nodes. */
	TSharedPtr< FUICommandInfo > ToggleHideUnrelatedNodes;

	/** Toggles real time expression nodes */
	TSharedPtr< FUICommandInfo > ToggleRealtimeExpressions;
	
	/** Always refresh all previews when enabled */
	TSharedPtr< FUICommandInfo > AlwaysRefreshAllPreviews;

	/** Toggles the material stats on the canvas pane */
	TSharedPtr< FUICommandInfo > ToggleMaterialStats;

	/** Shows material stats and errors for multiple shader platforms. */
	TSharedPtr< FUICommandInfo > TogglePlatformStats;
	
	/** Creates a new comment node */
	TSharedPtr< FUICommandInfo > NewComment;
	
	/** Uses the texture in the content browser for the selected node */
	TSharedPtr< FUICommandInfo > UseCurrentTexture;
	
	/** Pastes the copied items at the current location */
	TSharedPtr< FUICommandInfo > MatertialPasteHere;
	
	/** Converts selected objects to parameters */
	TSharedPtr< FUICommandInfo > ConvertObjects;

	/** Convert the selected parameters from 'float' to 'double' */
	TSharedPtr< FUICommandInfo > PromoteToDouble;

	/** Convert the selected parameters from 'double' to 'float' */
	TSharedPtr< FUICommandInfo > PromoteToFloat;

	/** Converts selected texture type into another */
	TSharedPtr< FUICommandInfo > ConvertToTextureObjects;
	TSharedPtr< FUICommandInfo > ConvertToTextureSamples;
	
	/** Converts selected objects to constants */
	TSharedPtr< FUICommandInfo > ConvertToConstant;

	/** Local variables select helpers */
	TSharedPtr< FUICommandInfo > SelectNamedRerouteDeclaration;
	TSharedPtr< FUICommandInfo > SelectNamedRerouteUsages;
	TSharedPtr<FUICommandInfo> CreateRerouteUsageFromDeclaration;

	/** Conversion between reroute nodes and local variables */
	TSharedPtr< FUICommandInfo > ConvertRerouteToNamedReroute;
	TSharedPtr< FUICommandInfo > ConvertNamedRerouteToReroute;

	/** Stops a node from being previewed in the viewport */
	TSharedPtr< FUICommandInfo > StopPreviewNode;
	
	/** Makes the selected node be previewed in the viewport */
	TSharedPtr< FUICommandInfo > StartPreviewNode;

	/** Previews the currently-hovered node */
	TSharedPtr< FUICommandInfo > PreviewHoveredNode;
	
	/** Enables realtime previewing of this node */
	TSharedPtr< FUICommandInfo > EnableRealtimePreviewNode;
	
	/** Disables realtime previewing of this node */
	TSharedPtr< FUICommandInfo > DisableRealtimePreviewNode;
	
	/** Breaks all outgoing links on the selected node */
	TSharedPtr< FUICommandInfo > BreakAllLinks;
	
	/** Duplicates all selected objects */
	TSharedPtr< FUICommandInfo > DuplicateObjects;
	
	/** Deletes all selected objects */
	TSharedPtr< FUICommandInfo > DeleteObjects;
	
	/** Selects all nodes that use the selected node's outgoing links */
	TSharedPtr< FUICommandInfo > SelectDownstreamNodes;
	
	/** Selects all nodes that use the selected node's incoming links */
	TSharedPtr< FUICommandInfo > SelectUpstreamNodes;
	
	/** Removes the selected expression from your favorites */
	TSharedPtr< FUICommandInfo > RemoveFromFavorites;
	
	/** Adds the selected expression to your favorites */
	TSharedPtr< FUICommandInfo > AddToFavorites;
	
	/** Deletes the selected link */
	TSharedPtr< FUICommandInfo > BreakLink;

	/** Forces a refresh of all previews */
	TSharedPtr< FUICommandInfo > ForceRefreshPreviews;

	/** Finds expressions in current material */
	TSharedPtr< FUICommandInfo > FindInMaterial;

	/** Create component mask node */
	TSharedPtr< FUICommandInfo > CreateComponentMaskNode;

	/** Promote pin to parameter */
	TSharedPtr< FUICommandInfo > PromoteToParameter;
	
	/** Reset pin to default value */
	TSharedPtr< FUICommandInfo > ResetToDefault;

	/** Delete Pin */
	TSharedPtr< FUICommandInfo > DeletePin;

	/** Create slab node */
	TSharedPtr< FUICommandInfo > CreateSlabNode;

	/** Create horizontal mix node */
	TSharedPtr< FUICommandInfo > CreateHorizontalMixNode;

	/** Create vertical layer node */
	TSharedPtr< FUICommandInfo > CreateVerticalLayerNode;

	/** Create weight node */
	TSharedPtr< FUICommandInfo > CreateWeightNode;

	/** Create select node */
	TSharedPtr< FUICommandInfo > CreateSelectNode;

	TSharedPtr< FUICommandInfo > QualityLevel_All;
	TSharedPtr< FUICommandInfo > QualityLevel_Epic;
	TSharedPtr< FUICommandInfo > QualityLevel_High;
	TSharedPtr< FUICommandInfo > QualityLevel_Medium;
	TSharedPtr< FUICommandInfo > QualityLevel_Low;

	TSharedPtr< FUICommandInfo > FeatureLevel_All;
	TSharedPtr< FUICommandInfo > FeatureLevel_Mobile;
	TSharedPtr< FUICommandInfo > FeatureLevel_SM5;
	TSharedPtr< FUICommandInfo > FeatureLevel_SM6;

	/**
	 * Initialize commands
	 */
	MATERIALEDITOR_API virtual void RegisterCommands() override;

public:
};

//////////////////////////////////////////////////////////////////////////
// FExpressionSpawnInfo

class FExpressionSpawnInfo
{
public:
	/** Constructor */
	FExpressionSpawnInfo(UClass* InMaterialExpressionClass) : MaterialExpressionClass(InMaterialExpressionClass) {}

	/** Holds the UI Command to verify chords for this action are held */
	TSharedPtr< FUICommandInfo > CommandInfo;

	/**
	 * Creates an action to be used for placing a node into the graph
	 *
	 * @param	InDestGraph		The graph the action should be created for
	 *
	 * @return					A fully prepared action containing the information to spawn the node
	 */
	TSharedPtr< FEdGraphSchemaAction > GetAction(UEdGraph* InDestGraph);

	UClass* GetClass() {return MaterialExpressionClass;}

private:
	/** Type of expression to spawn */
	UClass* MaterialExpressionClass;
};

//////////////////////////////////////////////////////////////////////////
// FMaterialEditorSpawnNodeCommands

/** Handles spawn node commands for the Material Editor */
class FMaterialEditorSpawnNodeCommands : public TCommands<FMaterialEditorSpawnNodeCommands>
{
public:
	/** Constructor */
	FMaterialEditorSpawnNodeCommands()
		: TCommands<FMaterialEditorSpawnNodeCommands>( TEXT("MaterialEditorSpawnNodes"), NSLOCTEXT("Contexts", "MaterialEditor_SpawnNodes", "Material Editor - Spawn Nodes"), NAME_None, FAppStyle::GetAppStyleSetName() )
	{
	}	

	/** TCommands interface */
	virtual void RegisterCommands() override;

	/**
	 * Returns a graph action assigned to the passed in chord
	 *
	 * @param InChord		The chord to use for lookup
	 * @param InDestGraph	The graph to create the graph action for, used for validation purposes and to link any important node data to the graph
	 */
	TSharedPtr< FEdGraphSchemaAction > GetGraphActionByChord(FInputChord& InChord, UEdGraph* InDestGraph) const;

	const TSharedPtr<const FInputChord> GetChordByClass(UClass* MaterialExpressionClass) const;

private:
	/** An array of all the possible commands for spawning nodes */
	TArray< TSharedPtr< class FExpressionSpawnInfo > > NodeCommands;
};
