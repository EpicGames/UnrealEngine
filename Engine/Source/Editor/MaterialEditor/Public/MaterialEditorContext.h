// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MaterialEditorContext.generated.h"

class IMaterialEditor;

UCLASS(MinimalAPI)
class UMaterialEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<IMaterialEditor> MaterialEditor;
};
