// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "MaterialEditor.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

#define UE_API MATERIALEDITOR_API

struct FMaterialEditorApplicationModes
{
	// Mode identifiers
	static UE_API const FName StandardMaterialEditorMode;
	static FText GetLocalizedMode(const FName InMode)
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add(StandardMaterialEditorMode, NSLOCTEXT("MaterialEditor", "StandardMaterialEditorMode", "Graph"));
		}

		check(InMode != NAME_None);
		const FText* OutDesc = LocModes.Find(InMode);
		check(OutDesc);
		return *OutDesc;
	}
	static UE_API TSharedPtr<FTabManager::FLayout> GetDefaultEditorLayout(TSharedPtr<class FMaterialEditor> InMaterialEditor);
private:
	FMaterialEditorApplicationModes() {}
};

// Even though we currently only have one mode of operation, we still need a application mode to handle factory tab spawning
class FMaterialEditorApplicationMode : public FApplicationMode
{
public:
	UE_API FMaterialEditorApplicationMode(TSharedPtr<class FMaterialEditor> InMaterialEditor);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
public:

protected:
	TWeakPtr<FMaterialEditor> MyMaterialEditor;

	// Set of spawnable tabs handled by workflow factories
	FWorkflowAllowedTabSet MaterialEditorTabFactories;
};

#undef UE_API
