// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Toolkits/AssetEditorToolkit.h"

class IMaterialEditor;
class UMaterial;
class UMaterialFunction;
class UMaterialFunctionInstance;
class UMaterialInstance;
class UMaterialInterface;
struct FMaterialParameterInfo;

extern const FName MaterialEditorAppIdentifier;
extern const FName MaterialInstanceEditorAppIdentifier;

/**
 * Material editor module interface
 */
class IMaterialEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/**
	 * Creates a new material editor, either for a material or a material function
	 */
	virtual TSharedRef<IMaterialEditor> CreateMaterialEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterial* Material ) = 0;
	virtual TSharedRef<IMaterialEditor> CreateMaterialEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterialFunction* MaterialFunction ) = 0;
	virtual TSharedRef<IMaterialEditor> CreateMaterialInstanceEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterialInstance* MaterialInstance ) = 0;
	virtual TSharedRef<IMaterialEditor> CreateMaterialInstanceEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMaterialFunctionInstance* MaterialFunction ) = 0;

	/**
	 * Retrieves all visible parameters within the material.
	 *
	 * @param	Material			The material to retrieve the parameters from.
	 * @param	MaterialInstance	The material instance that contains all parameter overrides.
	 * @param	VisibleExpressions	The array that will contain the name's of the visible parameter expressions.
	 */
	virtual void GetVisibleMaterialParameters(const class UMaterial* Material, class UMaterialInstance* MaterialInstance, TArray<FMaterialParameterInfo>& VisibleExpressions) = 0;

	/** Delegates to be called to extend the material menus */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<FExtender>, FMaterialMenuExtender, const TSharedRef<FUICommandList>);
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<FExtender>, FMaterialMenuExtender_MaterialInterface, const UMaterialInterface*);
	virtual TArray<FMaterialMenuExtender>& GetAllMaterialCanvasMenuExtenders() {return MaterialCanvasMenuExtenders;}
	virtual TArray<FMaterialMenuExtender_MaterialInterface>& GetAllMaterialDragDropContextMenuExtenders() {return MaterialInheritanceMenuExtenders;}

	/** Delegate to be called when a Material Editor is created, for toolbar, tab, and menu extension **/
	DECLARE_EVENT_OneParam(IMaterialEditorModule, FMaterialEditorOpenedEvent, TWeakPtr<IMaterialEditor>);
	virtual FMaterialEditorOpenedEvent& OnMaterialEditorOpened() { return MaterialEditorOpenedEvent; };

	/** Delegate to be called when a Material Function Editor is created, for toolbar, tab, and menu extension **/
	DECLARE_EVENT_OneParam(IMaterialEditorModule, FMaterialFunctionEditorOpenedEvent, TWeakPtr<IMaterialEditor>);
	virtual FMaterialFunctionEditorOpenedEvent& OnMaterialFunctionEditorOpened() { return MaterialFunctionEditorOpenedEvent; };

	/** Delegate to be called when a Material Instance Editor is created, for toolbar, tab, and menu extension **/
	DECLARE_EVENT_OneParam(IMaterialEditorModule, FMaterialInstanceEditorOpenedEvent, TWeakPtr<IMaterialEditor>);
	virtual FMaterialInstanceEditorOpenedEvent& OnMaterialInstanceEditorOpened() { return MaterialInstanceEditorOpenedEvent; };

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IMaterialEditorModule& Get()
	{
		static const FName ModuleName("MaterialEditor");
		return FModuleManager::LoadModuleChecked<IMaterialEditorModule>(ModuleName);
	}

private:
	/** All extender delegates for the material menus */
	TArray<FMaterialMenuExtender> MaterialCanvasMenuExtenders;
	TArray<FMaterialMenuExtender_MaterialInterface> MaterialInheritanceMenuExtenders;

	FMaterialEditorOpenedEvent MaterialEditorOpenedEvent;
	FMaterialFunctionEditorOpenedEvent MaterialFunctionEditorOpenedEvent;
	FMaterialInstanceEditorOpenedEvent MaterialInstanceEditorOpenedEvent;
};
