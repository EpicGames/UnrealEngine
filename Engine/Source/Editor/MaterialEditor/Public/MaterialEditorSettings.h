// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "MaterialEditorSettings.generated.h"

/**
 * Enumerates offline shader compiler for the material editor
 */
UENUM()
enum class EOfflineShaderCompiler : uint8
{
	Mali UMETA(DisplayName = "Mali Offline Compiler"),
	Adreno UMETA(DisplayName = "Adreno Offline Compiler")
};

/**
 * Enumerates background for the material editor UI viewport (similar to ETextureEditorBackgrounds)
 */
UENUM()
enum class EBackgroundType : uint8
{
	SolidColor UMETA(DisplayName = "Solid Color"),
	Checkered UMETA(DisplayName = "Checkered")
};

USTRUCT()
struct FCheckerboardSettings
{
	GENERATED_BODY()
	
	// The first color of the checkerboard.
	UPROPERTY(EditAnywhere, Category = "Checkerboard")
	FColor ColorOne;

	// The second color of the checkerboard
	UPROPERTY(EditAnywhere, Category = "Checkerboard")
	FColor ColorTwo;

	// The size of the checkered tiles
	UPROPERTY(EditAnywhere, meta = (ClampMin = "2", ClampMax = "4096"), Category = "Checkerboard")
	int32 Size;

	FCheckerboardSettings()
		: ColorOne(FColor(128, 128, 128))
		, ColorTwo(FColor(64, 64, 64))
		, Size(32)
	{

	}
};

inline bool operator==(const FCheckerboardSettings& Lhs, const FCheckerboardSettings& Rhs)
{
	return Lhs.ColorOne == Rhs.ColorOne
		&& Lhs.ColorTwo == Rhs.ColorTwo
		&& Lhs.Size == Rhs.Size;
}

inline bool operator!=(const FCheckerboardSettings& Lhs, const FCheckerboardSettings& Rhs)
{
	return !operator==(Lhs, Rhs);
}

USTRUCT()
struct FPreviewBackgroundSettings
{
	GENERATED_BODY()
	
	// If true, displays a border around the texture (configured via the material editor)
	UPROPERTY()
	bool bShowBorder;

	// Color to use for the border, if enabled
	UPROPERTY(EditAnywhere, Category = "Background")
	FColor BorderColor;

	// The type of background to show (configured via the material editor)
	UPROPERTY()
	EBackgroundType BackgroundType;

	// The color used as the background of the preview
	UPROPERTY(EditAnywhere, Category = "Background")
	FColor BackgroundColor;

	UPROPERTY(EditAnywhere, Category = "Background")
	FCheckerboardSettings Checkerboard;

	// Note: For now these defaults match the historical material editor behavior, not the texture editor's defaults
	FPreviewBackgroundSettings()
		: bShowBorder(false)
		, BorderColor(FColor::White)
		, BackgroundType(EBackgroundType::SolidColor)
		, BackgroundColor(FColor::Black)
	{

	}

};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UMaterialEditorSettings : public UObject
{
	GENERATED_BODY()

public:

	/** The configuration used in the Material Asset Wizard when creating a new Material. If not set, Content Browser uses the old factory without wizard. */
	UPROPERTY(config, EditAnywhere, Category = Wizard, meta=(AllowedClasses="/Script/UserAssetTagsEditor.TaggedAssetBrowserConfiguration"))
	FSoftObjectPath MaterialWizardConfiguration;

	/** The configuration used in the Material Asset Wizard when creating a new Material Instance. If not set, Content Browser uses the old factory without wizard. */
	UPROPERTY(config, EditAnywhere, Category = Wizard, meta=(AllowedClasses="/Script/UserAssetTagsEditor.TaggedAssetBrowserConfiguration"))
	FSoftObjectPath MaterialInstanceWizardConfiguration;
	
	/**
	Allow ignoring compilation errors of platform shaders and derived materials.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Editor Defaults", meta = (DisplayName = "Allow Ignoring Compilation Errors"))
	bool bAllowIgnoringCompilationErrors = true;

	/** The amount of weight placed on the search items title". */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Node Title Weight"))
	float ContextMenuNodeTitleWeight = 25.0f;

	/** The amount of weight placed on search items keyword. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Keyword Weight"))
	float ContextMenuKeywordWeight = 30.0f;

	/**  amount of weight placed on description that match what the user has typed in. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Menu description Weight"))
	float ContextMenuDescriptionWeight = 4.0f;

	/** The amount of weight placed on categories that match what the user has typed. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Category Weight"))
	float ContextMenuCategoryWeight = 4.0f;

	/** The multiplier given if there is an exact localized match to the search. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Whole Match Localized Weight Multiplier"))
	float ContextMenuWholeMatchLocalizedWeightMultiplier = 0.5f;

	/** The multiplier given if there is an exact match to the search term. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Whole Match Weight Multiplier"))
	float ContextMenuWholeMatchWeightMultiplier = 0.5f;

	/** The multiplier given if the keyword starts with a term the user typed in. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Starts With Bonus Weight Multiplier"))
	float ContextMenuStartsWithBonusWeightMultiplier = 4.0f;

	/** A multiplier for how much weight to give something based on the percentage match it is. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Percentage Match Weight Multiplier"))
	float ContextMenuPercentageMatchWeightMultiplier = 1.0f;

	/** Increasing this weight will make shorter words preferred. */
	UPROPERTY(config, EditAnywhere, Category = "Context Menu", meta = (DisplayName = "Shorter Match Weight"))
	float ContextMenuShorterMatchWeight = 10.0f;

	/**
	Offline shader compiler to use.
	Mali Offline Compiler: Official website address: https://developer.arm.com/products/software-development-tools/graphics-development-tools/mali-offline-compiler/downloads
	Adreno Offline Compiler: Official website address: https://developer.qualcomm.com/software/adreno-gpu-sdk/tools
	*/
	UPROPERTY(config, EditAnywhere, Category = "Offline Shader Compilers", meta = (DisplayName = "Offline Shader Compiler"))
	EOfflineShaderCompiler OfflineCompiler;
	
	/**
	Path to user installed shader compiler that can be used by the material editor to compile and extract shader informations for Android platforms.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Offline Shader Compilers", meta = (DisplayName = "Offline Compiler Path"))
	FFilePath OfflineCompilerPath;

	/**
	The GPU target if the offline shader compiler needs one (Adreno GPU only).
	*/
	UPROPERTY(config, EditAnywhere, Category = "Offline Shader Compilers", meta = (DisplayName = "GPU Target", EditCondition = "OfflineCompiler == EOfflineShaderCompiler::Adreno"))
	FString GPUTarget;

	/**
	Whether to save offline compiler stats files to Engine\Shaders
	*/
	UPROPERTY(config, EditAnywhere, Category = "Offline Shader Compilers", meta = (DisplayName = "Save Compiler Stats Files", EditCondition = "OfflineCompiler == EOfflineShaderCompiler::Adreno"))
	bool bSaveCompilerStatsFiles = false;

	/**
	Whether to dump stats only or all information to file (Adreno GPU only).
	*/
	UPROPERTY(config, EditAnywhere, Category = "Offline Shader Compilers", meta = (DisplayName = "Dump All Stats To Compiler Stats Files", EditCondition = "OfflineCompiler == EOfflineShaderCompiler::Adreno"))
	bool bDumpAll;

protected:
	// The width (in pixels) of the preview viewport when a material editor is first opened
	UPROPERTY(config, EditAnywhere, meta=(ClampMin=1, ClampMax=4096), Category="User Interface Domain")
	int32 DefaultPreviewWidth = 250;

	// The height (in pixels) of the preview viewport when a material editor is first opened
	UPROPERTY(config, EditAnywhere, meta=(ClampMin=1, ClampMax=4096), Category="User Interface Domain")
	int32 DefaultPreviewHeight = 250;

public:
	// Configures the background shown behind the UI material preview
	UPROPERTY(config, EditAnywhere, Category = "User Interface Domain")
	FPreviewBackgroundSettings PreviewBackground;

	FIntPoint GetPreviewViewportStartingSize() const
	{
		return FIntPoint(DefaultPreviewWidth, DefaultPreviewHeight);
	}

#if WITH_EDITOR
	// Allow listening for changes just to this settings object without having to listen to all UObjects
	FSimpleMulticastDelegate OnPostEditChange;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);
		OnPostEditChange.Broadcast();
	}
#endif // WITH_EDITOR
};
