// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "EdGraph/EdGraphNode.h"
#include "MaterialShared.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

#define UE_API MATERIALEDITOR_API

class IAssetReferenceFilter;
class IMaterialEditor;
class UMaterial;
class UMaterialExpressionComment;
class UMaterialExpressionComposite;
class UMaterialExpressionFunctionInput;
class UMaterialInstance;
class UMaterialInterface;
struct FGraphActionMenuBuilder;
struct FMaterialParameterInfo;

//////////////////////////////////////////////////////////////////////////
// FMaterialEditorUtilities

class FGetVisibleMaterialParametersFunctionState
{
public:

	FGetVisibleMaterialParametersFunctionState(UMaterialExpressionMaterialFunctionCall* InFunctionCall) :
		FunctionCall(InFunctionCall)
	{
		if (FunctionCall)
		{
			StackParameterInfo = FunctionCall->FunctionParameterInfo;
		}
	}

	class UMaterialExpressionMaterialFunctionCall* FunctionCall;
	TArray<FMaterialExpressionKey> ExpressionStack;
	TSet<FMaterialExpressionKey> VisitedExpressions;
	FMaterialParameterInfo StackParameterInfo;
};

class FMaterialEditorUtilities
{
public:
	/**
	 * Creates a new material expression of the specified class on the material represented by a graph.
	 *
	 * @param	Graph				Graph representing a material.
	 * @param	NewExpressionClass	The type of material expression to add.  Must be a child of UMaterialExpression.
	 * @param	NodePos				Position of the new node.
	 * @param	bAutoSelect			If true, deselect all expressions and select the newly created one.
	 * @param	bAutoAssignResource	If true, assign resources to new expression.
	 */
	static UE_API UMaterialExpression* CreateNewMaterialExpression(const class UEdGraph* Graph, UClass* NewExpressionClass, const UE::Slate::FDeprecateVector2DParameter& NodePos, bool bAutoSelect, bool bAutoAssignResource);

	/**
	 * Creates a new material expression of the specified class on the material represented by a graph.
	 *
	 * @param	Graph					Graph representing a material.
	 * @param	NodePos					Position of the new node.
	 * @param	bAutoSelect				If true, deselect all expressions and select the newly created one.
	 * @param	bAutoAssignResource		If true, assign resources to new expression.
	 */
	template<typename MaterialExpressionClass>
	static MaterialExpressionClass* CreateNewMaterialExpression(const UEdGraph* Graph, const UE::Slate::FDeprecateVector2DParameter& NodePos, bool bAutoSelect, bool bAutoAssignResource = false)
	{
		static_assert(TIsDerivedFrom<MaterialExpressionClass, UMaterialExpression>::Value, "MaterialExpressionClass needs to inherit from UMaterialExpression");
		return CastChecked<MaterialExpressionClass>(CreateNewMaterialExpression(Graph, MaterialExpressionClass::StaticClass(), NodePos, bAutoSelect, bAutoAssignResource), ECastCheckedType::NullAllowed);
	}
	
	/**
	 * Creates a new material expression composite on the material represented by a graph.
	 *
	 * @param	Graph	Graph to add comment to
	 * @param	NodePos	Position to place new comment at
	 *
	 * @return	UMaterialExpressionComment*	Newly created comment
	 */
	static UE_API UMaterialExpressionComposite* CreateNewMaterialExpressionComposite(const class UEdGraph* Graph, const FVector2D& NodePos);

	/**
	 * Creates a new material expression comment on the material represented by a graph.
	 *
	 * @param	Graph	Graph to add comment to
	 * @param	NodePos	Position to place new comment at
	 *
	 * @return	UMaterialExpressionComment*	Newly created comment
	 */
	static UE_API UMaterialExpressionComment* CreateNewMaterialExpressionComment(const class UEdGraph* Graph, const FVector2D& NodePos);

	/**
	 * Refreshes all material expression previews, regardless of whether or not realtime previews are enabled.
	 *
	 * @param	Graph	Graph representing a material.
	 */
	static UE_API void ForceRefreshExpressionPreviews(const class UEdGraph* Graph);

	/**
	 * Add the specified object to the list of selected objects
	 *
	 * @param	Graph	Graph representing a material.
	 * @param	Obj		Object to add to selection.
	 */
	static UE_API void AddToSelection(const class UEdGraph* Graph, UMaterialExpression* Expression);

	/**
	 * Disconnects and removes the selected material graph nodes.
	 *
	 * @param	Graph	Graph representing a material.
	 */
	static UE_API void DeleteSelectedNodes(const class UEdGraph* Graph);

	/**
	 * Delete the specified nodes from the graph.
	 * @param Graph Graph representing the material.
	 * @param NodesToDelete Array of nodes to be removed from the graph.
	*/
	static UE_API void DeleteNodes(const class UEdGraph* Graph, const TArray<UEdGraphNode*>& NodesToDelete);


	/**
	 * Gets the name of the material or material function that we are editing
	 *
	 * @param	Graph	Graph representing a material or material function.
	 */
	static UE_API FText GetOriginalObjectName(const class UEdGraph* Graph);

	/**
	 * Creates an IAssetReferenceFilter that can be used to check if an asset is valid to be used with this material.
	 	 *
	 * @param	Graph	Graph representing a material or material function.
	 */
	static UE_API TSharedPtr<IAssetReferenceFilter> MakeMaterialAssetReferenceFilter(const class UEdGraph* Graph);

	/**
	 * Re-links the material and updates its representation in the editor
	 *
	 * @param	Graph	Graph representing a material or material function.
	 */
	static UE_API void UpdateMaterialAfterGraphChange(const class UEdGraph* Graph);

	/** Mark the material as dirty. */
	static UE_API void MarkMaterialDirty(const class UEdGraph* Graph);

	static UE_API void UpdateDetailView(const class UEdGraph* Graph);

	/** Can we paste to this graph? */
	static UE_API bool CanPasteNodes(const class UEdGraph* Graph);

	/** Perform paste on graph, at location */
	static UE_API void  PasteNodesHere(class UEdGraph* Graph, const FVector2D& Location);

	/** Gets the number of selected nodes on this graph */
	static UE_API int32 GetNumberOfSelectedNodes(const class UEdGraph* Graph);

	/**
	 * Get all actions for placing material expressions
	 *
	 * @param [in,out]	ActionMenuBuilder	The output menu builder.
	 * @param bMaterialFunction				Whether we're dealing with a Material Function
	 */
	UE_DEPRECATED(5.8, "Should call version that provides a UMaterial or UMaterialFunction")
	static UE_API void GetMaterialExpressionActions(FGraphActionMenuBuilder& ActionMenuBuilder, bool bMaterialFunction);

	/**
	 * Get all actions for placing material expressions
	 *
	 * @param [in,out]	ActionMenuBuilder	The output menu builder.
	 * @param MaterialOrFunction			A UMaterial or UMaterialFunction object
	 */
	static UE_API void GetMaterialExpressionActions(FGraphActionMenuBuilder& ActionMenuBuilder, const UObject* MaterialOrFunction);

	/**
	 * Check whether an expression is in the favourites list
	 *
	 * @param	InExpression	The Expression we are checking
	 */
	static UE_API bool IsMaterialExpressionInFavorites(UMaterialExpression* InExpression);

	/**
	 * Get a preview material render proxy for an expression
	 *
	 * @param	Graph			Graph representing material
	 * @param	InExpression	Expression we want preview for
	 *
	 * @return	FMaterialRenderProxy*	Preview for this expression or NULL
	 */
	static UE_API FMaterialRenderProxy* GetExpressionPreview(const class UEdGraph* Graph, UMaterialExpression* InExpression);

	/**
	 * Updates the material editor search results
	 *
	 * @param	Graph			Graph representing material
	 */
	static UE_API void UpdateSearchResults(const class UEdGraph* Graph);

	/////////////////////////////////////////////////////
	// Static functions moved from SMaterialEditorCanvas

	/**
	 * Retrieves all visible parameters within the material.
	 *
	 * @param	Material			The material to retrieve the parameters from.
	 * @param	MaterialInstance	The material instance that contains all parameter overrides.
	 * @param	VisisbleExpressions	The array that will contain the name's of the visible parameter expressions.
	 */
	static UE_API void GetVisibleMaterialParameters(const UMaterial *Material, UMaterialInstance *MaterialInstance, TArray<FMaterialParameterInfo> &VisibleExpressions);

	/** Finds an input in the passed in array with a matching Id. */
	static UE_API const FFunctionExpressionInput* FindInputById(const UMaterialExpressionFunctionInput* InputExpression, const TArray<FFunctionExpressionInput>& Inputs);

	/** 
	* Returns the value for a static switch material expression.
	*
	* @param	MaterialInstance		The material instance that contains all parameter overrides.
	* @param	SwitchValueExpression	The switch expression to find the value for.
	* @param	OutValue				The value for the switch expression.
	* @param	OutExpressionID			The Guid of the expression that is input as the switch value.
	* @param	FunctionStack			The current function stack frame.
	* 
	* @return	Returns true if a value for the switch expression is found, otherwise returns false.
	*/
	static UE_API bool GetStaticSwitchExpressionValue(UMaterialInstance* MaterialInstance, UMaterialExpression *SwitchValueExpression, bool& OutValue, FGuid& OutExpressionID, TArray<FGetVisibleMaterialParametersFunctionState*>& FunctionStack);

	/**
	 * Populates the specified material's Expressions array (eg if cooked out or old content).
	 * Also ensures materials and expressions are RF_Transactional for undo/redo support.
	 *
	 * @param	Material	Material we are initializing
	 */
	static UE_API void InitExpressions(UMaterial* Material);

	/**
	 * Build the texture streaming data for a given material. Also update the parent hierarchy has only the delta are stored.
	 *
	 * @param	MaterialInterface	The material to update.
	 */
	static UE_API void BuildTextureStreamingData(UMaterialInterface* MaterialInterface);

	/** Get IMaterialEditor for given object, if it exists */
	static UE_API TSharedPtr<class IMaterialEditor> GetIMaterialEditorForObject(const UObject* ObjectToFocusOn);

	/** Get IMaterialEditor to focus on given object, if it exists */
	static UE_API void BringFocusAttentionOnObject(const UObject* ObjectToFocusOn);


	/** Commands for the Parents menu */
	static UE_API void OnOpenMaterial(const FAssetData InMaterial);
	static UE_API void OnOpenFunction(const FAssetData InFunction);
	static UE_API void OnShowMaterialInContentBrowser(const FAssetData InMaterial);
	static UE_API void OnShowFunctionInContentBrowser(const FAssetData InFunction);

	/**
	 * Triggers a refresh or redraw for post process preview materials and material instances.  A refresh is useful when a material generating a UserSceneTexture
	 * output is loaded or unloaded.  Other loaded materials that have UserSceneTexture inputs may include the material in question in their preview, which
	 * necessitates a refresh.  A redraw is useful when debug settings change that may affect post process material viewports.
	 */
	static UE_API void RefreshPostProcessPreviewMaterials(UMaterialInterface* ExcludeMaterialInterface, bool bRedrawOnly = false);

private:

	static UE_API void OpenSelectedParentEditor(UMaterialFunctionInterface* InMaterialFunction);
	static UE_API void OpenSelectedParentEditor(UMaterialInterface* InMaterialInterface);

	/**
	 * Recursively walks the expression tree and parses the visible expression branches.
	 *
	 * @param	MaterialExpression				The expression to parse.
	 * @param	MaterialInstance				The material instance that contains all parameter overrides.
	 * @param	VisisbleExpressions				The array that will contain the name's of the visible parameter expressions.
	 */
	static UE_API void GetVisibleMaterialParametersFromExpression(
		FMaterialExpressionKey MaterialExpressionKey, 
		UMaterialInstance* MaterialInstance, 
		TArray<FMaterialParameterInfo>& VisibleExpressions, 
		TArray<FGetVisibleMaterialParametersFunctionState*>& FunctionStack);

	/**
	 * Adds a category of Material Expressions to an action builder
	 *
	 * @param	ActionMenuBuilder	The builder to add to.
	 * @param	CategoryName		The name of the category.
	 * @param	MaterialExpressions	List of Material Expressions in the category.
	 * @param	bMaterialFunction	Whether we are building for a material function.
	 */
	UE_DEPRECATED(5.8, "Should call version that provides a UMaterial or UMaterialFunction")
	static UE_API void AddMaterialExpressionCategory(FGraphActionMenuBuilder& ActionMenuBuilder, FText CategoryName, TArray<struct FMaterialExpression>* MaterialExpressions, bool bMaterialFunction);

	/**
	* Adds a category of Material Expressions to an action builder
	*
	* @param	ActionMenuBuilder	The builder to add to.
	* @param	CategoryName		The name of the category.
	* @param	MaterialExpressions	List of Material Expressions in the category.
	* @param	MaterialOrFunction	UMaterial or UMaterialFunction object
	*/
	static UE_API void AddMaterialExpressionCategory(FGraphActionMenuBuilder& ActionMenuBuilder, FText CategoryName, TArray<struct FMaterialExpression>* MaterialExpressions, const UObject* MaterialOrFunction);

public:

	/**
	 * Checks whether a Material Expression class has any connections that are compatible with a type/direction
	 *
	 * @param	ExpressionClass		Class of Expression we are testing against.
	 * @param	TestType			Material Value Type we are testing.
	 * @param	TestDirection		Pin Direction we are testing.
	 * @param	bMaterialFunction	Whether we are testing for a material function.
	*/
	UE_DEPRECATED(5.8, "Use the HasCompatibleConnection(..., EMaterialValueType, ...) overload instead")
	static UE_API bool HasCompatibleConnection(UClass* ExpressionClass, uint32 TestType, EEdGraphPinDirection TestDirection, bool bMaterialFunction);
	static UE_API bool HasCompatibleConnection(UClass* ExpressionClass, EMaterialValueType TestType, EEdGraphPinDirection TestDirection, bool bMaterialFunction);

	/** Constructor */
	FMaterialEditorUtilities() {}
};

#undef UE_API
