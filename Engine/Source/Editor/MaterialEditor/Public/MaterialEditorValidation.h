// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "MaterialEditorValidation.generated.h"

#define UE_API MATERIALEDITOR_API

class UMaterialInterface;

/** Context for validators to use/fill during Validate() call. */
struct FMaterialEditorValidatorContext
{
	FText Message;
	FText HyperlinkText;
	FSimpleDelegate HyperlinkAction;
};

/** Base interface for validators that can be registered with the UMaterialEditorValidationSubsystem. */
UCLASS(MinimalAPI)
class UMaterialEditorValidatorBase : public UObject
{
	GENERATED_BODY()

public:
	/** Return false if there are validation errors, and output any validation meesage in InContext. */
	virtual bool Validate(UMaterialInterface* InMaterial, FMaterialEditorValidatorContext& InContext) const { return true; }
};

/** Editor subsystem for registering and running material validators in the material editor tools. */
UCLASS(MinimalAPI, Config = Editor)
class UMaterialEditorValidationSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/** Register a validator. Added validators will be called on Validate(). */
	UE_API void AddValidator(UMaterialEditorValidatorBase* InValidator);
	/** Unregister a validator. */
	UE_API void RemoveValidator(UMaterialEditorValidatorBase* InValidator);
	
	/** Call Validate() on all registered validators.*/
	UE_API bool Validate(UMaterialInterface* InMaterial, FMaterialEditorValidatorContext& InContext) const;

private:
	/** The array of registered validators. */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialEditorValidatorBase>> Validators;
};

#undef UE_API
