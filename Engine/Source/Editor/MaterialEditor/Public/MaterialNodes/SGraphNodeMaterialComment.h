// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "SGraphNodeComment.h"
#include "SNodePanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API MATERIALEDITOR_API

class UMaterialGraphNode_Comment;


template <>
struct TWidgetTypeTraits<class SGraphNodeMaterialComment>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeMaterialComment : public SGraphNodeComment
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMaterialComment){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UMaterialGraphNode_Comment* InNode);

	// SNodePanel::SNode interface
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	// End of SNodePanel::SNode interface

private:
	/** Cached material graph node pointer to avoid casting */
	UMaterialGraphNode_Comment* CommentNode;
};

#undef UE_API
