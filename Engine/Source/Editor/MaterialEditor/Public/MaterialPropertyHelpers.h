// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Layout/Visibility.h"
#include "Materials/MaterialLayersFunctions.h"
#include "IDetailPropertyRow.h"
#include "Input/Reply.h"
#include "Widgets/Layout/SSplitter.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "RenderUtils.h"
#include "MaterialPropertyHelpers.generated.h"

#define UE_API MATERIALEDITOR_API


struct FAssetData;
class IDetailGroup;
class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IDetailTreeNode;
class IPropertyHandle;
class UDEditorParameterValue;
enum class ECheckBoxState : uint8;
class UMaterialInterface;

DECLARE_DELEGATE_OneParam(FGetShowHiddenParameters, bool&);

enum EStackDataType
{
	Stack,
	Asset,
	Group,
	Property,
	PropertyChild,
};

USTRUCT()
struct FSortedParamData
{
	GENERATED_USTRUCT_BODY()

public:
	EStackDataType StackDataType;

	UPROPERTY(Transient)
	TObjectPtr<UDEditorParameterValue> Parameter = nullptr;

	FName PropertyName;

	FEditorParameterGroup Group;

	FMaterialParameterInfo ParameterInfo;

	TSharedPtr<IDetailTreeNode> ParameterNode;

	TSharedPtr<IPropertyHandle> ParameterHandle;

	TArray<TSharedPtr<struct FSortedParamData>> Children;

	FString NodeKey;
};

USTRUCT()
struct FUnsortedParamData
{
	GENERATED_USTRUCT_BODY()
	UPROPERTY(Transient)
	TObjectPtr<UDEditorParameterValue> Parameter = nullptr;
	FEditorParameterGroup ParameterGroup;
	TSharedPtr<IDetailTreeNode> ParameterNode;
	FName UnsortedName;
	TSharedPtr<IPropertyHandle> ParameterHandle;
};

/*
 * Interface for items that can be dragged and can show a layer handle
 */
class IDraggableItem
{
public:
	virtual void OnLayerDragEnter(const FDragDropEvent& DragDropEvent) = 0; 
	virtual void OnLayerDragLeave(const FDragDropEvent& DragDropEvent) = 0;
	virtual void OnLayerDragDetected() = 0;
};

class SLayerHandle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLayerHandle)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(TSharedPtr<IDraggableItem>, OwningStack)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};


	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TSharedPtr<class FLayerDragDropOp> CreateDragDropOperation(TSharedPtr<IDraggableItem> InOwningStack);

private:
	TWeakPtr<IDraggableItem> OwningStack;
};


class FLayerDragDropOp final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FLayerDragDropOp, FDecoratedDragDropOp)

	FLayerDragDropOp(TSharedPtr<IDraggableItem> InOwningStack)
	{
		OwningStack = InOwningStack;
		DecoratorWidget = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("LayerDragDrop", "PlaceLayerHere", "Place Layer and Blend Here"))
				]
			];

		Construct();
	};

	TSharedPtr<SWidget> DecoratorWidget;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return DecoratorWidget;
	}

	TWeakPtr<class IDraggableItem> OwningStack;
};

/** Overridable properties on material parameter property rows. */
struct FMaterialParameterPropertyRowOverrides
{
public:
	/**
	 * Override the default EditConditionValue attribute.
	 * If not specified, FMaterialPropertyHelpers::IsOverriddenExpression will
	 * be used.
	 */
	TAttribute<bool> EditConditionValue;

	/**
	 * Override the default OnEditConditionValueChanged delegate.
	 * If not specified, FMaterialPropertyHelpers::OnOverrideParameter will be
	 * used.
	 */
	TOptional<FOnBooleanValueChanged> OnEditConditionValueChanged;

	/**
	 * Override the default Visibility attribute.
	 * If not specified, FMaterialPropertyHelpers::ShouldShowExpression will be
	 * used.
	 */
	TAttribute<EVisibility> Visibility;

	/**
	 * If Visibility is not overridden, this delegate will be passed as the
	 * last argument to FMaterialPropertyHelpers::ShouldShowExpression.
	 */
	FGetShowHiddenParameters ShowHiddenDelegate;
};

/*-----------------------------------------------------------------------------
   FMaterialInstanceBaseParameterDetails
-----------------------------------------------------------------------------*/

class FMaterialPropertyHelpers
{
public:
	static UE_API bool ShouldCreatePropertyRowForParameter(UDEditorParameterValue* Parameter);
	static UE_API bool ShouldCreateChildPropertiesForParameter(UDEditorParameterValue* Parameter);
	static UE_API void ConfigurePropertyRowForParameter(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorInstanceConstant* MaterialEditorInstance, const FMaterialParameterPropertyRowOverrides& RowOverrides = FMaterialParameterPropertyRowOverrides());

	static UE_API void SetPropertyRowParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterProperty, UMaterialEditorParameters* MaterialEditorParameters);
	static UE_API void SetPropertyRowMaskParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty);
	static UE_API void SetPropertyRowVectorChannelMaskParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters);
	static UE_API void SetPropertyRowScalarAtlasPositionParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters);
	static UE_API void SetPropertyRowScalarEnumerationParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, const TSoftObjectPtr<UObject>& SoftEnumeration);
	static UE_API void SetPropertyRowTextureParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, UMaterialEditorParameters* MaterialEditorParameters);
	static UE_API void SetPropertyRowParameterCollectionParameterWidget(IDetailPropertyRow& PropertyRow, UDEditorParameterValue* Parameter, TSharedPtr<IPropertyHandle> ParameterValueProperty, class UMaterialParameterCollection* BaseParameterCollection, TArray<TSharedRef<IPropertyHandle>>&& AllPropertyHandles);

	/** Returns true if the parameter is being overridden */
	static UE_API bool IsOverriddenExpression(UDEditorParameterValue* Parameter);
	static bool IsOverriddenExpression(TObjectPtr<UDEditorParameterValue> Parameter) { return IsOverriddenExpression(Parameter.Get()); }
	static bool IsOverriddenExpression(TWeakObjectPtr<UDEditorParameterValue> Parameter) { return IsOverriddenExpression(Parameter.Get()); }
	static UE_API ECheckBoxState IsOverriddenExpressionCheckbox(UDEditorParameterValue* Parameter);
	static UE_API bool UsesCustomPrimitiveData(UDEditorParameterValue* Parameter);

	/** Gets the expression description of this parameter from the base material */
	static	UE_API FText GetParameterExpressionDescription(UDEditorParameterValue* Parameter, UObject* MaterialEditorInstance);
	
	static UE_API FText GetParameterTooltip(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance);
	static UE_API FText GetParameterTooltip(UDEditorParameterValue* Parameter, UMaterial* Material);

	UE_DEPRECATED(5.8, "Please use the fully typed method instead.")
	static UE_API FText GetParameterTooltip(UDEditorParameterValue* Parameter, UObject* MaterialEditorInstance);
	/**
	 * Called when a parameter is overridden;
	 */
	static UE_API void OnOverrideParameter(bool NewValue, UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance);

	static UE_API EVisibility ShouldShowExpression(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, FGetShowHiddenParameters ShowHiddenDelegate);
	static EVisibility ShouldShowExpression(TObjectPtr<UDEditorParameterValue> Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, FGetShowHiddenParameters ShowHiddenDelegate) { return ShouldShowExpression(Parameter.Get(), MaterialEditorInstance, ShowHiddenDelegate); }

	/** Generic material property reset to default implementation.  Resets Parameter to default */
	static UE_API FResetToDefaultOverride CreateResetToDefaultOverride(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, TSharedPtr<IPropertyHandle> ParameterValueProperty);
	static UE_API void ResetToDefault(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance, TSharedPtr<IPropertyHandle> ParameterValueProperty);
	static UE_API bool ShouldShowResetToDefault(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance);
	
	/** Specific resets for layer and blend asses */
	static UE_API void ResetLayerAssetToDefault(UDEditorParameterValue* InParameter, TEnumAsByte<EMaterialParameterAssociation> InAssociation, int32 Index, UMaterialEditorInstanceConstant* MaterialEditorInstance);
	/** If reset to default button should show for a layer or blend asset*/
	static UE_API bool ShouldLayerAssetShowResetToDefault(TSharedPtr<FSortedParamData> InParameterData, UMaterialInstanceConstant* InMaterialInstance);
	static bool ShouldLayerAssetShowResetToDefault(TSharedPtr<FSortedParamData> InParameterData, TObjectPtr<UMaterialInstanceConstant> InMaterialInstance) { return ShouldLayerAssetShowResetToDefault(InParameterData, InMaterialInstance.Get()); }

	static UE_API void OnMaterialLayerAssetChanged(const struct FAssetData& InAssetData, int32 Index, EMaterialParameterAssociation MaterialType, TSharedPtr<class IPropertyHandle> InHandle, FMaterialLayersFunctions* InMaterialFunction);

	static UE_API bool FilterLayerAssets(const struct FAssetData& InAssetData, FMaterialLayersFunctions* LayerFunction, EMaterialParameterAssociation MaterialType, int32 Index);

	static UE_API FReply OnClickedSaveNewMaterialInstance(class UMaterialInterface* Object, UObject* EditorObject);
	static FReply OnClickedSaveNewMaterialInstance(TObjectPtr<class UMaterialInterface> Object, UObject* EditorObject) { return OnClickedSaveNewMaterialInstance(Object.Get(), EditorObject); }

	static UE_API void CopyMaterialToInstance(class UMaterialInstanceConstant* ChildInstance, TArray<FEditorParameterGroup> &ParameterGroups);
	static UE_API void TransitionAndCopyParameters(class UMaterialInstanceConstant* ChildInstance, TArray<FEditorParameterGroup> &ParameterGroups, bool bForceCopy = false);
	static UE_API FReply OnClickedSaveNewFunctionInstance(class UMaterialFunctionInterface* Object, class UMaterialInterface* PreviewMaterial, UObject* EditorObject);
	static UE_API FReply OnClickedSaveNewLayerInstance(class UMaterialFunctionInterface* Object, TSharedPtr<FSortedParamData> InSortedData);

	static UE_API void GetVectorChannelMaskComboBoxStrings(TArray<TSharedPtr<FString>>& OutComboBoxStrings, TArray<TSharedPtr<class SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems);
	static UE_API FString GetVectorChannelMaskValue(UDEditorParameterValue* InParameter);
	static UE_API void SetVectorChannelMaskValue(const FString& StringValue, TSharedPtr<IPropertyHandle> PropertyHandle, UDEditorParameterValue* InParameter, UObject* MaterialEditorInstance);

	static UE_API TArray<class UFactory*> GetAssetFactories(EMaterialParameterAssociation AssetType);
	/**
	*  Returns group for parameter. Creates one if needed.
	*
	* @param ParameterGroup		Name to be looked for.
	*/
	static UE_API FEditorParameterGroup&  GetParameterGroup(class UMaterial* InMaterial, FName& ParameterGroup, TArray<FEditorParameterGroup>& ParameterGroups);

	static UE_API TSharedRef<SWidget> MakeStackReorderHandle(TSharedPtr<IDraggableItem> InOwningStack);

	static UE_API bool OnShouldSetCurveAsset(const FAssetData& AssetData, TSoftObjectPtr<UCurveLinearColorAtlas> InAtlas);
	static UE_API bool OnShouldFilterCurveAsset(const FAssetData& AssetData, TSoftObjectPtr<UCurveLinearColorAtlas> InAtlas);
	static UE_API void SetPositionFromCurveAsset(const FAssetData& AssetData, TSoftObjectPtr<UCurveLinearColorAtlas> InAtlas, class UDEditorScalarParameterValue* InParameter, TSharedPtr<IPropertyHandle> PropertyHandle, UObject* MaterialEditorInstance);

	static UE_API void ResetCurveToDefault(UDEditorParameterValue* Parameter, UMaterialEditorInstanceConstant* MaterialEditorInstance);

	static  TSharedRef<SWidget> CreateScalarEnumerationParameterValueWidget(TSoftObjectPtr<UObject> const& SoftEnumeration, TSharedPtr<IPropertyHandle> ParameterValueProperty);

	static UE_API FText LayerID;
	static UE_API FText BlendID;
	static UE_API FName LayerParamName;
};

#undef UE_API
