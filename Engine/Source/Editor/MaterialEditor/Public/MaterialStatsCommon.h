// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SceneTypes.h"
#include "RHIDefinitions.h"
#include "MaterialShared.h"
#include "Styling/SlateColor.h"

#define UE_API MATERIALEDITOR_API

/** custom resource material class used to mark the resource as used for shader stats extraction */
class FMaterialResourceStats : public FMaterialResource
{
public:
	FMaterialResourceStats() {}
	virtual ~FMaterialResourceStats() = default;

	FORCEINLINE UMaterial* GetMaterial()
	{
		return Material;
	}

	/**
	 * Should shaders compiled for this material be saved to disk?
	 */
	virtual bool IsPersistent() const override { return false; }

	/** Use preview settings when compiling material. Used to avoid culling nodes behind landscape weights. */
	virtual bool IsPreview() const override { return true; }

	/** Material resource stats never requires a synchronous compilation, otherwise opening up empty (newly created) material instance will block compiling default mat's shaders. */
	virtual bool RequiresSynchronousCompilation() const override { return false; }

	/** this will enable shader source extraction and pass paths to (eventual) offline shader compilers */
	UE_API virtual void SetupExtraCompilationSettings(FExtraShaderCompilerSettings& Settings) const override;
};

/** enumeration used to group shader platforms */
enum class EPlatformCategoryType : int32
{
	Desktop,
	Android,
	IOS,
	Console,

	Num
};

/** enumeration containing the "types" of used shaders to display statistics  */
enum class ERepresentativeShader
{
	FirstFragmentShader,
	StationarySurface = FirstFragmentShader,
	StationarySurfaceCSM,
	StationarySurfaceNPointLights,
	DynamicallyLitObject,
	RuntimeVirtualTextureOutput,
	UIDefaultFragmentShader,
	UIDynamicFragmentShader,
	LastFragmentShader = UIDynamicFragmentShader,

	FirstVertexShader,
	StaticMesh = FirstVertexShader,
	SkeletalMesh,
	SkinnedCloth,

	UIDefaultVertexShader,
	UIInstancedVertexShader,
	LastVertexShader = UIInstancedVertexShader,

	NaniteMesh,

	Num
};

/** class used for various stats utilities */
class FMaterialStatsUtils
{
public:
	struct FShaderInstructionsInfo
	{
		ERepresentativeShader ShaderType;
		FString ShaderDescription;
		FString ShaderStatisticsString;
		int32 InstructionCount;
	};

	struct FRepresentativeShaderInfo
	{
		ERepresentativeShader ShaderType;
		FName ShaderName;
		FString ShaderDescription;

		FRepresentativeShaderInfo(const ERepresentativeShader _ShaderType, const FName _ShaderName, const FString& _StrDescription) :
			ShaderType(_ShaderType), ShaderName(_ShaderName), ShaderDescription(_StrDescription)
		{}
	};

public:
	/** call this to create an instance to FMaterialStats */
	static TSharedPtr<class FMaterialStats> CreateMaterialStats(class IMaterialEditor* MaterialEditor, const bool bShowMaterialInstancesMenu, const bool bAllowIgnoringCompilationErrors);

	/** utility functions that translate various enum values to strings */
	static FString MaterialQualityToString(const EMaterialQualityLevel::Type Quality);
	static FString MaterialQualityToShortString(const EMaterialQualityLevel::Type Quality);
	static EMaterialQualityLevel::Type StringToMaterialQuality(const FString& StrQuality);

	static FString GetPlatformTypeName(const EPlatformCategoryType InEnumValue);
	static FString ShaderPlatformTypeName(const EShaderPlatform PlatformID);

	/**
	* Gets instruction counts that best represent the likely usage of this material based on shading model and other factors.
	* @param Results - an array of descriptions to be populated
	*/
	static void GetRepresentativeInstructionCounts(TArray<FShaderInstructionsInfo>& Results, const class FMaterialResource* Target);

	MATERIALEDITOR_API static void GetRepresentativeShaderTypesAndDescriptions(TMap<FName, TArray<FRepresentativeShaderInfo>>& OutShaderTypeNameAndDescriptions, const class FMaterial* TargetMaterial);
	MATERIALEDITOR_API static void ExtractMatertialStatsInfo(EShaderPlatform ShaderPlatform, struct FShaderStatsInfo& OutInfo, const FMaterialResource* Target);

	static FString RepresentativeShaderTypeToString(const ERepresentativeShader ShaderType);

	static FSlateColor QualitySettingColor(const EMaterialQualityLevel::Type QualityType);
	static FSlateColor PlatformTypeColor(EPlatformCategoryType PlatformType);

	MATERIALEDITOR_API static bool IsPlatformOfflineCompilerAvailable(const EShaderPlatform ShaderPlatform);
	UE_DEPRECATED(5.5, "GetPlatformOfflineCompilerPath is no longer used; Use GetPlatformOfflineCompilerSettings instead.")
	MATERIALEDITOR_API static FString GetPlatformOfflineCompilerPath(const EShaderPlatform ShaderPlatform);
	MATERIALEDITOR_API static void GetPlatformOfflineCompilerSettings(const EShaderPlatform ShaderPlatform, FExtraShaderCompilerSettings& SCSettings);
	MATERIALEDITOR_API static bool PlatformNeedsOfflineCompiler(const EShaderPlatform ShaderPlatform);
};

#undef UE_API
